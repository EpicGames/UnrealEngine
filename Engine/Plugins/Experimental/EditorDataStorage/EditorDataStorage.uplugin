{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "TEDS: Editor Data Storage",
	"Description": "A central extendable data storage for editors and their corresponding data with support for viewing and editing through a collection of widgets.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "TedsCore",
			"Type": "UncookedOnly"
		},
		{
			"Name": "TedsUI",
			"Type": "Editor"
		}
	],
	"Plugins": [
	],
	"SupportedPrograms": ["ChaosVisualDebugger"]
}