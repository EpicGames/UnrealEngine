// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Misc/TVariant.h"
#include "Templates/Function.h"
#include "TypedElementDatabaseScratchBuffer.h"

namespace UE::Editor::DataStorage
{
	/** Empty command that does nothing. Can be used as placeholder or to disable an existing command. */
	struct FNopCommand final {};

	/**
	 * Class to store, optimize and process commands. Only the list of provided command objects can be used or TNopCommand which will is 
	 * automatically added. Commands take the form of a struct with any number, including zero, member variables.
	 * The typical use case of the command buffer is to repeatedly call `AddCommand`. This can be safely done from any thread. Periodically,
	 * e.g. once per frame, `Collect` is called which collects all commands from all threads into a single local collection. This stores
	 * references to the commands so doesn't move or copy the commands out of the scratch buffer where commands are created. Keep the
	 * lifetime of objects in the scratch buffer in mind.
	 * The returned collection can be further processed. This typically involves steps such as patching commands e.g. by resolving the
	 * table index, sort commands in order, optimize commands to remove/replace/add commands as needed and finally the commands are
	 * processed so they get executed. A utility class is also provided to help with optimizing the command buffer.
	 */
	template<typename... TCommand>
	class FCommandBuffer final
	{
	public:
		// Note that FNopCommand is always inserted at index 0. This means that any code using sizeof...(TCommand) needs to add 1 to the count.
		using TCommandVariant = TVariant<FNopCommand, TCommand...>;

		/** Storage for commands collected from all threads that haven't been executed yet. */
		class FCollection final
		{
			friend class FCommandBuffer;
			friend class FOptimizer;
		public:
			FCollection();

			/** Adds a new command at the end of the command collection. */
			template<typename T>
			void AddCommand();
			/** Adds a new command at the end of the command collection. */
			template<typename T>
			void AddCommand(T&& Command);

			/**
			 * Replaces the command at the given index. Prefer this over directly replacing the command variant to make sure tracking
			 * in the buffer remains consistent.
			 */
			template<typename T>
			void ReplaceCommand(int32 Index);
			/**
			 * Replaces the command at the given index. Prefer this over directly replacing the command variant to make sure tracking
			 * in the buffer remains consistent.
			 */
			template<typename T>
			void ReplaceCommand(int32 Index, T&& Command);

			/** Inserts a new command before the provided index, moving the command at the given index one position up. */
			template<typename T>
			void InsertCommandBefore(int32 Index);

			/** Inserts a new command before the provided index, moving the command at the given index one position up. */
			template<typename T>
			void InsertCommandBefore(int32 Index, T&& Command);

			/**
			 * Iterates over all collected commands, allowing commands to be updated.
			 * Do not use this function to replace a command with new one as this will cause internal counters to mismatch.
			 */
			void ForEach(TFunctionRef<void(int32 Index, TCommandVariant& Command)> Iterator);

			/** Sorts commands using the provided compare function. */
			template<bool bStableSort = true>
			void Sort(TFunctionRef<bool(const TCommandVariant& Lhs, const TCommandVariant& Rhs)> Comparer);

			/**
			 * Processes the local commands, that are retrieved after calling Collect or that still remain in the local queue. The
			 * provided can be any object that has functions that can receive a comment. As an example
			 *		struct FProcessor{ void operator()(const TNopCommand&) {} };
			 */
			template<typename TProcessor>
			void Process(TProcessor&& Processor) const;

			/** Resets the locally stored information, effectively clearing the command buffer until the next collect is called. */
			void Reset();

			/** Returns the number of instances of a command that are locally queued. */
			template<typename Command>
			uint32 GetCommandCount() const;

			/** Returns the total number of commands, excluding nop operations. */
			uint32 GetTotalCommandCount() const;
		private:
			TArray<TCommandVariant*> CommandReferences;
			uint32 CommandCount[sizeof...(TCommand) + 1];
			FScratchBuffer* ScratchBuffer = nullptr;
		};

		/** Utility class to help optimize a collection of commands. */
		class FOptimizer
		{
		public:
			using IsValidCallback = TFunction<bool(const TCommandVariant& Command)>;

			explicit FOptimizer(FCommandBuffer::FCollection& InCommands);

			TCommandVariant& GetLeft();
			TCommandVariant& GetRight();

			/**
			 * Moves the left index one command to the right. If the right index is on the same command, right will also be moved one 
			 * position to the right.
			 * Before: 0 | L1 | R2 | 3  | 4
			 * After:  0 | 1  | L2 | R3 | 4
			 */
			void MoveToNextLeft();
			/**
			 * Move the right index one command to the right
			 * Before: 0 | L1 | R2 | 3  | 4
			 * After:  0 | L1 | 2  | R3 | 4
			 */
			void MoveToNextRight();
			/** 
			 * Moves both the left and right index one command to the right
			 * Before: 0 | L1 | 2  | R3 | 4
			 * After:  0 | 1  | L2 | 3  | R4
			 */
			void MoveToNextLeftAndRight();
			/**
			 * Resets the right index to be one next to the left index.
			 * Before: 0 | L1 | 2  | 3 | R4
			 * After:  0 | L1 | R2 | 3 | 4
			 */
			void ResetRightNextToLeft();
			/** 
			 * Moves the left index to one less than the right index.
			 * Before: L0 | 1 | 2  | R3 | 4
			 * After:  0  | 1 | L2 | R3 | 4
			 */
			void MoveLeftBeforeRight();
			/** 
			 * Moves the left index to the right index and moves the right index one to the right.
			 * Before: L0 | 1 | 2 | R3 | 4
			 * After:  0  | 1 | 2 | L3 | R4
			 */
			void MoveLeftToRight();

			/** Replaces the command at the left index with a new command. */
			template<typename T>
			void ReplaceLeft(T&& Command);
			/** Replaces the command at the left index with a new command. */
			template<typename T>
			void ReplaceLeft();
			/** Replaces the command at the right index with a new command. */
			template<typename T>
			void ReplaceRight(T&& Command);
			/** Replaces the command at the right index with a new command. */
			template<typename T>
			void ReplaceRight();

			/**
			 * Inserts a new command before the command at the left index.
			 * Before: 10 | L20 | 30  | R40 | 50
			 * After:  10 | 15  | L20 | 30  | R40 | 50
			 */
			template<typename T>
			void InsertBeforeLeft(T&& Command);
			/**
			 * Inserts a new command before the command at the left index.
			 * Before: 10 | L20 | 30  | R40 | 50
			 * After:  10 | 15  | L20 | 30  | R40 | 50
			 */
			template<typename T>
			void InsertBeforeLeft();

			/** Creates a new optimizer starting at the left of the current optimizer. */
			FOptimizer BranchOnLeft();
			/** Creates a new optimizer starting at the left of the current optimizer. The provides callback is used to check range. */
			FOptimizer BranchOnLeft(IsValidCallback Callback);
			/** Creates a new optimizer starting at the right of the current optimizer. */
			FOptimizer BranchOnRight();
			/** Creates a new optimizer starting at the right of the current optimizer. The provides callback is used to check range. */
			FOptimizer BranchOnRight(IsValidCallback Callback);

			bool IsValid() const;

		private:
			FOptimizer(FCommandBuffer::FCollection& InCommands, int32 InLeft);
			FOptimizer(FCommandBuffer::FCollection& InCommands, int32 InLeft, IsValidCallback InCallback);

			int32 MoveToNextNonNop(int32 Location);
			int32 MoveToPreviousNonNop(int32 Location);

			IsValidCallback IsValidCheck;
			int32 Left;
			int32 Right;
			FCommandBuffer::FCollection& Commands;
		};

		void Initialize(FScratchBuffer& InScratchBuffer);

		/** Adds a command the command buffer in a thread-safe manner. T has to be one of the provided commands or TNopCommand. */
		template<typename T>
		void AddCommand();

		/** Adds a command the command buffer in a thread-safe manner. T has to be one of the provided commands or TNopCommand. */
		template<typename T>
		void AddCommand(T&& Command);

		/**
		 * Collects commands from all thread locally for further processing. This needs to be called every frame as commands are
		 * stored in a temporary buffer that will get cleared after the frame.
		 * Returns the number of collected commands.
		 */
		SIZE_T Collect(FCollection& Storage);

	private:
		// Wrapper object used to store a command in the scratch buffer.
		struct FCommandInstance final
		{
			FCommandInstance() = default;
			template <typename U, typename... TArgs>
			explicit FCommandInstance(TInPlaceType<U>&&, TArgs&&... Args);

			TCommandVariant Command;
			std::atomic<FCommandInstance*> Next = nullptr;
		};
		FCommandInstance* CommandFront = nullptr;
		std::atomic<FCommandInstance*> CommandBack = nullptr;

		FScratchBuffer* ScratchBuffer = nullptr;
	};
} // namespace UE::Editor::DataStorage

#include "Commands/EditorDataStorageCommandBuffer.inl"