// Copyright Epic Games, Inc. All Rights Reserved.

#include "Algo/BinarySearch.h"
#include "HAL/UnrealMemory.h"

namespace UE::Editor::DataStorage
{
	//
	// FCommandBuffer::FCollection
	//

	template<typename... TCommand>
	FCommandBuffer<TCommand...>::FCollection::FCollection()
	{
		FMemory::Memset(CommandCount, 0);
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::AddCommand()
	{
		CommandReferences.Add(ScratchBuffer->Emplace<TCommandVariant>(TInPlaceType<T>()));
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::AddCommand(T&& Command)
	{
		CommandReferences.Add(ScratchBuffer->Emplace<TCommandVariant>(TInPlaceType<T>(), Forward<T>(Command)));
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::ReplaceCommand(int32 Index)
	{
		checkf(Index < CommandReferences.Num(), TEXT("Attempting to replace a command which is not in a valid position."));

		TCommandVariant* OriginalCommand = CommandReferences[Index];
		ensureMsgf(CommandCount[OriginalCommand->GetIndex()]-- > 0,
			TEXT("A command is being replaced in TEDS command buffer isn't matching the expected count."));
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
		OriginalCommand->template Emplace<T>();
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::ReplaceCommand(int32 Index, T&& Command)
	{
		checkf(Index < CommandReferences.Num(), TEXT("Attempting to replace a command which is not in a valid position."));

		TCommandVariant* OriginalCommand = CommandReferences[Index];
		ensureMsgf(CommandCount[OriginalCommand->GetIndex()]-- > 0,
			TEXT("A command is being replaced in TEDS command buffer isn't matching the expected count."));
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
		OriginalCommand->template Emplace<T>(Forward<T>(Command));
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::InsertCommandBefore(int32 Index)
	{
		TCommandVariant* CommandStorage = ScratchBuffer->Emplace<TCommandVariant>(TInPlaceType<T>());
		CommandReferences.Insert(CommandStorage, Index);
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FCollection::InsertCommandBefore(int32 Index, T&& Command)
	{
		TCommandVariant* CommandStorage = ScratchBuffer->Emplace<TCommandVariant>(TInPlaceType<T>(), Forward<T>(Command));
		CommandReferences.Insert(CommandStorage, Index);
		CommandCount[TCommandVariant::template IndexOfType<T>()]++;
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FCollection::ForEach(TFunctionRef<void(int32 Index, TCommandVariant& Command)> Iterator)
	{
		// Don't use a local iterator or cache the size as the array can change during processing.
		for (int32 Index = 0; Index < CommandReferences.Num(); ++Index)
		{
			Iterator(Index, *CommandReferences[Index]);
		}
	}

	template<typename... TCommand>
	template<bool bStableSort>
	void FCommandBuffer<TCommand...>::FCollection::Sort(TFunctionRef<bool(const TCommandVariant&, const TCommandVariant&)> Comparer)
	{
		if constexpr (bStableSort)
		{
			CommandReferences.StableSort(Comparer);
		}
		else
		{
			CommandReferences.Sort(Comparer);
		}
	}

	template<typename... TCommand>
	template<typename TProcessor>
	void FCommandBuffer<TCommand...>::FCollection::Process(TProcessor&& Processor) const
	{
		for (TCommandVariant* Command : CommandReferences)
		{
			Visit(Processor, *Command);
		}
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FCollection::Reset()
	{
		CommandReferences.Reset();
		FMemory::Memset(CommandCount, 0);
	}

	template<typename... TCommand>
	template<typename Command>
	uint32 FCommandBuffer<TCommand...>::FCollection::GetCommandCount() const
	{
		return CommandCount[FCommandBuffer::TCommandVariant::template IndexOfType<Command>()];
	}

	template<typename... TCommand>
	uint32 FCommandBuffer<TCommand...>::FCollection::GetTotalCommandCount() const
	{
		uint32 Result = 0;
		for (uint32 Index = 1; Index < sizeof...(TCommand) + 1; ++Index)
		{
			Result += CommandCount[Index];
		}
		return Result;
	}


	//
	// FCommandBuffer::FOptimizer
	//

	template<typename... TCommand>
	FCommandBuffer<TCommand...>::FOptimizer::FOptimizer(FCommandBuffer::FCollection& InCommands)
		: Left(0)
		, Right(1)
		, Commands(InCommands)
	{
		if (!Commands.CommandReferences.IsEmpty() && Commands.CommandReferences[0]->template IsType<FNopCommand>())
		{
			Left = MoveToNextNonNop(0);
			Right = MoveToNextNonNop(Left);
		}
		else if (Commands.CommandReferences.IsValidIndex(1) && Commands.CommandReferences[1]->template IsType<FNopCommand>())
		{
			Right = MoveToNextNonNop(1);
		}
	}

	template<typename... TCommand>
	FCommandBuffer<TCommand...>::FOptimizer::FOptimizer(FCommandBuffer::FCollection& InCommands, int32 InLeft)
		: Left(InLeft)
		, Right(InLeft + 1)
		, Commands(InCommands)
	{
		if (Commands.CommandReferences.IsValidIndex(Left) && Commands.CommandReferences[Left]->template IsType<FNopCommand>())
		{
			Left = MoveToNextNonNop(Left);
			Right = MoveToNextNonNop(Left);
		}
		else if (Commands.CommandReferences.IsValidIndex(Right) && Commands.CommandReferences[Right]->template IsType<FNopCommand>())
		{
			Right = MoveToNextNonNop(Right);
		}
	}

	template<typename... TCommand>
	FCommandBuffer<TCommand...>::FOptimizer::FOptimizer(FCommandBuffer::FCollection& InCommands, int32 InLeft, IsValidCallback InCallback)
		: IsValidCheck(MoveTemp(InCallback))
		, Left(InLeft)
		, Right(InLeft + 1)
		, Commands(InCommands)
	{
		if (Commands.CommandReferences.IsValidIndex(Left) && Commands.CommandReferences[Left]->template IsType<FNopCommand>())
		{
			Left = MoveToNextNonNop(Left);
			Right = MoveToNextNonNop(Left);
		}
		else if (Commands.CommandReferences.IsValidIndex(Right) && Commands.CommandReferences[Right]->template IsType<FNopCommand>())
		{
			Right = MoveToNextNonNop(Right);
		}
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::TCommandVariant& FCommandBuffer<TCommand...>::FOptimizer::GetLeft()
	{
		checkf(Left < Commands.CommandReferences.Num(), TEXT("Attempting to access an invalid left command from a command buffer."));
		return *Commands.CommandReferences[Left];
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::TCommandVariant& FCommandBuffer<TCommand...>::FOptimizer::GetRight()
	{
		checkf(Right < Commands.CommandReferences.Num(), TEXT("Attempting to access an invalid right command from a command buffer."));
		return *Commands.CommandReferences[Right];
	}

	template<typename... TCommand>
	int32 FCommandBuffer<TCommand...>::FOptimizer::MoveToNextNonNop(int32 Location)
	{
		int32 CommandCount = Commands.CommandReferences.Num();
		do
		{
			++Location;
		} while (Location < CommandCount && Commands.CommandReferences[Location]->template IsType<FNopCommand>());
		return Location;
	}

	template<typename... TCommand>
	int32 FCommandBuffer<TCommand...>::FOptimizer::MoveToPreviousNonNop(int32 Location)
	{
		while (Location > 0)
		{
			--Location;
			if (!Commands.CommandReferences[Location]->template IsType<FNopCommand>())
			{
				break;
			}
		}
		return Location;
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::MoveToNextLeft()
	{
		Left = MoveToNextNonNop(Left);
		if (Left == Right)
		{
			Right = MoveToNextNonNop(Right);
		}
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::MoveToNextRight()
	{
		Right = MoveToNextNonNop(Right);
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::MoveToNextLeftAndRight()
	{
		Left = MoveToNextNonNop(Left);
		Right = MoveToNextNonNop(Right);
		if (Right == Left)
		{
			Right = MoveToNextNonNop(Right);
		}
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::ResetRightNextToLeft()
	{
		Right = Left;
		Right = MoveToNextNonNop(Right);
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::MoveLeftBeforeRight()
	{
		Left = MoveToPreviousNonNop(Right);
	}

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::FOptimizer::MoveLeftToRight()
	{
		Left = Right;
		Right = MoveToNextNonNop(Right);
	}

	template<typename... TCommand>
	bool FCommandBuffer<TCommand...>::FOptimizer::IsValid() const
	{
		if (!IsValidCheck)
		{
			return Right < Commands.CommandReferences.Num();
		}
		else
		{
			return Right < Commands.CommandReferences.Num() ? IsValidCheck(*Commands.CommandReferences[Right]) : false;
		}
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::ReplaceLeft(T&& Command)
	{
		Commands.ReplaceCommand(Left, Forward<T>(Command));
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::ReplaceLeft()
	{
		Commands.template ReplaceCommand<T>(Left);
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::ReplaceRight(T&& Command)
	{
		Commands.ReplaceCommand(Right, Forward<T>(Command));
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::ReplaceRight()
	{
		Commands.template ReplaceCommand<T>(Right);
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::InsertBeforeLeft(T&& Command)
	{
		Commands.InsertCommandBefore(Left, Forward<T>(Command));
		++Left; // Move left to stay on the same object as before the insert.
		++Right; // Move right one up as well so it stays on the same object.
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::FOptimizer::InsertBeforeLeft()
	{
		Commands.template InsertCommandBefore<T>(Left);
		++Left; // Move left to stay on the same object as before the insert.
		++Right; // Move right one up as well so it stays on the same object.
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::FOptimizer FCommandBuffer<TCommand...>::FOptimizer::BranchOnLeft()
	{
		return FOptimizer(Commands, Left);
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::FOptimizer FCommandBuffer<TCommand...>::FOptimizer::BranchOnLeft(IsValidCallback Callback)
	{
		return FOptimizer(Commands, Left, MoveTemp(Callback));
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::FOptimizer FCommandBuffer<TCommand...>::FOptimizer::BranchOnRight()
	{
		return FOptimizer(Commands, Right);
	}

	template<typename... TCommand>
	typename FCommandBuffer<TCommand...>::FOptimizer FCommandBuffer<TCommand...>::FOptimizer::BranchOnRight(IsValidCallback Callback)
	{
		return FOptimizer(Commands, Right, MoveTemp(Callback));
	}




	//
	// FCommandBuffer::FCommandInstance
	//

	template<typename... TCommand>
	template <typename U, typename... TArgs>
	FCommandBuffer<TCommand...>::FCommandInstance::FCommandInstance(TInPlaceType<U>&&, TArgs&&... Args)
		: Command(TInPlaceType<U>(), Forward<TArgs>(Args)...)
	{
	}



	//
	// FCommandBuffer
	//

	template<typename... TCommand>
	void FCommandBuffer<TCommand...>::Initialize(FScratchBuffer& InScratchBuffer)
	{
		ScratchBuffer = &InScratchBuffer;
		CommandFront = ScratchBuffer->Emplace<FCommandInstance>();
		CommandBack = CommandFront;
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::AddCommand()
	{
		checkf(ScratchBuffer, TEXT("Attempting to add a command to the command buffer before it's been initialized."));

		FCommandInstance* AllocatedCommand = ScratchBuffer->Emplace<FCommandInstance>(TInPlaceType<T>());
		FCommandInstance* Previous = CommandBack.load();
		while (!CommandBack.compare_exchange_weak(Previous, AllocatedCommand)) {}
		Previous->Next = AllocatedCommand;
	}

	template<typename... TCommand>
	template<typename T>
	void FCommandBuffer<TCommand...>::AddCommand(T&& Command)
	{
		checkf(ScratchBuffer, TEXT("Attempting to add a command to the command buffer before it's been initialized."));

		FCommandInstance* AllocatedCommand = ScratchBuffer->Emplace<FCommandInstance>(TInPlaceType<T>(), Forward<T>(Command));
		FCommandInstance* Previous = CommandBack.load();
		while (!CommandBack.compare_exchange_weak(Previous, AllocatedCommand)){}
		Previous->Next = AllocatedCommand;
	}

	template<typename... TCommand>
	SIZE_T FCommandBuffer<TCommand...>::Collect(FCollection& Storage)
	{
		checkf(ScratchBuffer, TEXT("Attempting to collect the command buffer before it's been initialized."));
		Storage.ScratchBuffer = ScratchBuffer;

		// Claim the command buffer by replacing it with a fresh one.
		FCommandInstance* NewCommandFront = ScratchBuffer->Emplace<FCommandInstance>();
		FCommandInstance* CurrentBack = CommandBack.load();
		while (!CommandBack.compare_exchange_weak(CurrentBack, NewCommandFront)) {}

		// Walk the list of commands and copy them locally. Skip the first entry as it will always be a no-op.
		checkf(CommandFront->Command.template IsType<FNopCommand>(),
			TEXT("The first operation in TEDS' command buffers should always be a no-op. ")
			TEXT("As this is not the case, the buffer may have gotten corrupted."));
		SIZE_T Result = 0;
		FCommandInstance* Front = CommandFront->Next;
		while (Front)
		{
			Storage.CommandReferences.Add(&Front->Command);
			Storage.CommandCount[Front->Command.GetIndex()]++;
			Front = Front->Next;
			++Result;
		}
		CommandFront = NewCommandFront;

		return Result;
	}
} // namespace UE::Editor::DataStorage