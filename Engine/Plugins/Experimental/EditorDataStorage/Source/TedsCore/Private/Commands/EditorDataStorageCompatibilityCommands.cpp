// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/EditorDataStorageCompatibilityCommands.h"

#include "Algo/BinarySearch.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Memento/TypedElementMementoSystem.h"
#include "TypedElementDatabaseCompatibility.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementDatabaseScratchBuffer.h"

namespace UE::Editor::DataStorage
{
	const UScriptStruct* FAddSyncFromWorldTag::GetType()
	{
		return *GetTypeAddress();
	}

	const UScriptStruct** FAddSyncFromWorldTag::GetTypeAddress()
	{
		static const UScriptStruct* Type = FTypedElementSyncFromWorldTag::StaticStruct();
		return &Type;
	}

	const UScriptStruct* FAddInteractiveSyncFromWorldTag::GetType()
	{
		return *GetTypeAddress();
	}

	const UScriptStruct** FAddInteractiveSyncFromWorldTag::GetTypeAddress()
	{
		static const UScriptStruct* Type = FTypedElementSyncFromWorldInteractiveTag::StaticStruct();
		return &Type;
	}

	//
	// FObjectTypeInfo
	//

	FObjectTypeInfo::FObjectTypeInfo(const UScriptStruct* InScriptStruct)
		: TypeInfoType(EObjectType::Struct)
		, ScriptStruct(InScriptStruct)
	{}

	FObjectTypeInfo::FObjectTypeInfo(const UClass* InClass)
		: TypeInfoType(EObjectType::Class)
		, Class(InClass)
	{}

	FName FObjectTypeInfo::GetFName() const
	{
		switch (TypeInfoType)
		{
		case EObjectType::Struct: return ScriptStruct->GetFName();
		case EObjectType::Class: return Class->GetFName();
		default: return FName();
		}
	}



	//
	// FTypeBatchInfoReinstanced
	//

	template<typename T>
	FTypeInfoReinstanced* FTypeBatchInfoReinstanced::FindObject(TArrayView<FTypeInfoReinstanced> Range, const TWeakObjectPtr<T>& Object)
	{
		// If performance of this suffers too much due to the hash having too many collisions moving to an FObjectKey might be more
		// more efficient as there will be guaranteed no collisions. However there's currently no way to go from a TWeakObjectPtr
		// to a TObjectKey<UObject> or FObjectKey.

		uint32 TargetHash = Object.GetWeakPtrTypeHash();
		int32 FoundIndex = Algo::LowerBoundBy(Range, TargetHash,
			[](FTypeInfoReinstanced& Reinstanced)
			{
				return Reinstanced.Original.GetWeakPtrTypeHash();
			});
		// Keep searching linearly since the hash for weak pointers has a high chance of collisions.
		FTypeInfoReinstanced* Result = Range.GetData() + FoundIndex;
		FTypeInfoReinstanced* End = Range.end();
		while (Result < End
			&& Result->Original.GetWeakPtrTypeHash() == TargetHash
			&& !Result->Original.HasSameIndexAndSerialNumber(Object))
		{
			Result++;
		}
		if (Result < End && Result->Original.HasSameIndexAndSerialNumber(Object))
		{
			return Result;
		}
		return nullptr;
	}

	template<typename T>
	TWeakObjectPtr<T> FTypeBatchInfoReinstanced::FindObjectRecursively(
		TArrayView<FTypeInfoReinstanced> Range, const TWeakObjectPtr<T>& Object)
	{
		if (FTypeInfoReinstanced* NewObject = FindObject(Range, Object))
		{
			FTypeInfoReinstanced* LastNewObject = NewObject;
			while (FTypeInfoReinstanced* NextNewObject = FindObject(Range, LastNewObject->Reinstanced))
			{
				LastNewObject = NextNewObject;
			}
			return Cast<T>(LastNewObject->Reinstanced);
		}
		return Object;
	}



	//
	// FPatchData
	//

	FPatchData::FPatchCommand::FPatchCommand(TArrayView<FTypeInfoReinstanced>& InReinstances)
		: Reinstances(InReinstances)
	{
	}

	void FPatchData::FPatchCommand::operator()(FRegisterTypeTableAssociation& Command)
	{
		Command.TypeInfo = FTypeBatchInfoReinstanced::FindObjectRecursively(Reinstances, Command.TypeInfo);
	}

	void FPatchData::FPatchCommand::operator()(FAddCompatibleExternalObject& Command)
	{
		Command.TypeInfo = FTypeBatchInfoReinstanced::FindObjectRecursively(Reinstances, Command.TypeInfo);
		checkf(Command.TypeInfo.IsValid(), TEXT("A script struct has be re-instanced to an object that's not a script struct."))
	}

	void FPatchData::FPatchCommand::operator()(FBatchAddCompatibleExternalObject& Command)
	{
		TArrayView<TWeakObjectPtr<const UScriptStruct>> TypeInfoArray = 
			TArrayView<TWeakObjectPtr<const UScriptStruct>>(Command.TypeInfoArray, Command.Count);

		for (TWeakObjectPtr<const UScriptStruct>& TypeInfo : TypeInfoArray)
		{
			TypeInfo = FTypeBatchInfoReinstanced::FindObjectRecursively(Reinstances, TypeInfo);
			checkf(TypeInfo.IsValid(), TEXT("A script struct has be re-instanced to an object that's not a script struct."))
		}
	}


	bool FPatchData::IsPatchingRequired(const CompatibilityCommandBuffer::FCollection& Commands)
	{
		return Commands.GetCommandCount<FTypeInfoReinstanced>() > 0;
	}

	void FPatchData::RunPatch(CompatibilityCommandBuffer::FCollection& Commands, UEditorDataStorageCompatibility& StorageCompat,
		FScratchBuffer& ScratchBuffer)
	{
		uint32 ReinstanceCount = Commands.GetCommandCount<FTypeInfoReinstanced>();
		TArrayView<FTypeInfoReinstanced> ReinstanceArray = ScratchBuffer.EmplaceArray<FTypeInfoReinstanced>(ReinstanceCount);
		
		// Populate the list of re-instance data and disable the commands.
		FTypeInfoReinstanced* ReinstanceArrayIt = ReinstanceArray.GetData();
		Commands.ForEach(
			[&Commands, &ReinstanceArrayIt]
			(int32 Index, CompatibilityCommandBuffer::TCommandVariant& Command)
			{
				if (FTypeInfoReinstanced* ReinstanceData = Command.TryGet<FTypeInfoReinstanced>())
				{
					*ReinstanceArrayIt = *ReinstanceData;
					Commands.ReplaceCommand<FNopCommand>(Index);
					++ReinstanceArrayIt;
				}
			});

		// Add a command to process type information stored in TEDS itself in a later step.
		Commands.AddCommand<FTypeBatchInfoReinstanced>(FTypeBatchInfoReinstanced{ .Batch = ReinstanceArray });

		// Sort the extracted array by the index of the original value for faster indexing in the future.
		ReinstanceArray.Sort([](const FTypeInfoReinstanced& Lhs, const FTypeInfoReinstanced& Rhs)
			{
				return Lhs.Original.GetWeakPtrTypeHash() < Rhs.Original.GetWeakPtrTypeHash();
			});

		// Patch the type info table.
		// There can be a large number of re-instanced type information but typically a limited number of type-to-table mappings. Instead
		// of searching through all the type information updates, copy the type-to-table locally and rebuild it with update info.
		using TypeTablePair = TPair<TWeakObjectPtr<UStruct>, TableHandle>;
		TArrayView<TypeTablePair> TempTypeToTable = ScratchBuffer.EmplaceArray<TypeTablePair>(StorageCompat.TypeToTableMap.Num());
		TypeTablePair* TempTypeToTableIt = TempTypeToTable.GetData();
		for (decltype(StorageCompat.TypeToTableMap)::TIterator It = StorageCompat.TypeToTableMap.CreateIterator(); It; ++It)
		{
			TempTypeToTableIt->Key = It.Key();
			TempTypeToTableIt->Value = It.Value();
			++TempTypeToTableIt;
		}
		StorageCompat.TypeToTableMap.Reset();
		for (TypeTablePair It : TempTypeToTable)
		{
			StorageCompat.TypeToTableMap.Add(FTypeBatchInfoReinstanced::FindObjectRecursively(ReinstanceArray, It.Key), It.Value);
		}

		// Patch locally cached type information
		const UScriptStruct** LocalTypeInfoStorage[] =
		{ 
			FAddSyncFromWorldTag::GetTypeAddress(),
			FAddInteractiveSyncFromWorldTag::GetTypeAddress()
		};
		TArrayView<const UScriptStruct**> LocalTypeInfo(LocalTypeInfoStorage, sizeof(LocalTypeInfoStorage) / sizeof(UScriptStruct**));
		for (const UScriptStruct** TypeInfo : LocalTypeInfo)
		{
			*TypeInfo = Cast<const UScriptStruct>(
				FTypeBatchInfoReinstanced::FindObjectRecursively(ReinstanceArray, TWeakObjectPtr<const UStruct>(*TypeInfo)));
		}

		// Patch existing commands with the new type information.
		Commands.Process(FPatchCommand(ReinstanceArray));
	}
	


	//
	// FPrepareCommands
	//

	FPrepareCommands::FPrepareCommands(ICoreProvider& InStorage, UEditorDataStorageCompatibility& InStorageCompat,
		CompatibilityCommandBuffer::FCollection& InCommands)
		: Storage(InStorage)
		, StorageCompat(InStorageCompat)
		, Commands(InCommands)
	{}

	void FPrepareCommands::operator()(FAddCompatibleUObject& Object)
	{

		UObject* ObjectPtr = Object.Object.Get();
		if (ObjectPtr && Storage.IsRowAvailable(Object.Row))
		{
			Object.Table = StorageCompat.FindBestMatchingTable(ObjectPtr->GetClass());
			checkf(Object.Table != InvalidTableHandle,
				TEXT("The data storage could not find any matching tables for object of type '%s'. "
					"This can mean that the object doesn't derive from UObject or that a table for UObject is no longer registered."),
				*Object.Object->GetClass()->GetFName().ToString());
		}
		else
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	void FPrepareCommands::operator()(FAddCompatibleExternalObject& Object)
	{
		if (Storage.IsRowAvailable(Object.Row) && Object.Object != nullptr)
		{
			Object.Table = StorageCompat.FindBestMatchingTable(Object.TypeInfo.Get());
			Object.Table = (Object.Table != InvalidTableHandle) ? Object.Table : StorageCompat.StandardExternalObjectTable;
		}
		else
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	void FPrepareCommands::operator()(FRemoveCompatibleUObject& Command)
	{
		if (Command.ObjectRow == InvalidRowHandle)
		{
			FMapKeyView Key = FMapKeyView(Command.Object);
			Command.ObjectRow = Storage.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, Key);
			if (!Storage.IsRowAvailable(Command.ObjectRow))
			{
				Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
			}
		}
	}

	void FPrepareCommands::operator()(FRemoveCompatibleExternalObject& Command)
	{
		FMapKeyView Key = FMapKeyView(Command.Object);
		Command.ObjectRow = Storage.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, Key);
		if (!Storage.IsRowAvailable(Command.ObjectRow))
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	void FPrepareCommands::operator()(FAddSyncFromWorldTag& Command)
	{
		Command.Row = StorageCompat.FindRowWithCompatibleObject(Command.Target);
		if (!Storage.IsRowAvailable(Command.Row))
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	void FPrepareCommands::operator()(FRemoveInteractiveSyncFromWorldTag& Command)
	{
		Command.Row = StorageCompat.FindRowWithCompatibleObject(Command.Target);
		if (!Storage.IsRowAvailable(Command.Row))
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	void FPrepareCommands::RunPreparation(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompat,
		CompatibilityCommandBuffer::FCollection& Commands)
	{
		FPrepareCommands PrepareVisitor(Storage, StorageCompat, Commands);
		Commands.ForEach(
			[&PrepareVisitor](int32 Index, CompatibilityCommandBuffer::TCommandVariant& Command)
			{
				PrepareVisitor.CurrentIndex = Index;
				Visit(PrepareVisitor, Command);
			});
	}

	void FPrepareCommands::operator()(FAddInteractiveSyncFromWorldTag& Command)
	{
		Command.Row = StorageCompat.FindRowWithCompatibleObject(Command.Target);
		if (!Storage.IsRowAvailable(Command.Row))
		{
			Commands.ReplaceCommand<FNopCommand>(CurrentIndex);
		}
	}

	//
	// FGetSourceRowHandle
	//

	RowHandle FGetSourceRowHandle::operator()(const FAddCompatibleUObject& Command)
	{
		return Command.Row;
	}

	RowHandle FGetSourceRowHandle::operator()(const FAddCompatibleExternalObject& Command)
	{
		return Command.Row;
	}

	RowHandle FGetSourceRowHandle::operator()(const FCreateMemento& Command)
	{
		return Command.TargetRow;
	}

	RowHandle FGetSourceRowHandle::operator()(const FRestoreMemento& Command)
	{
		return Command.TargetRow;
	}

	RowHandle FGetSourceRowHandle::operator()(const FDestroyMemento& Command)
	{
		return Command.MementoRow;
	}

	RowHandle FGetSourceRowHandle::operator()(const FRemoveCompatibleUObject& Command)
	{
		return Command.ObjectRow;
	}

	RowHandle FGetSourceRowHandle::operator()(const FRemoveCompatibleExternalObject& Command)
	{
		return Command.ObjectRow;
	}

	RowHandle FGetSourceRowHandle::Get(const CompatibilityCommandBuffer::TCommandVariant& Command)
	{
		return Visit(FGetSourceRowHandle{}, Command);
	}

	RowHandle FGetSourceRowHandle::operator()(const FAddInteractiveSyncFromWorldTag& Command)
	{
		return Command.Row;
	}

	RowHandle FGetSourceRowHandle::operator()(const FRemoveInteractiveSyncFromWorldTag& Command)
	{
		return Command.Row;
	}

	RowHandle FGetSourceRowHandle::operator()(const FAddSyncFromWorldTag& Command)
	{
		return Command.Row;
	}



	//
	// FGetCommandGroupId
	//

	// FAddCompatibleUObject group begin: The following commands needs to be sorted together with FAddCompatibleUObject.

	SIZE_T FGetCommandGroupId::operator()(const FBatchAddCompatibleUObject& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FAddCompatibleExternalObject& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FBatchAddCompatibleExternalObject& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FCreateMemento& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FRestoreMemento& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FDestroyMemento& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FRemoveCompatibleUObject& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	SIZE_T FGetCommandGroupId::operator()(const FRemoveCompatibleExternalObject& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddCompatibleUObject>();
	}

	// FAddCompatibleUObject group end
	
	// FAddInteractiveSyncFromWorldTag group begin: The following commands needs to be sorted together with FAddInteractiveSyncFromWorldTag.

	SIZE_T FGetCommandGroupId::operator()(const FRemoveInteractiveSyncFromWorldTag& Command)
	{
		return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<FAddInteractiveSyncFromWorldTag>();
	}

	// FAddInteractiveSyncFromWorldTag group end

	SIZE_T FGetCommandGroupId::Get(const CompatibilityCommandBuffer::TCommandVariant& Command)
	{
		return Visit(FGetCommandGroupId{}, Command);
	}





	//
	// FSorter
	//

	void FSorter::SortCommands(CompatibilityCommandBuffer::FCollection& Commands)
	{
		// First sort commands by the row they operate on, using stable sort so commands remain in the same range as they get processed.
		// Within a row, commands that can't be reordered relative to each other are grouped together while the remaining commands are
		// ordered in the order they were declared in the command buffer.
		Commands.Sort<true /* Use stable sort */>(
			[](const CompatibilityCommandBuffer::TCommandVariant& Lhs, const CompatibilityCommandBuffer::TCommandVariant& Rhs)
			{
				RowHandle Left = FGetSourceRowHandle::Get(Lhs);
				RowHandle Right = FGetSourceRowHandle::Get(Rhs);
				
				// Sort commands by index to make sure operations stay close together so the same table gets accessed more
				// frequently, but operations continue to execute in the order they were issued. For instance an Add + Remove
				// gives a different result from a Remove + Add.
				if (Left < Right)
				{
					return true;
				}
				else if (Left == Right)
				{
					SIZE_T LeftGroupId = FGetCommandGroupId::Get(Lhs);
					SIZE_T RightGroupId = FGetCommandGroupId::Get(Rhs);
					return LeftGroupId < RightGroupId;
				}
				return false;
			});
	}



	//
	// FCommandProcessor
	//

	FCommandProcessor::FCommandProcessor(ICoreProvider& InStorage, UEditorDataStorageCompatibility& InStorageCompatibility)
		: Storage(InStorage)
		, StorageCompatibility(InStorageCompatibility)
		, MementoSystem(InStorageCompatibility.Environment->GetMementoSystem())
	{}

	void FCommandProcessor::SetupRow(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompatibility, 
		RowHandle Row, UObject* Object)
	{
		Storage.AddColumn(Row, FTypedElementUObjectColumn{ .Object = Object });
		Storage.AddColumn(Row, FTypedElementUObjectIdColumn
			{
				.Id = Object->GetUniqueID(),
				.SerialNumber = GUObjectArray.GetSerialNumber(Object->GetUniqueID())
			});
		Storage.AddColumn(Row, FTypedElementClassTypeInfoColumn{ .TypeInfo = Object->GetClass() });
		if (Object->HasAnyFlags(RF_ClassDefaultObject))
		{
			Storage.AddColumn<FTypedElementClassDefaultObjectTag>(Row);
		}
		// Make sure the new row is tagged for update.
		Storage.AddColumn<FTypedElementSyncFromWorldTag>(Row);
		StorageCompatibility.TriggerOnObjectAdded(Object, Object->GetClass(), Row);
	}

	void FCommandProcessor::SetupRow(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompatibility, 
		RowHandle Row, void* Object, TWeakObjectPtr<const UScriptStruct> TypeInfo)
	{
		Storage.AddColumn(Row, FTypedElementExternalObjectColumn{ .Object = Object });
		Storage.AddColumn(Row, FTypedElementScriptStructTypeInfoColumn{ .TypeInfo = TypeInfo });
		// Make sure the new row is tagged for update.
		Storage.AddColumn<FTypedElementSyncFromWorldTag>(Row);
		StorageCompatibility.TriggerOnObjectAdded(Object, TypeInfo.Get(), Row);
	}

	void FCommandProcessor::operator()(FTypeBatchInfoReinstanced& Command)
	{
		using namespace UE::Editor::DataStorage::Queries;

		StorageCompatibility.Storage->RunQuery(StorageCompatibility.ClassTypeInfoQuery, CreateDirectQueryCallbackBinding(
			[Range = Command.Batch](IDirectQueryContext& Context, FTypedElementClassTypeInfoColumn& Type)
			{
				Type.TypeInfo = FTypeBatchInfoReinstanced::FindObjectRecursively(Range, Type.TypeInfo);
				checkf(Type.TypeInfo.IsValid(),
					TEXT("Type info column in data storage has been re-instanced to an object without class type information"));
			}));

		StorageCompatibility.Storage->RunQuery(StorageCompatibility.ScriptStructTypeInfoQuery, CreateDirectQueryCallbackBinding(
			[Range = Command.Batch](IDirectQueryContext& Context, FTypedElementScriptStructTypeInfoColumn& Type)
			{
				Type.TypeInfo = FTypeBatchInfoReinstanced::FindObjectRecursively(Range, Type.TypeInfo);
				checkf(Type.TypeInfo.IsValid(),
					TEXT("Type info column in data storage has been re-instanced to an object without struct type information"));
			}));
	}

	void FCommandProcessor::operator()(FRegisterTypeTableAssociation& Command)
	{
		StorageCompatibility.TypeToTableMap.Add(Command.TypeInfo, Command.Table);
	}

	void FCommandProcessor::operator()(FRegisterObjectAddedCallback& Command)
	{
		StorageCompatibility.ObjectAddedCallbackList.Emplace(MoveTemp(Command.Callback), Command.Handle);
	}

	void FCommandProcessor::operator()(FUnregisterObjectAddedCallback& Command)
	{
		StorageCompatibility.ObjectAddedCallbackList.RemoveAll(
			[Handle = Command.Handle](const TPair<ObjectAddedCallback, FDelegateHandle>& Element) -> bool
			{
				return Element.Value == Handle;
			});
	}

	void FCommandProcessor::operator()(FAddCompatibleUObject& Object)
	{
		UObject* ObjectPtr = Object.Object.Get();
		checkf(ObjectPtr, TEXT(
			"Expected a valid object pointer. If there isn't one here then the filter pass did not correctly clean up this command."));
		Storage.AddRow(Object.Row, Object.Table, 
			[ObjectPtr, this](RowHandle Row)
			{
				SetupRow(Storage, StorageCompatibility, Row, ObjectPtr);
			});
	}

	void FCommandProcessor::operator()(FBatchAddCompatibleUObject& Batch)
	{
		TConstArrayView<RowHandle> Rows(Batch.RowArray, Batch.Count);
		RowHandle* RowIterator = Batch.RowArray;
		TWeakObjectPtr<UObject>* ObjectIterator = Batch.ObjectArray;

		Storage.BatchAddRow(Batch.Table, Rows, [&RowIterator, &ObjectIterator, this](RowHandle Row)
			{
				checkf(Row == *RowIterator++, TEXT("Expecting the same sequence of rows when batch adding object to TEDS Compatibility."));
				UObject* ObjectPtr = ObjectIterator->Get();
				checkf(ObjectPtr, TEXT(
					"Expected a valid object pointer. If there isn't one here then the filter pass did not correctly clean up this command."));
				SetupRow(Storage, StorageCompatibility, Row, ObjectPtr);
				ObjectIterator++;
			});
	}

	void FCommandProcessor::operator()(FAddCompatibleExternalObject& Object)
	{
		Storage.AddRow(Object.Row, Object.Table, [&Object, this](RowHandle Row)
			{
				SetupRow(Storage, StorageCompatibility, Object.Row, Object.Object, Object.TypeInfo);
			});
	}

	void FCommandProcessor::operator()(FBatchAddCompatibleExternalObject& Batch)
	{
		TConstArrayView<RowHandle> Rows(Batch.RowArray, Batch.Count);
		RowHandle* RowIterator = Batch.RowArray;
		void** ObjectIterator = Batch.ObjectArray;
		TWeakObjectPtr<const UScriptStruct>* TypeInfoIterator = Batch.TypeInfoArray;

		Storage.BatchAddRow(Batch.Table, Rows, [&RowIterator, &ObjectIterator, &TypeInfoIterator, this](RowHandle Row)
			{
				checkf(Row == *RowIterator++, TEXT("Expecting the same sequence of rows when batch adding object to TEDS Compatibility."));
				SetupRow(Storage, StorageCompatibility, Row, *ObjectIterator++, *TypeInfoIterator++);
			});
	}

	void FCommandProcessor::operator()(FCreateMemento& Command)
	{
		MementoSystem.CreateMemento(Command.ReservedMementoRow, Command.TargetRow);
	}

	void FCommandProcessor::operator()(FRestoreMemento& Command)
	{
		MementoSystem.RestoreMemento(Command.MementoRow, Command.TargetRow);
	}

	void FCommandProcessor::operator()(FDestroyMemento& Command)
	{
		MementoSystem.DestroyMemento(Command.MementoRow);
	}

	void FCommandProcessor::operator()(FRemoveCompatibleUObject& Command)
	{
		if (Storage.IsRowAssigned(Command.ObjectRow))
		{
			const FTypedElementClassTypeInfoColumn* TypeInfoColumn = Storage.GetColumn<FTypedElementClassTypeInfoColumn>(Command.ObjectRow);
			checkf(TypeInfoColumn, 
				TEXT("Missing type information for removed UObject at ptr 0x%p [%s]"), Command.Object, *Command.Object->GetName());
			StorageCompatibility.TriggerOnPreObjectRemoved(Command.Object, TypeInfoColumn->TypeInfo.Get(), Command.ObjectRow);
		}
		Storage.RemoveRow(Command.ObjectRow);
	}

	void FCommandProcessor::operator()(FRemoveCompatibleExternalObject& Command)
	{
		if (Storage.IsRowAssigned(Command.ObjectRow))
		{
			const FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = Storage.GetColumn<FTypedElementScriptStructTypeInfoColumn>(Command.ObjectRow);
			checkf(TypeInfoColumn, TEXT("Missing type information for removed void* object at ptr 0x%p"), Command.Object);
			StorageCompatibility.TriggerOnPreObjectRemoved(Command.Object, TypeInfoColumn->TypeInfo.Get(), Command.ObjectRow);
		}
		Storage.RemoveRow(Command.ObjectRow);
	}

	void FCommandProcessor::operator()(FAddInteractiveSyncFromWorldTag& Command)
	{
		Storage.AddColumn(Command.Row, FAddInteractiveSyncFromWorldTag::GetType());
	}

	void FCommandProcessor::operator()(FRemoveInteractiveSyncFromWorldTag& Command)
	{
		Storage.AddRemoveColumns(Command.Row,
			{ FAddSyncFromWorldTag::GetType()},
			{ FAddInteractiveSyncFromWorldTag::GetType() });
	}

	void FCommandProcessor::operator()(FAddSyncFromWorldTag& Command)
	{
		Storage.AddColumn(Command.Row, FAddSyncFromWorldTag::GetType());
	}




	//
	// FRecordCommands
	//

	void FRecordCommands::operator()(const FNopCommand& Command) 
	{
		if (bIncludeNops)
		{
			CommandDescriptions.Append(TEXT("    FNopCommand\n"));
		}
	}

	void FRecordCommands::operator()(const FTypeInfoReinstanced& Command)
	{
		CommandDescriptions.Append(TEXT("    FTypeInfoReinstanced: '"));
		Command.Reinstanced.GetEvenIfUnreachable()->AppendName(CommandDescriptions);
		CommandDescriptions.Append(TEXT("' \n"));
	}

	void FRecordCommands::operator()(const FTypeBatchInfoReinstanced& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FTypeBatchInfoReinstanced: %i re-instances\n"), Command.Batch.Num());
		for (FTypeInfoReinstanced& TypeInfo : Command.Batch)
		{
			CommandDescriptions.Append(TEXT("        "));
			TypeInfo.Reinstanced.GetEvenIfUnreachable()->AppendName(CommandDescriptions);
			CommandDescriptions.Append(TEXT("\n"));
		}
	}

	void FRecordCommands::operator()(const FRegisterTypeTableAssociation& Command)
	{
		CommandDescriptions.Append(TEXT("    FRegisterTypeTableAssociation: "));
		Command.TypeInfo->AppendName(CommandDescriptions);
		CommandDescriptions.Append(TEXT("\n"));
	}

	void FRecordCommands::operator()(const FRegisterObjectAddedCallback& Command)
	{
		CommandDescriptions.Append(TEXT("    FRegisterObjectAddedCallback\n"));
	}

	void FRecordCommands::operator()(const FUnregisterObjectAddedCallback& Command)
	{
		CommandDescriptions.Append(TEXT("    FUnregisterObjectAddedCallback\n"));
	}

	void FRecordCommands::operator()(const FAddCompatibleUObject& Command)
	{
		CommandDescriptions.Append(TEXT("    FAddCompatibleUObject: '"));
		Command.Object->AppendName(CommandDescriptions);
		CommandDescriptions.Appendf(TEXT("' row %llu, table %llu\n"), Command.Row, Command.Table);
	}

	void FRecordCommands::operator()(const FBatchAddCompatibleUObject& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FBatchAddCompatibleUObject: %u objects, table %llu\n"), Command.Count, Command.Table);
		for (uint32 Index = 0; Index < Command.Count; ++Index)
		{
			CommandDescriptions.Append(TEXT("        '"));
			Command.ObjectArray[Index]->AppendName(CommandDescriptions);
			CommandDescriptions.Appendf(TEXT("`row %llu\n"), Command.RowArray[Index]);
		}
	}

	void FRecordCommands::operator()(const FAddCompatibleExternalObject& Command)
	{
		CommandDescriptions.Append(TEXT("    FAddCompatibleExternalObject: '"));
		Command.TypeInfo->AppendName(CommandDescriptions);
		CommandDescriptions.Appendf(TEXT("' row %llu, table %llu\n"), Command.Row, Command.Table);
	}

	void FRecordCommands::operator()(const FBatchAddCompatibleExternalObject& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FBatchAddCompatibleExternalObject: %u objects, table %llu\n"), Command.Count, Command.Table);
	}

	void FRecordCommands::operator()(const FCreateMemento& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FCreateMemento: row %llu, memento row %llu\n"), Command.TargetRow, Command.ReservedMementoRow);
	}

	void FRecordCommands::operator()(const FRestoreMemento& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FRestoreMemento: row %llu, memento row %llu\n"), Command.TargetRow, Command.MementoRow);
	}

	void FRecordCommands::operator()(const FDestroyMemento& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FDestroyMemento: memento row %llu\n"), Command.MementoRow);
	}

	void FRecordCommands::operator()(const FRemoveCompatibleUObject& Command)
	{
		CommandDescriptions.Append(TEXT("    FRemoveCompatibleUObject: '"));
		Command.Object->AppendName(CommandDescriptions);
		CommandDescriptions.Appendf(TEXT("' row %llu\n"), Command.ObjectRow);
	}

	void FRecordCommands::operator()(const FRemoveCompatibleExternalObject& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FRemoveCompatibleExternalObject: row %llu\n"), Command.ObjectRow);
	}

	void FRecordCommands::operator()(const FAddInteractiveSyncFromWorldTag& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FAddInteractiveSyncFromWorldTag: row %llu\n"), Command.Row);
	}

	void FRecordCommands::operator()(const FRemoveInteractiveSyncFromWorldTag& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FRemoveInteractiveSyncFromWorldTag: row %llu\n"), Command.Row);
	}

	void FRecordCommands::operator()(const FAddSyncFromWorldTag& Command)
	{
		CommandDescriptions.Appendf(TEXT("    FAddSyncFromWorldTag: row %llu\n"), Command.Row);
	}

	FString FRecordCommands::PrintToString(CompatibilityCommandBuffer::FCollection& Commands, bool bIncludeNops)
	{
		FRecordCommands Printer;
		if (bIncludeNops)
		{
			Printer.bIncludeNops = true;
			Commands.Process(Printer);
		}
		else
		{
			Printer.bIncludeNops = false;
			Commands.Process(Printer);
		}
		return MoveTemp(Printer.CommandDescriptions);
	}



	
	//
	// FCommandOptimizer
	//

	FCommandOptimizer::FCommandOptimizer(
		CompatibilityCommandBuffer::FOptimizer& InOptimizer, FScratchBuffer& InScratchBuffer)
		: Optimizer(InOptimizer)
		, ScratchBuffer(InScratchBuffer)
	{}

	void FCommandOptimizer::operator()(const FAddCompatibleUObject& Command)
	{
		int32 Count = FoldCommandsForAdd(Command);

		// If there are more than 1 adds, batch them together into a batch call and nop the additional adds out.
		if (Count > 1)
		{
			TArrayView<RowHandle> Rows = ScratchBuffer.AllocateUninitializedArray<RowHandle>(Count);
			TArrayView<TWeakObjectPtr<UObject>> Objects = ScratchBuffer.EmplaceArray<TWeakObjectPtr<UObject>>(Count);

			Rows[0] = Command.Row;
			Objects[0] = Command.Object;

			for (int32 Index = 1; Index < Count;)
			{
				if (FAddCompatibleUObject* Right = Optimizer.GetRight().TryGet<FAddCompatibleUObject>())
				{
					Rows[Index] = Right->Row;
					Objects[Index] = Right->Object;
					Optimizer.ReplaceRight<FNopCommand>();
					++Index;
				}
				Optimizer.MoveToNextRight();
			}
			Optimizer.ReplaceLeft<FBatchAddCompatibleUObject>(
				FBatchAddCompatibleUObject
				{
					.ObjectArray = Objects.GetData(),
					.RowArray = Rows.GetData(),
					.Table = Command.Table,
					.Count = static_cast<uint32>(Count)
				});
			// Skip over the nops for the next optimization.
			Optimizer.MoveLeftToRight();
		}
	}
	
	void FCommandOptimizer::operator()(const FAddCompatibleExternalObject& Command)
	{
		int32 Count = FoldCommandsForAdd(Command);

		// If there are more than 1 adds, batch them together into a batch call and nop them out.
		if (Count > 1)
		{
			TArrayView<RowHandle> Rows = ScratchBuffer.AllocateUninitializedArray<RowHandle>(Count);
			TArrayView<void*> Objects = ScratchBuffer.AllocateUninitializedArray<void*>(Count);
			TArrayView<TWeakObjectPtr<const UScriptStruct>> TypeInfo = ScratchBuffer.EmplaceArray<TWeakObjectPtr<const UScriptStruct>>(Count);

			Rows[0] = Command.Row;
			Objects[0] = Command.Object;
			TypeInfo[0] = Command.TypeInfo;

			for (int32 Index = 1; Index < Count; ++Index)
			{
				FAddCompatibleExternalObject& Right = Optimizer.GetRight().Get<FAddCompatibleExternalObject>();
				Rows[Index] = Right.Row;
				Objects[Index] = Right.Object;
				TypeInfo[Index] = Right.TypeInfo;
				Optimizer.ReplaceRight<FNopCommand>();
				Optimizer.MoveToNextRight();
			}
			Optimizer.GetLeft().Emplace<FBatchAddCompatibleExternalObject>(
				FBatchAddCompatibleExternalObject
				{
					.ObjectArray = Objects.GetData(),
					.TypeInfoArray = TypeInfo.GetData(),
					.RowArray = Rows.GetData(),
					.Table = Command.Table,
					.Count = static_cast<uint32>(Count)
				});
			// Skip over the nops for the next optimization.
			Optimizer.MoveLeftToRight();
		}
	}

	void FCommandOptimizer::operator()(const FRemoveCompatibleUObject& Command)
	{
		// Remove everything after the delete because no operation on a row is going to succeed after
		// the row has been deleted.
		while(Optimizer.IsValid())
		{
			RowHandle RightRow = FGetSourceRowHandle::Get(Optimizer.GetRight());
			if (RightRow == Command.ObjectRow)
			{
				Optimizer.ReplaceRight<FNopCommand>();
				Optimizer.MoveToNextRight();
			}
			else
			{
				break;
			}
		}
		Optimizer.MoveLeftBeforeRight();
	}

	void FCommandOptimizer::operator()(const FRemoveCompatibleExternalObject& Command)
	{		
		// Remove everything after the delete because no operation on a row is going to succeed after
		// the row has been deleted.
		while (Optimizer.IsValid())
		{
			RowHandle RightRow = FGetSourceRowHandle::Get(Optimizer.GetRight());
			if (RightRow == Command.ObjectRow)
			{
				Optimizer.ReplaceRight<FNopCommand>();
				Optimizer.MoveToNextRight();
			}
			else
			{
				break;
			}
		}
		Optimizer.MoveLeftBeforeRight();
	}

	template<typename AddCommandType>
	int32 FCommandOptimizer::FoldCommandsForAdd(const AddCommandType& Command)
	{
		// Discover the longest chain of adds for the same table and remove any async tag additions as it'll be included
		// in the table for UObjects.
		int32 Count = 1;
		RowHandle SourceRow = Command.Row;
		for (; Optimizer.IsValid(); Optimizer.MoveToNextRight())
		{
			const CompatibilityCommandBuffer::TCommandVariant& TargetCommand = Optimizer.GetRight();
			if (const AddCommandType* Right = TargetCommand.TryGet<AddCommandType>())
			{
				if (Right->Table == Command.Table)
				{
					++Count;
					SourceRow = Right->Row;
				}
				else
				{
					// No longer in the same table, so stop.
					break;
				}
			}
			else
			{
				RowHandle TargetRow = FGetSourceRowHandle::Get(TargetCommand);
				if (TargetRow == SourceRow)
				{
					if (TargetCommand.IsType<FAddSyncFromWorldTag>())
					{
						// These are always ordered after the interactive tags, so if there are any interactive tags these
						// are already folded into a single one. If there are not then this pass will fold them into a single
						// one.
						Optimizer.ReplaceRight<FNopCommand>();
					}
					else if (TargetCommand.IsType<FAddInteractiveSyncFromWorldTag>() ||
						TargetCommand.IsType<FRemoveInteractiveSyncFromWorldTag>())
					{
						// Run optimizations on the interactive tags so they can get folded and avoid complex checks here.
						RunRightOnRowCluster(SourceRow);
						// After folding the remaining options, ignoring nops, are:
						//		1. Add Interactive Sync Tag
						//		2. Remove Interactive Sync Tag
						//		3. Add Sync Tag
						//		4. Add Interactive Sync Tag + Add Sync Tag
						// The versions using tag are folded into the addition in the above check, so 3 and 4 do not need to be folded here.
						// Adding an interactive tag will be needed so leave that untouched, but do nop the remove interactive sync tag as
						// there's guaranteed to be no interactive sync tag added at this point.
						if (Optimizer.GetRight().IsType<FNopCommand>())
						{
							// Clearing out commands could leave right pointing at a nop, so skip that one and any following nops.
							Optimizer.MoveToNextRight();
						}
						if (Optimizer.GetRight().IsType<FRemoveInteractiveSyncFromWorldTag>() ||
							Optimizer.GetRight().IsType<FAddSyncFromWorldTag>())
						{
							Optimizer.ReplaceRight<FNopCommand>();
						}
					}
					else
					{
						// This is a command that might require the order to remain stable so stop optimizations at this point.
						break;
					}
				}
				else
				{
					break;
				}
			}
		}
		Optimizer.ResetRightNextToLeft();
		return Count;
	}

	void FCommandOptimizer::FoldSyncFromWorldTags(CompatibilityCommandBuffer::FOptimizer& Cluster)
	{
		// The following combinations are possible for interactive tags:
		//		1. Add + Remove -> nop + nop
		//		2. Remove + Add -> nop + Add
		//		3. Add + Add -> Add + nop
		//		4. Remove + Remove -> Remove + nop
		// Sync from world tags are always after the interactive tags, so combinations are:
		//		5. Add Interactive + Sync Tag -> Add Interactive + Sync Tag
		//		6. Remove Interactive + Sync Tag -> Remove Interactive + nop (Remove Interactive also adds a sync tag).
		//		7. Sync Tag + Sync Tag -> Sync Tag + nop
		while (Cluster.IsValid())
		{
			const CompatibilityCommandBuffer::TCommandVariant& Left = Cluster.GetLeft();
			const CompatibilityCommandBuffer::TCommandVariant& Right = Cluster.GetRight();

			if (Left.IsType<FAddInteractiveSyncFromWorldTag>() && Right.IsType<FRemoveInteractiveSyncFromWorldTag>()) // 1
			{
				Cluster.ReplaceLeft<FNopCommand>();
				Cluster.ReplaceRight<FNopCommand>();
				
				// Move forward to the next sync tag command.
				Cluster.MoveLeftToRight();
				Cluster.MoveToNextLeft();
				for (; Cluster.IsValid(); Cluster.MoveToNextRight())
				{
					const CompatibilityCommandBuffer::TCommandVariant& NewLeft = Cluster.GetLeft();
					if (NewLeft.IsType<FAddInteractiveSyncFromWorldTag>() ||
						NewLeft.IsType<FRemoveInteractiveSyncFromWorldTag>() ||
						NewLeft.IsType<FAddSyncFromWorldTag>())
					{
						break;
					}
				}
			}
			else if (Left.IsType<FRemoveInteractiveSyncFromWorldTag>() && Right.IsType<FAddInteractiveSyncFromWorldTag>()) // 2
			{
				Cluster.ReplaceLeft<FNopCommand>();
				Cluster.MoveLeftToRight();
			}
			else if (
				(Left.IsType<FAddInteractiveSyncFromWorldTag>() && Right.IsType<FAddInteractiveSyncFromWorldTag>()) || // 3
				(Left.IsType<FRemoveInteractiveSyncFromWorldTag>() && Right.IsType<FRemoveInteractiveSyncFromWorldTag>()) || // 4
				(Left.IsType<FRemoveInteractiveSyncFromWorldTag>() && Right.IsType<FAddSyncFromWorldTag>()) || // 6
				(Left.IsType<FAddSyncFromWorldTag>() && Right.IsType<FAddSyncFromWorldTag>())) // 7
			{
				Cluster.ReplaceRight<FNopCommand>();
				Cluster.MoveToNextRight();
			}
			else if (Left.IsType<FAddInteractiveSyncFromWorldTag>() && Right.IsType<FAddSyncFromWorldTag>()) // 5
			{
				Cluster.MoveLeftToRight();
			}
			else
			{
				Cluster.MoveToNextRight();
			}
		}
	}

	void FCommandOptimizer::operator()(const FAddInteractiveSyncFromWorldTag& Command)
	{
		CompatibilityCommandBuffer::FOptimizer Range = CreateRangeOptimizer(Command.Row);
		FoldSyncFromWorldTags(Range);
	}

	void FCommandOptimizer::operator()(const FRemoveInteractiveSyncFromWorldTag& Command)
	{
		CompatibilityCommandBuffer::FOptimizer Range = CreateRangeOptimizer(Command.Row);
		FoldSyncFromWorldTags(Range);
	}

	void FCommandOptimizer::operator()(const FAddSyncFromWorldTag& Command)
	{
		CompatibilityCommandBuffer::FOptimizer Range = CreateRangeOptimizer(Command.Row);
		FoldSyncFromWorldTags(Range);
	}

	CompatibilityCommandBuffer::FOptimizer FCommandOptimizer::CreateRangeOptimizer(RowHandle RowCluster)
	{
		return Optimizer.BranchOnLeft(
			[RowCluster](const CompatibilityCommandBuffer::TCommandVariant& Command)
			{
				return FGetSourceRowHandle::Get(Command) == RowCluster;
			});
	}

	void FCommandOptimizer::RunRightOnRowCluster(RowHandle RowCluster)
	{
		CompatibilityCommandBuffer::FOptimizer SubOptimizer = Optimizer.BranchOnRight(
			[RowCluster](const CompatibilityCommandBuffer::TCommandVariant& Command)
			{
				return FGetSourceRowHandle::Get(Command) == RowCluster;
			});
		FCommandOptimizer OptimizationSelector(SubOptimizer, ScratchBuffer);
		if (SubOptimizer.IsValid())
		{
			Visit(OptimizationSelector, SubOptimizer.GetLeft());
		}
	}

	void FCommandOptimizer::Run(CompatibilityCommandBuffer::FCollection& Commands, FScratchBuffer& ScratchBuffer)
	{
		CompatibilityCommandBuffer::FOptimizer Optimizer(Commands);
		FCommandOptimizer OptimizationSelector(Optimizer, ScratchBuffer);
		while (Optimizer.IsValid())
		{
			Visit(OptimizationSelector, Optimizer.GetLeft());
			Optimizer.MoveToNextLeft();
			Optimizer.ResetRightNextToLeft();
		}
	}
} // namespace UE::Editor::DataStorage
