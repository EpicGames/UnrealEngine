// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commands/EditorDataStorageCommandBuffer.h"
#include "DataStorage/Handles.h"
#include "Delegates/IDelegateInstance.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;
class UEditorDataStorageCompatibility;
class UObject;

namespace UE::Editor::DataStorage
{
	class FScratchBuffer;
	class FMementoSystem;
	class ICoreProvider;

	enum class EObjectType : uint8
	{
		Struct,
		Class
	};

	/**
	 * Objects with type info defined in either UScriptStruct or UClass can be stored into TEDS via the
	 * ICompatibilityProvider
	 * This is a discriminated union which aids with callbacks made when objects are added
	 */
	struct FObjectTypeInfo
	{
		EObjectType TypeInfoType;

		union
		{
			const UScriptStruct* ScriptStruct;
			const UClass* Class;
		};

		FObjectTypeInfo(const UScriptStruct* InScriptStruct);
		FObjectTypeInfo(const UClass* InClass);

		FName GetFName() const;
	};

	struct FNopCommand;

	struct FTypeInfoReinstanced final
	{
		TWeakObjectPtr<UStruct> Original;
		TWeakObjectPtr<UStruct> Reinstanced;
	};

	struct FTypeBatchInfoReinstanced final
	{
		TArrayView<FTypeInfoReinstanced> Batch;

		template<typename T>
		static FTypeInfoReinstanced* FindObject(TArrayView<FTypeInfoReinstanced> Range, const TWeakObjectPtr<T>& Object);
		template<typename T>
		static TWeakObjectPtr<T> FindObjectRecursively(TArrayView<FTypeInfoReinstanced> Range, const TWeakObjectPtr<T>& Object);
	};

	struct FRegisterTypeTableAssociation final
	{
		TWeakObjectPtr<UStruct> TypeInfo;
		TableHandle Table;
	};

	struct FAddCompatibleUObject final
	{
		TWeakObjectPtr<UObject> Object;
		RowHandle Row;
		TableHandle Table;
	};

	using ObjectAddedCallback = TFunction<void(
		const void* /*Object*/, 
		const FObjectTypeInfo&, /*Type information*/
		RowHandle /*Row*/)>;
	struct FRegisterObjectAddedCallback final
	{
		ObjectAddedCallback Callback;
		FDelegateHandle Handle;
	};

	struct FUnregisterObjectAddedCallback final
	{
		FDelegateHandle Handle;
	};

	using ObjectRemovedCallback = TFunction<void(
		const void* /*Object*/, 
		const FObjectTypeInfo&, /*Type information*/
		RowHandle /*Row*/)>;

	struct FBatchAddCompatibleUObject final
	{
		TWeakObjectPtr<UObject>* ObjectArray;
		RowHandle* RowArray;
		TableHandle Table;
		uint32 Count;
	};

	struct FAddCompatibleExternalObject final
	{
		void* Object;
		TWeakObjectPtr<const UScriptStruct> TypeInfo;
		RowHandle Row;
		TableHandle Table;
	};

	struct FCreateMemento final
	{
		RowHandle ReservedMementoRow;
		RowHandle TargetRow;
	};

	struct FRestoreMemento final
	{
		RowHandle MementoRow;
		RowHandle TargetRow;
	};

	struct FDestroyMemento final
	{
		RowHandle MementoRow;
	};

	struct FRemoveCompatibleUObject final
	{
		const UObject* Object;
		RowHandle ObjectRow = InvalidRowHandle;
	};

	struct FRemoveCompatibleExternalObject final
	{
		void* Object;
		RowHandle ObjectRow;
	};

	struct FBatchAddCompatibleExternalObject final
	{
		void** ObjectArray;
		TWeakObjectPtr<const UScriptStruct>* TypeInfoArray;
		RowHandle* RowArray;
		TableHandle Table;
		uint32 Count;
	};

	struct FAddSyncFromWorldTag final
	{
		TObjectKey<const UObject> Target;
		RowHandle Row;

		static const UScriptStruct* GetType();
		static const UScriptStruct** GetTypeAddress();
	};

	struct FAddInteractiveSyncFromWorldTag final
	{
		TObjectKey<const UObject> Target;
		RowHandle Row;

		static const UScriptStruct* GetType();
		static const UScriptStruct** GetTypeAddress();
	};

	struct FRemoveInteractiveSyncFromWorldTag final
	{
		TObjectKey<const UObject> Target;
		RowHandle Row;
	};

	using CompatibilityCommandBuffer = FCommandBuffer
	<
		FTypeInfoReinstanced,
		FTypeBatchInfoReinstanced,
		FRegisterTypeTableAssociation,
		FRegisterObjectAddedCallback,
		FUnregisterObjectAddedCallback,
		
		FAddCompatibleUObject,
		FBatchAddCompatibleUObject,
		FAddCompatibleExternalObject,
		FBatchAddCompatibleExternalObject,
		
		FCreateMemento,
		FRestoreMemento,
		FDestroyMemento,
		
		FRemoveCompatibleUObject,
		FRemoveCompatibleExternalObject,
		
		FAddInteractiveSyncFromWorldTag,
		FRemoveInteractiveSyncFromWorldTag,
		FAddSyncFromWorldTag
	>;

	/**
	 * Patches data in preparation for processing. This can include fixing tables like the type-to-table map if type information has 
	 * changed.
	 */
	struct FPatchData final
	{
		struct FPatchCommand final
		{
			explicit FPatchCommand(TArrayView<FTypeInfoReinstanced>& InReinstances);

			template<typename T>
			void operator()(T&) {}

			void operator()(FRegisterTypeTableAssociation& Command);
			void operator()(FAddCompatibleExternalObject& Command);
			void operator()(FBatchAddCompatibleExternalObject& Command);

			TArrayView<FTypeInfoReinstanced>& Reinstances;
		};

		static bool IsPatchingRequired(const CompatibilityCommandBuffer::FCollection& Commands);
		static void RunPatch(CompatibilityCommandBuffer::FCollection& Commands, UEditorDataStorageCompatibility& StorageCompat,
			FScratchBuffer& ScratchBuffer);
	};

	/** Prepares each command for further processing, e.g. resolving the target table. */
	struct FPrepareCommands final
	{
		FPrepareCommands(ICoreProvider& InStorage, UEditorDataStorageCompatibility& InStorageCompat,
			CompatibilityCommandBuffer::FCollection& InCommands);

		template<typename T>
		void operator()(T&) {}
		void operator()(FAddCompatibleUObject& Object);
		void operator()(FAddCompatibleExternalObject& Object);
		void operator()(FRemoveCompatibleUObject& Command);
		void operator()(FRemoveCompatibleExternalObject& Command);
		void operator()(FAddInteractiveSyncFromWorldTag& Command);
		void operator()(FRemoveInteractiveSyncFromWorldTag& Command);
		void operator()(FAddSyncFromWorldTag& Command);

		ICoreProvider& Storage;
		UEditorDataStorageCompatibility& StorageCompat;
		CompatibilityCommandBuffer::FCollection& Commands;
		int32 CurrentIndex = 0;

		static void RunPreparation(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompat, 
			CompatibilityCommandBuffer::FCollection& Commands);
	};

	/** Retrieve the source row handle from a command. */
	struct FGetSourceRowHandle
	{
		template<typename T>
		RowHandle operator()(const T&){ return InvalidRowHandle; }
		RowHandle operator()(const FAddCompatibleUObject& Command);
		RowHandle operator()(const FAddCompatibleExternalObject& Command);
		RowHandle operator()(const FCreateMemento& Command);
		RowHandle operator()(const FRestoreMemento& Command);
		RowHandle operator()(const FDestroyMemento& Command);
		RowHandle operator()(const FRemoveCompatibleUObject& Command);
		RowHandle operator()(const FRemoveCompatibleExternalObject& Command);
		RowHandle operator()(const FAddInteractiveSyncFromWorldTag& Command);
		RowHandle operator()(const FRemoveInteractiveSyncFromWorldTag& Command);
		RowHandle operator()(const FAddSyncFromWorldTag& Command);
		static RowHandle Get(const CompatibilityCommandBuffer::TCommandVariant& Command);
	};

	/** 
	 * Group id for a command. Commands within the same group are not reordered for commands on the same row. The default order
	 * follows the order in which the commands are declared in the command buffer.
	 */
	struct FGetCommandGroupId
	{
		template<typename T>
		SIZE_T operator()(const T&) { return CompatibilityCommandBuffer::TCommandVariant::template IndexOfType<T>(); }
		
		// FAddCompatibleUObject group
		SIZE_T operator()(const FBatchAddCompatibleUObject& Command);
		SIZE_T operator()(const FAddCompatibleExternalObject& Command);
		SIZE_T operator()(const FBatchAddCompatibleExternalObject& Command);
		SIZE_T operator()(const FCreateMemento& Command);
		SIZE_T operator()(const FRestoreMemento& Command);
		SIZE_T operator()(const FDestroyMemento& Command);
		SIZE_T operator()(const FRemoveCompatibleUObject& Command);
		SIZE_T operator()(const FRemoveCompatibleExternalObject& Command);
		// FAddInteractiveSyncFromWorldTag group
		SIZE_T operator()(const FRemoveInteractiveSyncFromWorldTag& Command);
		static SIZE_T Get(const CompatibilityCommandBuffer::TCommandVariant& Command);
	};

	struct FSorter final
	{
		static void SortCommands(CompatibilityCommandBuffer::FCollection& Commands);
	};

	/** Executes the commands in the command buffer for TEDS Compatibility. */
	struct FCommandProcessor final
	{
		FCommandProcessor(ICoreProvider& InStorage, UEditorDataStorageCompatibility& InStorageCompatibility);

		static void SetupRow(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompatibility, 
			RowHandle Row, UObject* Object);
		static void SetupRow(ICoreProvider& Storage, UEditorDataStorageCompatibility& StorageCompatibility, 
			RowHandle Row, void* Object, TWeakObjectPtr<const UScriptStruct> TypeInfo);
		
		void operator()(FNopCommand&) {}
		void operator()(FTypeInfoReinstanced&) {}
		void operator()(FTypeBatchInfoReinstanced& Command);
		void operator()(FRegisterTypeTableAssociation& Command);
		void operator()(FRegisterObjectAddedCallback& Command);
		void operator()(FUnregisterObjectAddedCallback& Command);
		void operator()(FAddCompatibleUObject& Object);
		void operator()(FBatchAddCompatibleUObject& Batch);
		void operator()(FAddCompatibleExternalObject& Object);
		void operator()(FBatchAddCompatibleExternalObject& Object);
		void operator()(FCreateMemento& Command);
		void operator()(FRestoreMemento& Command);
		void operator()(FDestroyMemento& Command);
		void operator()(FRemoveCompatibleUObject& Command);
		void operator()(FRemoveCompatibleExternalObject& Command);
		void operator()(FAddInteractiveSyncFromWorldTag& Command);
		void operator()(FRemoveInteractiveSyncFromWorldTag& Command);
		void operator()(FAddSyncFromWorldTag& Command);

		ICoreProvider& Storage;
		UEditorDataStorageCompatibility& StorageCompatibility;
		FMementoSystem& MementoSystem;
	};

	struct FRecordCommands final
	{
		void operator()(const FNopCommand& Command);
		void operator()(const FTypeInfoReinstanced& Command);
		void operator()(const FTypeBatchInfoReinstanced& Command);
		void operator()(const FRegisterTypeTableAssociation& Command);
		void operator()(const FRegisterObjectAddedCallback& Command);
		void operator()(const FUnregisterObjectAddedCallback& Command);
		void operator()(const FAddCompatibleUObject& Command);
		void operator()(const FBatchAddCompatibleUObject& Command);
		void operator()(const FAddCompatibleExternalObject& Command);
		void operator()(const FBatchAddCompatibleExternalObject& Command);
		void operator()(const FCreateMemento& Command);
		void operator()(const FRestoreMemento& Command);
		void operator()(const FDestroyMemento& Command);
		void operator()(const FRemoveCompatibleUObject& Command);
		void operator()(const FRemoveCompatibleExternalObject& Command);
		void operator()(const FAddInteractiveSyncFromWorldTag& Command);
		void operator()(const FRemoveInteractiveSyncFromWorldTag& Command);
		void operator()(const FAddSyncFromWorldTag& Command);

		FString CommandDescriptions;
		bool bIncludeNops = false;

		static FString PrintToString(CompatibilityCommandBuffer::FCollection& Commands, bool bIncludeNops);
	};

	/** Looks at the current command and applies the next optimization. */
	struct FCommandOptimizer final
	{
		// TODO: Allow [add object + restore memento] to be folded into one and [create memento + remove object].
		//		Not sure how that will fit into the batch add though.
		FCommandOptimizer(CompatibilityCommandBuffer::FOptimizer& InOptimizer, FScratchBuffer& InScratchBuffer);

		template<typename T>
		void operator()(const T&) {}
		void operator()(const FAddCompatibleUObject& Command);
		void operator()(const FAddCompatibleExternalObject& Command);
		void operator()(const FRemoveCompatibleUObject& Command);
		void operator()(const FRemoveCompatibleExternalObject& Command);
		void operator()(const FAddInteractiveSyncFromWorldTag& Command);
		void operator()(const FRemoveInteractiveSyncFromWorldTag& Command);
		void operator()(const FAddSyncFromWorldTag& Command);

		CompatibilityCommandBuffer::FOptimizer& Optimizer;
		FScratchBuffer& ScratchBuffer;

		template<typename AddCommandType>
		int32 FoldCommandsForAdd(const AddCommandType& Command);
		void FoldSyncFromWorldTags(CompatibilityCommandBuffer::FOptimizer& Cluster);
		/**
		 * Creates a new optimizer starting at the left of the current optimizer that will run only as long as commands on the right use 
		 * the same row as the provided argument.
		 */
		CompatibilityCommandBuffer::FOptimizer CreateRangeOptimizer(RowHandle RowCluster);
		/** Keeps processing a subset of commands until the right no longer has a command with the provided source row. */
		void RunRightOnRowCluster(RowHandle RowCluster);
		static void Run(CompatibilityCommandBuffer::FCollection& Commands, FScratchBuffer& ScratchBuffer);
	};
} // namespace UE::Editor::DataStorage
