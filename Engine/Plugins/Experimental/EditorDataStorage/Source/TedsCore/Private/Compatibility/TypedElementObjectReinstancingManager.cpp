// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementObjectReinstancingManager.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "HAL/IConsoleManager.h"
#include "Memento/TypedElementMementoRowTypes.h"
#include "TypedElementDatabase.h"
#include "TypedElementDatabaseCompatibility.h"
#include "TypedElementDatabaseEnvironment.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementObjectReinstancingManager)

DECLARE_LOG_CATEGORY_CLASS(LogTedsObjectReinstancing, Log, Log)

UTedsObjectReinstancingManager::UTedsObjectReinstancingManager()
	: MementoRowBaseTable(UE::Editor::DataStorage::InvalidTableHandle)
{
}

void UTedsObjectReinstancingManager::Initialize(UEditorDataStorage& InDataStorage, UEditorDataStorageCompatibility& InDataStorageCompatibility)
{
	using namespace UE::Editor::DataStorage;

	DataStorage = &InDataStorage;
	DataStorageCompatibility = &InDataStorageCompatibility;
	
	UpdateCompletedCallbackHandle = 
		DataStorage->OnUpdateCompleted().AddUObject(this, &UTedsObjectReinstancingManager::UpdateCompleted);
	ReinstancingCallbackHandle = 
		FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &UTedsObjectReinstancingManager::HandleOnObjectsReinstanced);
	ObjectRemovedCallbackHandle = DataStorageCompatibility->RegisterObjectRemovedCallback(
		[this](const void* Object, const FObjectTypeInfo& TypeInfo, RowHandle Row)
		{
			HandleOnObjectPreRemoved(Object, TypeInfo, Row);
		});
}

void UTedsObjectReinstancingManager::Deinitialize()
{
	FCoreUObjectDelegates::OnObjectsReinstanced.Remove(ReinstancingCallbackHandle);
	DataStorageCompatibility->UnregisterObjectRemovedCallback(ObjectRemovedCallbackHandle);
	DataStorage->OnUpdateCompleted().Remove(UpdateCompletedCallbackHandle);

	DataStorageCompatibility = nullptr;
	DataStorage = nullptr;
}

void UTedsObjectReinstancingManager::UpdateCompleted()
{
	using namespace UE::Editor::DataStorage;

	FMementoSystem& MementoSystem = DataStorage->GetEnvironment().GetMementoSystem();
	for (TMap<const void*, RowHandle>::TConstIterator It = OldObjectToMementoMap.CreateConstIterator(); It; ++It)
	{
		MementoSystem.DestroyMemento(It.Value());
	}
	OldObjectToMementoMap.Reset();
}

void UTedsObjectReinstancingManager::HandleOnObjectPreRemoved(
	const void* Object, 
	const UE::Editor::DataStorage::FObjectTypeInfo& TypeInfo, 
	UE::Editor::DataStorage::RowHandle ObjectRow)
{
	// This is the chance to record the old object to memento
	UE::Editor::DataStorage::RowHandle Memento = DataStorage->GetEnvironment().GetMementoSystem().CreateMemento(ObjectRow);
	OldObjectToMementoMap.Add(Object, Memento);
}

void UTedsObjectReinstancingManager::HandleOnObjectsReinstanced(
	const FCoreUObjectDelegates::FReplacementObjectMap& ObjectReplacementMap)
{
	using namespace UE::Editor::DataStorage;

	ICoreProvider* Interface = DataStorage;
	for (FCoreUObjectDelegates::FReplacementObjectMap::TConstIterator Iter = ObjectReplacementMap.CreateConstIterator(); Iter; ++Iter)
	{
		const void* PreDeleteObject = Iter->Key;
		if (const RowHandle* MementoRowPtr = OldObjectToMementoMap.Find(PreDeleteObject))
		{
			RowHandle Memento = *MementoRowPtr;
			
			UObject* NewInstanceObject = Iter->Value;
			if (NewInstanceObject == nullptr)
			{
				continue;
			}
			
			RowHandle NewObjectRow = DataStorageCompatibility->FindRowWithCompatibleObjectExplicit(NewInstanceObject);
			// Do the addition only if there's a recorded memento. Having a memento implies the object was previously registered and there's
			// still an interest in it. Any other objects can therefore be ignored.
			if (!Interface->IsRowAvailable(NewObjectRow))
			{
				NewObjectRow = DataStorageCompatibility->AddCompatibleObjectExplicit(NewInstanceObject);
			}

			// Kick off re-instantiation of NewObjectRow from the Memento
			DataStorage->GetEnvironment().GetMementoSystem().RestoreMemento(Memento, NewObjectRow);
		}
	}
}
