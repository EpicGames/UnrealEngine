// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Memento/TypedElementMementoSystem.h"
#include "UObject/UObjectGlobals.h"

#include "TypedElementObjectReinstancingManager.generated.h"

class UEditorDataStorageCompatibility;
class UTypedElementMementoSystem;
class UEditorDataStorage;

namespace UE::Editor::DataStorage
{
	struct FObjectTypeInfo;
	class ICompatibilityProvider;
}

UCLASS(Transient)
class UTedsObjectReinstancingManager : public UObject
{
	GENERATED_BODY()
public:
	UTedsObjectReinstancingManager();

	void Initialize(UEditorDataStorage& InDataStorage, UEditorDataStorageCompatibility& InDataStorageCompatibility);
	void Deinitialize();

private:
	void UpdateCompleted();
	void HandleOnObjectPreRemoved(
		const void* Object, 
		const UE::Editor::DataStorage::FObjectTypeInfo& TypeInfo, 
		UE::Editor::DataStorage::RowHandle ObjectRow);
	void HandleOnObjectsReinstanced(const FCoreUObjectDelegates::FReplacementObjectMap& ObjectReplacementMap);

	UPROPERTY()
	TObjectPtr<UEditorDataStorage> DataStorage = nullptr;
	UPROPERTY()
	TObjectPtr<UEditorDataStorageCompatibility> DataStorageCompatibility = nullptr;
	
	// Reverse lookup that holds all populated mementos for recently deleted objects
	// Entry removed when the memento is removed
	TMap<const void*, UE::Editor::DataStorage::RowHandle> OldObjectToMementoMap;
	
	UE::Editor::DataStorage::TableHandle MementoRowBaseTable;
	FDelegateHandle UpdateCompletedCallbackHandle;
	FDelegateHandle ReinstancingCallbackHandle;
	FDelegateHandle ObjectRemovedCallbackHandle;
};
