// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementUObjectNameQueries.h"

#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementUObjectNameQueries)

void UObjectNameDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterUObjectAddNameColumn(DataStorage);
}

void UObjectNameDataStorageFactory::RegisterUObjectAddNameColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync UObject Name to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](const FTypedElementUObjectColumn& Object, FUObjectIdNameColumn& IdName)
			{
				if (UObject* ObjectInstance = Object.Object.Get())
				{
					IdName.IdName = ObjectInstance->GetFName();
				}
			}
		)
		.Where()
			.All<FTypedElementSyncFromWorldTag>()
		.Compile());
}
