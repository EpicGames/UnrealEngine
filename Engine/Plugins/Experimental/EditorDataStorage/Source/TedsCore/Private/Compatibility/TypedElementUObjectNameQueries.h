// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementUObjectNameQueries.generated.h"

UCLASS()
class UObjectNameDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UObjectNameDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:

	/**
	 * Adds the Name columns to new actors that do not have one already.
	 */
	void RegisterUObjectAddNameColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

};

