// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TypedElementUObjectWorldQueries.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementUObjectWorldQueries)

void UObjectWorldDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterAddWorldColumn(DataStorage);
	RegisterUpdateOrRemoveWorldColumn(DataStorage);
	RegisterPIEWorldSyncQueries(DataStorage);
}

void UObjectWorldDataStorageFactory::RegisterAddWorldColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Add world column to UObject"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				// Not all objects, in particular actors, are always correctly cleaned up, resulting in dangling
				// pointers in TEDS.
				if (UObject* ObjectInstance = Object.Object.Get())
				{
					if (UWorld* World = ObjectInstance->GetWorld())
					{
						Context.AddColumn(Row, FTypedElementWorldColumn{ .World = World });
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag>()
			.None<FTypedElementWorldColumn, FTypedElementClassDefaultObjectTag>()
		.Compile());
}

void UObjectWorldDataStorageFactory::RegisterUpdateOrRemoveWorldColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync UObject's world to column"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, FTypedElementWorldColumn& World)
			{
				// Not all objects, in particular actors, are always correctly cleaned up, resulting in dangling
				// pointers in TEDS.
				if (UObject* ObjectInstance = Object.Object.Get())
				{
					if (UWorld* UObjectWorld = ObjectInstance->GetWorld())
					{
						World.World = UObjectWorld;
						return;
					}
				}
				Context.RemoveColumns<FTypedElementWorldColumn>(Row);
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag>()
		.Compile());
}

void UObjectWorldDataStorageFactory::RegisterPIEWorldSyncQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync PIE Object Tag"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementWorldColumn& WorldColumn)
			{
				if (UWorld* World = WorldColumn.World.Get())
				{
					if (World->IsPlayInEditor())
					{
						Context.AddColumns<FPieObjectTag>(Row);
					}
					else
					{
						Context.RemoveColumns<FPieObjectTag>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag>()
		.Compile());
}
