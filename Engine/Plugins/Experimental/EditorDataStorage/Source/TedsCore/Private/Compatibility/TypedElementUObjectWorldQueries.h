// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementUObjectWorldQueries.generated.h"

UCLASS()
class UObjectWorldDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UObjectWorldDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Checks rows with UObjects that don't have a world column yet if one needs to be added whenever
	 * the row is marked for updates.
	 */
	void RegisterAddWorldColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Updates the world column with the world in the UObject or removes it if there's no world associated
	 * with the UObject anymore.
	 */
	void RegisterUpdateOrRemoveWorldColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	
	void RegisterPIEWorldSyncQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
