// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseDebugTypes.h"
#include "DataStorage/Features.h"
#include "DataStorage/Debug/Log.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "HAL/IConsoleManager.h"
#include "Misc/OutputDevice.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::Editor::DataStorage::Private
{
	static const UStruct* GetTypeInfo(const TWeakObjectPtr<const UClass>& TypeInfo) { return TypeInfo.Get(); }
	static const UStruct* GetTypeInfo(const TWeakObjectPtr<const UScriptStruct>& TypeInfo) { return TypeInfo.Get(); }
	static const UStruct* GetTypeInfo(const TObjectPtr<const UClass>& TypeInfo) { return TypeInfo; }
	static const UStruct* GetTypeInfo(const TObjectPtr<const UScriptStruct>& TypeInfo) { return TypeInfo; }
	static const UStruct* GetTypeInfo(const UClass* TypeInfo) { return TypeInfo; }
	static const UStruct* GetTypeInfo(const UScriptStruct* TypeInfo) { return TypeInfo; }

	template<typename TypeInfoType>
	void PrintObjectTypeInformation(ICoreProvider* DataStorage, FString Message, FOutputDevice& Output)
	{
		using namespace UE::Editor::DataStorage::Queries;

		static QueryHandle Query = DataStorage->RegisterQuery(
			Select()
				.ReadOnly<TypeInfoType>()
			.Compile());

		if (Query != InvalidQueryHandle)
		{
			DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding(
				[&Output, &Message](IDirectQueryContext& Context, const TypeInfoType* Types)
				{
					Message.Reset();
					Message += TEXT("  Batch start\n");

					TConstArrayView<TypeInfoType> TypeList(Types, Context.GetRowCount());
					for (const TypeInfoType& Type : TypeList)
					{
						if (const UStruct* TypeInfo = GetTypeInfo(Type.TypeInfo))
						{
							Message += TEXT("    Type: ");
							TypeInfo->AppendName(Message);
							Message += TEXT('\n');
						}
						else
						{
							Message += TEXT("    Type: [Invalid]\n");
						}
					}
					Message += TEXT("  Batch end\n");
					Output.Log(Message);
				}));
		}
	}

	template<typename... Conditions>
	void PrintObjectLabels(FOutputDevice& Output)
	{
		using namespace UE::Editor::DataStorage::Queries;

		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			static QueryHandle LabelQuery = [DataStorage]
			{
				if constexpr (sizeof...(Conditions) > 0)
				{
					return DataStorage->RegisterQuery(
						Select()
							.ReadOnly<FTypedElementUObjectColumn, FTypedElementLabelColumn>()
						.Where()
							.All(Conditions::StaticStruct()...)
						.Compile());
				}
				else
				{
					return DataStorage->RegisterQuery(
						Select()
							.ReadOnly<FTypedElementUObjectColumn, FTypedElementLabelColumn>()
						.Compile());
				}
			}();

			if (LabelQuery != InvalidQueryHandle)
			{
				FString Message;
				DataStorage->RunQuery(LabelQuery, CreateDirectQueryCallbackBinding(
					[&Output, &Message](IDirectQueryContext& Context, const FTypedElementUObjectColumn* Objects, const FTypedElementLabelColumn* Labels)
					{
						const uint32 Count = Context.GetRowCount();

						const FTypedElementLabelColumn* LabelsIt = Labels;
						int32 CharacterCount = 2; // Initial blank space and new line.
						// Reserve memory first to avoid repeated memory allocations.
						for (uint32 Index = 0; Index < Count; ++Index)
						{
							CharacterCount
								+= 4 /* Indention */
								+ 16 /* Hex address of actor */
								+ 2 /* Colon and space */
								+ LabelsIt->Label.Len()
								+ 1 /* Trailing new line */;
							++LabelsIt;
						}
						Message.Reset(CharacterCount);
						Message = TEXT(" \n");

						LabelsIt = Labels;
						for (uint32 Index = 0; Index < Count; ++Index)
						{
							Message.Appendf(TEXT("    0x%p: %s\n"), Objects->Object.Get(), *LabelsIt->Label);

							++LabelsIt;
							++Objects;
						}

						Output.Log(Message);
					}));
			}
		}
	}
} // namespace UE::Editor::DataStorage::Private

FAutoConsoleCommandWithOutputDevice PrintObjectTypeInformationConsoleCommand(
	TEXT("TEDS.Debug.PrintObjectTypeInfo"),
	TEXT("Prints the type information of any rows that has a type information column."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			using namespace UE::Editor::DataStorage;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.PrintObjectTypeInfo);

			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				FString Message;
				Output.Log(TEXT("The Typed Elements Data Storage has the types:"));
				Private::PrintObjectTypeInformation<FTypedElementClassTypeInfoColumn>(DataStorage, Message, Output);
				Private::PrintObjectTypeInformation<FTypedElementScriptStructTypeInfoColumn>(DataStorage, Message, Output);
				Output.Log(TEXT("End of Typed Elements Data Storage type list."));
			}
		}
	));

FAutoConsoleCommandWithOutputDevice PrintAllUObjectsLabelsConsoleCommand(
	TEXT("TEDS.Debug.PrintAllUObjectsLabels"),
	TEXT("Prints out the labels for all UObjects found in the Typed Elements Data Storage."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			using namespace UE::Editor::DataStorage;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.PrintAllUObjectsLabels);
			Output.Log(TEXT("The Typed Elements Data Storage has the following UObjects:"));
			Private::PrintObjectLabels(Output);
			Output.Log(TEXT("End of Typed Elements Data Storage UObjects list."));
		}));

FAutoConsoleCommandWithOutputDevice PrintActorLabelsConsoleCommand(
	TEXT("TEDS.Debug.PrintActorLabels"),
	TEXT("Prints out the labels for all actors found in the Typed Elements Data Storage."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			using namespace UE::Editor::DataStorage;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.PrintActorLabels);
			Output.Log(TEXT("The Typed Elements Data Storage has the following actors:"));
			Private::PrintObjectLabels<FTypedElementActorTag>(Output);
			Output.Log(TEXT("End of Typed Elements Data Storage actors list."));
		}));

FAutoConsoleCommandWithOutputDevice ListExtensionsConsoleCommand(
	TEXT("TEDS.Debug.ListExtensions"),
	TEXT("Prints a list for all available extension names."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.ListExtensions);
			using namespace UE::Editor::DataStorage;

			FString Message;
			auto RecordExtensions = [&Message](FName Extension)
			{
				Message += TEXT("    ");
				Extension.AppendString(Message);
				Message += TEXT('\n');
			};

			if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				Message = TEXT("Data Storage Extensions: \n");
				DataStorage->ListExtensions(RecordExtensions);
			}
			if (const ICompatibilityProvider* DataStorageCompat = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
			{
				Message += TEXT("Data Storage Compatibility Extensions: \n");
				DataStorageCompat->ListExtensions(RecordExtensions);
			}
			if (const IUiProvider* DataStorageUi = GetDataStorageFeature<IUiProvider>(UiFeatureName))
			{
				Message += TEXT("Data Storage UI Extensions: \n");
				DataStorageUi->ListExtensions(RecordExtensions);
			}

			Output.Log(Message);
		}
	));



static FAutoConsoleCommand CVarCreateRow(
	TEXT("TEDS.Debug.CreateRow"),
	TEXT("Argument: \n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		static TableHandle Table = DataStorage->RegisterTable<FTestColumnA>(FName(TEXT("Debug.CreateRow Table")));

		const RowHandle RowHandle = DataStorage->AddRow(Table);

		UE_LOGF(LogEditorDataStorage, Warning, "Added Row %llu", static_cast<uint64>(RowHandle));
	}));

static FAutoConsoleCommand CVarAddDynamicColumnTag(
	TEXT("TEDS.Debug.DynamicColumn.AddTag"),
	TEXT("Argument: Row, Identifier\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		if (Args.Num() != 2)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Must be 2");
			return;
		}

		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		const FName Identifier = FName(*Args[1]);

		DataStorage->AddColumn<FTestDynamicTag>(Row, Identifier);
		
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarAddDynamicColumn(
	TEXT("TEDS.Debug.DynamicColumn.AddColumn"),
	TEXT("Argument: Row, Identifier\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		if (Args.Num() != 2)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Row, TagId");
			return;
		}

		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		const FName Identifier = FName(*Args[1]);
		
		bool bUseDefaultApi = false;

		if (bUseDefaultApi)
		{
			DataStorage->AddColumn<FTestDynamicColumn>(Row, Identifier);
		}
		else
		{
			FTestDynamicColumn TemplateColumn;
			DataStorage->AddColumn(Row, Identifier, MoveTemp(TemplateColumn));
		}		
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarRemoveDynamicColumn(
	TEXT("TEDS.Debug.DynamicColumn.RemoveColumn"),
	TEXT("Argument: Row, Identifier\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		if (Args.Num() != 2)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Row, TagId, [optional] default=true/false");
			return;
		}

		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		const FName Identifier = FName(*Args[1]);
		
		DataStorage->RemoveColumn<FTestDynamicColumn>(Row, Identifier);
	}),
	ECVF_Default);

// Adds a value array stored in the dynamic column denoted with the given TagId
static FAutoConsoleCommand CVarAddToDynamicColumn(
	TEXT("TEDS.Debug.DynamicColumn.AddToColumn"),
	TEXT("Argument: Row, TagId, Value, [optional] MethodId\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		if (Args.Num() < 3 || Args.Num() > 4)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Row, TagId, Value");
			return;
		}

		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		const FName TagId = FName(*Args[1]);
		const int32 Value = FCString::Strtoi(*Args[2], nullptr, 10);
		const uint64 MethodId = Args.Num() >= 4 ? FCString::Strtoui64(*Args[3], nullptr, 10) : 0;

		// Two methods
		// 0
		if (MethodId == 0)
		{
			FTestDynamicColumn* Column = DataStorage->GetColumn<FTestDynamicColumn>(Row, TagId);
			if (!Column)
			{
				UE_LOGF(LogEditorDataStorage, Warning, "Row does not contain dynamic column");
				return;
			}
			Column->IntArray.Add(Value);
		}
		else if (MethodId == 1)
		{
			FTestDynamicColumn* Column = DataStorage->GetColumn<FTestDynamicColumn>(Row, TagId);
			if (!Column)
			{
				UE_LOGF(LogEditorDataStorage, Warning, "Row does not contain dynamic column. Creating one.");
				FTestDynamicColumn TemplateColumn;
				TemplateColumn.IntArray.Add(Value);
				DataStorage->AddColumn(Row, TagId, MoveTemp(TemplateColumn));
			}
			else
			{
				// Move the column to a temporary... then mutate it, then move it back
				FTestDynamicColumn TemplateColumn = MoveTemp(*Column);
				TemplateColumn.IntArray.Add(Value);
				DataStorage->AddColumn(Row, TagId, MoveTemp(TemplateColumn));
			}
		}
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarPrintDynamicColumn(
	TEXT("TEDS.Debug.DynamicColumn.PrintColumn"),
	TEXT("Argument: Row, TagId\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		if (Args.Num() != 2)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Row, TagId");
			return;
		}
		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		const FName TagId = FName(*Args[1]);

		FTestDynamicColumn* Column = DataStorage->GetColumn<FTestDynamicColumn>(Row, TagId);
		if (!Column)
		{
			UE_LOGF(LogEditorDataStorage, Warning, "Row does not contain dynamic column");
			return;
		}

		TStringBuilder<512> StringBuilder;
		StringBuilder.Append(TEXT("Array: \n"));
		const int32 EndIndex = Column->IntArray.Num();
		for (int32 Index = 0; Index < EndIndex - 1; ++Index)
		{
			StringBuilder.Appendf(TEXT("[%d] %llu\n"), Index, Column->IntArray[Index]);
		}
		if (const int32 LastIndex = EndIndex - 1; LastIndex >= 0)
		{
			StringBuilder.Appendf(TEXT("[%d] %llu\n"), LastIndex, Column->IntArray[LastIndex]);
		}
		else
		{
			StringBuilder.Append(TEXT("Empty"));
		}

		UE_LOGF(LogEditorDataStorage, Warning, "%ls", StringBuilder.ToString());
		
	}),
	ECVF_Default);

/**
 * A Command to illustrate building a query and callback to read a dynamic column
 */
static FAutoConsoleCommand CVarPrintDynamicColumnWithQuery(
	TEXT("TEDS.Debug.DynamicColumn.PrintColumnWithQuery"),
	TEXT("Argument: Identifier\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage::Queries;
		
		// Print column using query
		if (Args.Num() != 1)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Identifier");
			return;
		}
		
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		const FName Identifier(*Args[0]);

		const QueryHandle Query = DataStorage->RegisterQuery(
			Select().
				// Specify ReadOnly access to a dynamic column of type FTestDynamicColumn specified by the Identifier
				ReadOnly<FTestDynamicColumn>(Identifier).
			Compile());

		TStringBuilder<1024> StringBuilder;
		FQueryResult Result = DataStorage->RunQuery(Query,CreateDirectQueryCallbackBinding(
			[Identifier, &StringBuilder](IDirectQueryContext& Context, const RowHandle* Rows)
			{
				const TArrayView<const RowHandle> RowView = MakeConstArrayView(Rows, Context.GetRowCount());
				// Get pointer to the start of the range of columns to process
				const FTestDynamicColumn* DynamicColumnsRangeStart = Context.GetColumn<FTestDynamicColumn>(Identifier);
				const TArrayView<const FTestDynamicColumn> DynamicColumnView = MakeConstArrayView(DynamicColumnsRangeStart, Context.GetRowCount());
				
				for (int32 Index = 0, End = RowView.Num(); Index < End; ++Index)
				{
					StringBuilder.Appendf(TEXT("%llu: {"), static_cast<uint64>(RowView[Index]));
					const FTestDynamicColumn& DynamicColumn = DynamicColumnView[Index];
					if (!DynamicColumn.IntArray.IsEmpty())
					{
						for (int32 Value : DynamicColumn.IntArray)
						{
							StringBuilder.Appendf(TEXT("%d, "), Value);
						}
						// Trim the last ', '
						StringBuilder.RemoveSuffix(2);
					}
					StringBuilder.Append(TEXT("}\n"));
				}
			}));

		const int32 RowCount = Result.Count;
		StringBuilder.Appendf(TEXT("Processed '%d' items."), RowCount);
		
		DataStorage->UnregisterQuery(Query);

		UE_LOGF(LogEditorDataStorage, Warning, "%ls", StringBuilder.ToString());
		
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarCountDynamicTagWithQuery(
	TEXT("TEDS.Debug.DynamicColumn.CountDynamicTagWithQuery"),
	TEXT("Argument: Identifier\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage::Queries;
				
		// Print column using query
		if (Args.Num() != 1)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. Identifier");
			return;
		}
				
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				
		const FName Identifier(*Args[0]);

		const QueryHandle Query = DataStorage->RegisterQuery(
			Select().
				// Specify Any access to a dynamically created tag
				Where().
					Any<FTestDynamicTag>(Identifier).
			Compile());

		int32 Count = 0;
		DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding([Identifier, &Count](IDirectQueryContext& Context, const RowHandle* Rows)
		{
			Count += Context.GetRowCount();
		}));

		DataStorage->UnregisterQuery(Query);
		
		UE_LOGF(LogEditorDataStorage, Warning, "Processed '%d' items.", Count);
		
	}),
	ECVF_Default);


static TArray<UE::Editor::DataStorage::QueryHandle> DynamicColumnQueries;
/**
 * Registers an activatable query which will run against rows that have the FTestDynamicTag::<Identifier> column
 */
static FAutoConsoleCommand CVarRegisterListDynamicColumnQuery(
	TEXT("TEDS.Debug.DynamicColumn.RegisterListDynamicColumnQuery"),
	TEXT("Argument: ActivationGroup [Identifier]\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage::Queries;
				
		// Print column using query
		if (Args.Num() < 1 || Args.Num() > 2)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. ActivationGroup [Identifer]");
			return;
		}
				
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		const FName ActivationGroup(*Args[0]);
		FName Identifier;
		if (Args.Num() == 2)
		{
			Identifier = *Args[1];
		}

		// Lists the rows processed that have 
		QueryHandle Handle = DataStorage->RegisterQuery(
			Select(
				TEXT("ProcessDynamicTagColumns"),
				FProcessor(EQueryTickPhase::FrameEnd, DataStorage->GetQueryTickGroupName(EQueryTickGroups::Default))
					.MakeActivatable(ActivationGroup),
				[](IQueryContext& Context, const RowHandle* Rows)
				{
					auto RowView = MakeConstArrayView(Rows, Context.GetRowCount());
					for (RowHandle Row : RowView)
					{
						UE_LOGF(LogEditorDataStorage, Log, "- '%llu'\n", Row);
					}
				})
			.Where(TColumn<FTestDynamicTag>(Identifier))
			.Compile());

		DynamicColumnQueries.Add(Handle);

		UE_LOGF(LogEditorDataStorage, Log, "Query registered for Dynamic Column FTestDynamicTag::%ls with activation group '%ls'", *Identifier.ToString(), *ActivationGroup.ToString());
	}),
	ECVF_Default);

/**
 * Registers an activatable query which will run against rows that have the FTestDynamicTag::<Identifier> column
 */
static FAutoConsoleCommand CVarUnRegisterListDynamicColumnQueries(
	TEXT("TEDS.Debug.DynamicColumn.UnregisterListDynamicColumnQueries"),
	TEXT(""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);

		for (UE::Editor::DataStorage::QueryHandle& Handle : DynamicColumnQueries)
		{
			DataStorage->UnregisterQuery(Handle);
		}
		DynamicColumnQueries.Empty();
		
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarActivateListDynamicColumnQuery(
	TEXT("TEDS.Debug.DynamicColumn.ActivateListDynamicColumnQuery"),
	TEXT("Argument: ActivationGroup\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		// Print column using query
		if (Args.Num() != 1)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments. ActivationGroup");
			return;
		}

		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		const FName ActivationGroup(*Args[0]);

		DataStorage->ActivateQueries(ActivationGroup);
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarAddValueTag(
	TEXT("TEDS.Debug.ValueTag.AddColumn"),
	TEXT("Argument: Row, Tag, Value\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() != 3)
		{
			return;
		}
		
		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;

		
		const FName Value(*Args[2]);

		constexpr bool bUseTemplateSugar = true;
		if constexpr (bUseTemplateSugar)
		{
			const FName Tag = *Args[1];
			DataStorage->AddColumn<FValueTag>(Row, Tag, Value); 
		}
		else
		{
			const FValueTag Tag(*Args[1]);
			DataStorage->AddColumn(Row, Tag, Value);
		}
		
		
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarRemoveValueTag(
	TEXT("TEDS.Debug.ValueTag.RemoveColumn"),
	TEXT("Argument: Row, Group\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() != 2)
		{
			return;
		}
		
		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const UE::Editor::DataStorage::RowHandle Row = RowAsU64;
		
		constexpr bool bUseTemplateSugar = true;
		if constexpr (bUseTemplateSugar)
		{
			using namespace UE::Editor::DataStorage;
			const FName Tag = *Args[1];
			DataStorage->RemoveColumn<FValueTag>(Row, Tag);
		}
		else
		{
			const UE::Editor::DataStorage::FValueTag Tag(*Args[1]);
			DataStorage->RemoveColumn(Row, Tag);
		}		
	}),
	ECVF_Default);
	
static FAutoConsoleCommand CVarMatchValueTag(
	TEXT("TEDS.Debug.ValueTag.RunQuery"),
	TEXT("Argument: Tag, [optional] Value\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage::Queries;
		
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() < 1 || Args.Num() > 2)
		{
			return;
		}			

		const QueryHandle Query = [&Args, DataStorage]() -> QueryHandle
		{
			const FName Tag(*Args[0]);
			if (Args.Num() == 1)
			{
				// Matches all rows with the 
				return DataStorage->RegisterQuery(
					Select()
					.Where()
						// Match all rows with a value tag of type Tag (ie. all rows with a value tag of "Color")
						.All<FValueTag>(Tag)
						.All<FTestColumnA>()
					.Compile());
			}
			else
			{
				const FName MatchValue(*Args[1]);
				return DataStorage->RegisterQuery(
					Select().
					Where().
						// Match all rows with a value tag of type Tag that has a MatchValue (ie. all rows with value tag "Color" with value "Red")
						All<FValueTag>(Tag, MatchValue).
						All<FTestColumnA>().
					Compile());
			}
		}();

		uint64 Count = 0;
		
		const FQueryResult Result = DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding(
			[&Count](const IDirectQueryContext& Context, const RowHandle*)
			{
				Count += Context.GetRowCount();
			}));
		DataStorage->UnregisterQuery(Query);

		UE_LOGF(LogEditorDataStorage, Warning, "Processed %llu rows", static_cast<uint64>(Count));
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarAddValueTagFromEnum(
	TEXT("TEDS.Debug.ValueTag.AddWithEnum"),
	TEXT("Argument: Row, EnumValue\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() < 1 || Args.Num() > 2)
		{
			return;
		}
		
		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;

		if (Args.Num() == 1)
		{
			// Use of a enum value directly as a template parameter.  Only useful if enum value known at compile time
			DataStorage->AddColumn<ETedsDebugEnum::Red>(Row);
		}
		else
		{
			// Use an enum value from a runtime source
			// In this case, the argument is parsed and converted to an enum type
			UEnum* Enum = StaticEnum<ETedsDebugEnum>();
			int64 EnumValueAsI64 = Enum->GetValueByNameString(*Args[1]);
			if (EnumValueAsI64 == INDEX_NONE)
			{
				return;
			}
			const ETedsDebugEnum EnumValue = static_cast<ETedsDebugEnum>(EnumValueAsI64);
			
			DataStorage->AddColumn(Row, EnumValue);
		}
	}),
	ECVF_Default);

static FAutoConsoleCommand CVarRemoveValueTagFromEnum(
	TEXT("TEDS.Debug.ValueTag.RemoveWithEnum"),
	TEXT("Argument: Row\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() != 1)
		{
			return;
		}
		
		const uint64 RowAsU64 = FCString::Strtoui64(*Args[0], nullptr, 10);
		const RowHandle Row = RowAsU64;
		
		DataStorage->RemoveColumn<ETedsDebugEnum>(Row);
	}),
	ECVF_Default);


static FAutoConsoleCommand CVarMatchValueTagFromEnum(
	TEXT("TEDS.Debug.ValueTag.RunQueryEnum"),
	TEXT("Argument: [optional] EnumValue\n"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		using namespace UE::Editor::DataStorage::Queries;
		
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		if (Args.Num() > 1)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments");
			return;
		}

		if (Args.Num() == 1)
		{
			// Make sure that the given enum value is actually a value
			UEnum* Enum = StaticEnum<ETedsDebugEnum>();
			int64 EnumValue = Enum->GetValueByNameString(*Args[0]);
			if (EnumValue == INDEX_NONE)
			{
				return;
			}
		}

		const QueryHandle Query = [&Args, DataStorage]() -> QueryHandle
		{
			if (Args.Num() == 0)
			{
				// Matches all rows with the 
				return DataStorage->RegisterQuery(
					Select()
					.Where()
						// Match all rows with an enum value tag of the hardcoded enum type
						.All<ETedsDebugEnum>()
					.Compile());
			}
			else if (Args.Num() == 1)
			{
				UEnum* Enum = StaticEnum<ETedsDebugEnum>();
				int64 EnumValue = Enum->GetValueByNameString(*Args[0]);
				return DataStorage->RegisterQuery(
					Select().
					Where().
						// Match all rows with a value tag of the hardcoded enum type that has the given value
						// Note, usually this would be written something like:
						//   All(ETedsDebugEnum::Red).
						// However it isn't possible to do that when getting the enum value from a string.  API is still exercised
						// using the static_cast
						All(static_cast<ETedsDebugEnum>(EnumValue)).
					Compile());
			}
			else
			{
				return InvalidQueryHandle;
			}
		}();
		if (Query == InvalidQueryHandle)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Invalid number of arguments");
			return;
		}
		
		uint64 Count = 0;
		
		const FQueryResult Result = DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding(
			[&Count](const IDirectQueryContext& Context, const RowHandle*)
			{
				Count += Context.GetRowCount();
			}));
		DataStorage->UnregisterQuery(Query);

		UE_LOGF(LogEditorDataStorage, Warning, "Processed %llu rows", static_cast<uint64>(Count));
		
	}),
	ECVF_Default);
