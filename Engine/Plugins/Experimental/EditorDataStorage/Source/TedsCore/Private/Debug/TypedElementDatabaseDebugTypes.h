// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementDatabaseDebugTypes.generated.h"

UENUM()
enum class ETedsDebugEnum : int8
{
	Red,
	Blue,
	Green,
	Yellow,
	Black,
	Pink,
	Orange,
	Purple
};

USTRUCT(meta=(EditorDataStorage_DynamicColumnTemplate))
struct FTestDynamicColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Note: Not trivial type to check that non-default move/copying works
	// Should work regardless of UPROPERTY
	TArray<int32> IntArray;
};

USTRUCT(meta=(EditorDataStorage_DynamicColumnTemplate))
struct FTestDynamicTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
