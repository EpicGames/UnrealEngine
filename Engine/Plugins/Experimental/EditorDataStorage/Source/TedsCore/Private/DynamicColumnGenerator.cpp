// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicColumnGenerator.h"

#include "AssetDefinitionDefault.h"
#include "TypedElementDataStorageSharedColumn.h"
#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementColumnUtils.h"
#include "Kismet2/ReloadUtilities.h"
#include "Queries/TypedElementExtendedQueryStore.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "MassEntityManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicColumnGenerator)

namespace UE::Editor::DataStorage
{
	void FDynamicColumnGenerator::SetQueryStore(FExtendedQueryStore& InQueryStore)
	{
		QueryStore = &InQueryStore;
	}

	FDynamicColumnGeneratorInfo FDynamicColumnGenerator::GenerateColumn(const UScriptStruct& Template, const FName& Identifier)
	{
		const FTemplateIdKey Key
		{
			.Template = Template,
			.Identifier = Identifier,
		};

		if (!ensureMsgf(!Identifier.IsNone(), TEXT("Identifier cannot be None")))
		{
			return FDynamicColumnGeneratorInfo
			{
				.Type = nullptr,
				.Template = nullptr,
				.bNewlyGenerated = false
			};
		}
	
		FDynamicColumnGeneratorInfo GeneratedColumnInfo;
		UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);
		
		{
			const int32* GeneratedColumnIndex = TemplateIdLookup.Find(Key);
			if (GeneratedColumnIndex != nullptr)
			{
				const FGeneratedColumnRecord& GeneratedColumnRecord = GeneratedColumnData[*GeneratedColumnIndex];
				GeneratedColumnInfo.Type = GeneratedColumnRecord.Type;
				GeneratedColumnInfo.Template = GeneratedColumnRecord.Template;
				GeneratedColumnInfo.bNewlyGenerated = false;
				return GeneratedColumnInfo;
			}		
		}
		
		auto IsValidType = [](const UScriptStruct& Template)
		{
			return
				Template.IsChildOf(FColumn::StaticStruct()) ||
				Template.IsChildOf(FTag::StaticStruct()) ||
				Template.IsChildOf(FTedsSharedColumn::StaticStruct());
		};
		auto HasTemplateMetadata = [](const UScriptStruct& Template)
		{
			return UE::Editor::DataStorage::ColumnUtils::IsDynamicTemplate(&Template);
		};
		bool bCanUseTemplate = true;
		if (!IsValidType(Template))
		{
			ensureAlwaysMsgf(false,  TEXT("Template struct [%s] must derive from Column, Tag or SharedColumn"), *Template.GetName());
			bCanUseTemplate = false;
		}
		if (!HasTemplateMetadata(Template))
		{
			ensureAlwaysMsgf(false,  TEXT("Template struct [%s] must be declared with 'meta=(EditorDataStorage_DynamicColumnTemplate)'"), *Template.GetName());
			bCanUseTemplate = false;
		}
		if (!bCanUseTemplate)
		{
			return FDynamicColumnGeneratorInfo
			{
				.Type = nullptr,
				.Template = nullptr,
				.bNewlyGenerated = false
			};
		}

		TemplateTypeLookup.Add(&Template);
	
		{
			checkf(Template.GetCppStructOps() != nullptr && Template.IsNative(), TEXT("Can only create column from native struct"));

			TStringBuilder<256> ObjectNameBuilder;
			ObjectNameBuilder.Append(Template.GetName());
			ObjectNameBuilder.Append(TEXT("::"));
			ObjectNameBuilder.Append(Identifier.ToString());

			const FName ObjectName = FName(ObjectNameBuilder);

			// Per-column transient package. Isolates this struct's metadata map from any
			// shared package whose metadata other systems may write to concurrently.
			TStringBuilder<256> PackagePathBuilder;
			PackagePathBuilder.Append(TEXT("/Engine/TedsDynamicColumns/"));
			PackagePathBuilder.Append(Template.GetName());
			PackagePathBuilder.Append(TEXT("_"));
			Identifier.AppendString(PackagePathBuilder);
			UPackage* ColumnPackage = NewObject<UPackage>(nullptr, *FString(PackagePathBuilder), RF_Transient);

			UScriptStruct* NewScriptStruct = NewObject<UScriptStruct>(ColumnPackage, ObjectName);
			// Ensure it is not garbage collected
			// FDynamicColumnGenerator is not a UObject and thus does not participate in GC.
			// The column's package is kept alive via the struct's Outer reference.
			NewScriptStruct->AddToRoot();
	
			// New struct subclasses the template to allow for casting back to template and usage of CppStructOps
			// for copy/move.
			NewScriptStruct->SetSuperStruct(&const_cast<UScriptStruct&>(Template));
			
			NewScriptStruct->Bind();
			NewScriptStruct->PrepareCppStructOps();
			NewScriptStruct->StaticLink(true);
			// Set metadata to indicate that this struct is derived from a dynamic template
			// The value will contain the identifier
			TStringBuilder<256> IdentifierStringBuilder;
			Identifier.AppendString(IdentifierStringBuilder);
			ColumnPackage->GetMetaData().SetValue(NewScriptStruct, TEXT("EditorDataStorage_DerivedFromDynamicTemplate"), IdentifierStringBuilder.ToString());
			const int32 Index = GeneratedColumnData.Emplace(FGeneratedColumnRecord
			{
				.Identifier = Identifier,
				.Template = &Template,
				.Type = NewScriptStruct
			});
			
			TemplateIdLookup.Add(Key, Index);
			GeneratedTypeLookup.Add(NewScriptStruct, Index);
			
			GeneratedColumnInfo.Type = NewScriptStruct;
			GeneratedColumnInfo.Template = &Template;
			GeneratedColumnInfo.bNewlyGenerated = true;

			if (QueryStore)
			{
				QueryStore->NotifyNewDynamicColumn(GeneratedColumnInfo);
			}
			
			return GeneratedColumnInfo;
		}
	}

	void FDynamicColumnGenerator::ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const FDynamicColumnGeneratorInfo&)> Callback) const
	{
		UE_MT_SCOPED_READ_ACCESS(AccessDetector);

		for (const FGeneratedColumnRecord& Record : GeneratedColumnData)
		{
			if (Record.Type->IsChildOf(&Template))
			{
				FDynamicColumnGeneratorInfo Info;
				Info.Template = &Template;
				Info.Type = Record.Type;
				Info.bNewlyGenerated = false;
				Callback(Info);
			}
		}
	}

	bool FDynamicColumnGenerator::IsDynamicTemplate(const UScriptStruct& InCandidate) const
	{
		return TemplateTypeLookup.Find(&InCandidate) != nullptr;
	}

	const UScriptStruct* FDynamicColumnGenerator::FindByTemplateId(const UScriptStruct& Template, const FName& Identifier) const
	{
		UE_MT_SCOPED_READ_ACCESS(AccessDetector);
	
		const int32* IndexPtr = TemplateIdLookup.Find(FTemplateIdKey{
			.Template = Template,
			.Identifier = Identifier
			});
		if (IndexPtr)
		{
			return GeneratedColumnData[*IndexPtr].Type;
		}
		return nullptr;
	}

	bool FDynamicColumnGenerator::FindByGeneratedType(const UScriptStruct& GeneratedType, FDynamicColumnGeneratorInfo& Out) const
	{
		UE_MT_SCOPED_READ_ACCESS(AccessDetector);

		const int32* IndexPtr = GeneratedTypeLookup.Find(&GeneratedType);
		if (IndexPtr)
		{
			const FGeneratedColumnRecord& Record = GeneratedColumnData[*IndexPtr];
			Out.Type = Record.Type;
			Out.Template = Record.Template;
			Out.bNewlyGenerated = false;
			return true;
		}
		return false;
	}

	FValueTagManager::FValueTagManager(FMassEntityManager& InEntityManager, FDynamicColumnGenerator& InColumnGenerator)
		: EntityManager(InEntityManager)
		, ColumnGenerator(InColumnGenerator)
	{
	}
	
	FConstSharedStruct FValueTagManager::GenerateValueTag(const FValueTag& InTag, const FName& InValue)
	{
		TPair<FValueTag, FName> Pair(InTag, InValue);
	
		UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);
		
		const UScriptStruct* ColumnType = GenerateColumnType(InTag);
		const FTedsValueTagColumn Overlay
		{
			.Value = InValue
		};
		
		const FConstSharedStruct& SharedStruct = EntityManager.GetOrCreateConstSharedFragment(*ColumnType, reinterpret_cast<const uint8*>(&Overlay));
		return SharedStruct;
	}
	
	const UScriptStruct* FValueTagManager::GenerateColumnType(const FValueTag& Tag)
	{
		const FDynamicColumnGeneratorInfo GeneratedColumnType = ColumnGenerator.GenerateColumn(*FTedsValueTagColumn::StaticStruct(), Tag.GetName());
		
		return GeneratedColumnType.Type;
	}
} // namespace UE::Editor::DataStorage
