// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TypedElementDataStorageSharedColumn.h"
#include "Containers/Map.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/MTAccessDetector.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/NameTypes.h"
#include "UObject/Class.h"

#include "DynamicColumnGenerator.generated.h"

class FName;
class UScriptStruct;
struct FMassEntityManager;
// The template struct that is used to generate the ValueTag column
USTRUCT(meta=(EditorDataStorage_DynamicColumnTemplate))
struct FTedsValueTagColumn : public FTedsSharedColumn
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName Value;
};

namespace UE::Editor::DataStorage
{
	class FExtendedQueryStore;
	using FValueTagColumn = FTedsValueTagColumn;

	struct FDynamicColumnInfo
	{
		const UScriptStruct* Type;
	};
	
	struct FDynamicColumnGeneratorInfo
	{
		const UScriptStruct* Type;
		const UScriptStruct* Template;
		bool bNewlyGenerated;
	};

	/**
	 * Utility class that TEDS can use to dynamically generate column types on the fly
	 */
	class FDynamicColumnGenerator
	{
	public:
		void SetQueryStore(FExtendedQueryStore& InQueryStore);
		
		/**
		 * Generates a dynamic TEDS column type based on a Template type (if it hasn't been generated before)
		 */
		FDynamicColumnGeneratorInfo GenerateColumn(const UScriptStruct& Template, const FName& Identifier);

		void ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const FDynamicColumnGeneratorInfo&)> Callback) const;

		bool IsDynamicTemplate(const UScriptStruct& InCandidate) const;
		const UScriptStruct* FindByTemplateId(const UScriptStruct& Template, const FName& Identifier) const;
		// Returns true if found, false otherwise.
		// If found, Out will be populated
		bool FindByGeneratedType(const UScriptStruct& GeneratedType, FDynamicColumnGeneratorInfo& Out) const;
	private:

		struct FGeneratedColumnRecord
		{
			FName Identifier;
			const UScriptStruct* Template;
			const UScriptStruct* Type;
		};

		struct FTemplateIdKey
		{
			const UScriptStruct& Template;
			FName Identifier;

			friend bool operator==(const FTemplateIdKey& Lhs, const FTemplateIdKey& Rhs)
			{
				return Lhs.Identifier == Rhs.Identifier && &Lhs.Template == &Rhs.Template;
			}

			friend bool operator!=(const FTemplateIdKey& Lhs, const FTemplateIdKey& Rhs)
			{
				return !operator==(Lhs, Rhs);
			}

			friend uint32 GetTypeHash(const FTemplateIdKey& Key)
			{
				return HashCombineFast(GetTypeHash(Key.Identifier), PointerHash(&Key.Template));
			}
		};

		UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector);

		TArray<FGeneratedColumnRecord> GeneratedColumnData;

		TSet<const UScriptStruct*> TemplateTypeLookup;
		
		// Looks up generated column index by template and ID used to generate it
		// Used to de-duplicate
		TMap<FTemplateIdKey, int32> TemplateIdLookup;

		// Looks up generated column data by the generated column type
		TMap<const UScriptStruct*, int32> GeneratedTypeLookup;

		FExtendedQueryStore* QueryStore = nullptr;
	};

	class FValueTagManager
	{
	public:
		explicit FValueTagManager(FMassEntityManager& InEntityManager, FDynamicColumnGenerator& InColumnGenerator);
		FConstSharedStruct GenerateValueTag(const FValueTag& InTag, const FName& InValue);
		const UScriptStruct* GenerateColumnType(const FValueTag& InTag);
	private:

		UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector);

		FMassEntityManager& EntityManager;
		FDynamicColumnGenerator& ColumnGenerator;
	};
} // namespace UE::Editor::DataStorage
