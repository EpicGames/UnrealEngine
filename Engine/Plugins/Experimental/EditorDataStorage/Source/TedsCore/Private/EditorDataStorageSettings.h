// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "EditorDataStorageSettings.generated.h"

UENUM()
enum class EChunkMemorySize : int32
{
	Size4Kb = 4 * 1024 UMETA(DisplayName = "4kb"),
	Size8Kb = 8 * 1024 UMETA(DisplayName = "8kb"),
	Size16Kb = 16 * 1024 UMETA(DisplayName = "16kb"),
	Size32Kb = 32 * 1024 UMETA(DisplayName = "32kb"),
	Size64Kb = 64 * 1024 UMETA(DisplayName = "64kb"),
	Size128Kb = 128 * 1024 UMETA(DisplayName = "128kb"),
	Size256Kb = 256 * 1024 UMETA(DisplayName = "256kb"),
	Size512Kb = 512 * 1024 UMETA(DisplayName = "512kb")
};

UCLASS(config = EditorPerProjectUserSettings, MinimalAPI)
class UEditorDataStorageSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category = MassSettings)
	EChunkMemorySize ChunkMemorySize = EChunkMemorySize::Size128Kb;

	UPROPERTY(EditAnywhere, config, Category = MassSettings)
	TMap<FName, EChunkMemorySize> TableSpecificChunkMemorySize;
};
