// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalLock.h"

namespace UE::Editor::DataStorage
{
	//
	// FGlobalLock
	//

	FTransactionallySafeRWLock FGlobalLock::Lock;
	thread_local EGlobalLockStatus FGlobalLock::LockStatus = EGlobalLockStatus::Unlocked;
	thread_local uint64 FGlobalLock::LockCount = 0;
	std::atomic<EGlobalLockStatus> FGlobalLock::InternalLockStatus = EGlobalLockStatus::Unlocked;

	void FGlobalLock::SharedLock(EGlobalLockScope Scope)
	{
		if (Scope == EGlobalLockScope::Public)
		{
			// The requirements for a shared lock are also satisfied if this thread has an exclusive lock.
			if (LockStatus == EGlobalLockStatus::Unlocked)
			{
				Lock.ReadLock();
				LockStatus = EGlobalLockStatus::SharedLocked;
			}
			LockCount++;
		}
		else
		{
			checkf(InternalLockStatus == EGlobalLockStatus::SharedLocked || LockStatus != EGlobalLockStatus::Unlocked,
				TEXT("Requesting an global internal shared lock in TEDS while the internal or shared lock hasn't been acquired."));
		}
	}

	void FGlobalLock::ExclusiveLock(EGlobalLockScope Scope)
	{
		if (Scope == EGlobalLockScope::Public)
		{
			if (LockStatus != EGlobalLockStatus::ExclusiveLocked)
			{
				checkf(LockStatus != EGlobalLockStatus::SharedLocked,
					TEXT("Attempting to get a global TEDS exclusive lock on a thread that's already shared locked."));
				Lock.WriteLock();
				LockStatus = EGlobalLockStatus::ExclusiveLocked;
			}
			LockCount++;
		}
		else
		{
			checkf(false, TEXT("Internal exclusive locks for TEDS can't be safely requested as internal locks are shared only."));
		}
	}

	void FGlobalLock::Unlock(EGlobalLockScope Scope)
	{
		if (Scope == EGlobalLockScope::Public)
		{
			switch (LockStatus)
			{
			default:
				// fall through.
			case EGlobalLockStatus::Unlocked:
				checkf(false, TEXT("Attempting to unlock the global TEDS lock that wasn't locked."));
				break;
			case EGlobalLockStatus::SharedLocked:
				checkf(LockCount > 0, TEXT("Attempting to unlock the global TEDS lock while its lock count is already zero."));
				if (--LockCount == 0)
				{
					LockStatus = EGlobalLockStatus::Unlocked;
					Lock.ReadUnlock();
				}
				break;
			case EGlobalLockStatus::ExclusiveLocked:
				checkf(LockCount > 0, TEXT("Attempting to unlock the global TEDS lock while its lock count is already zero."));
				if (--LockCount == 0)
				{
					LockStatus = EGlobalLockStatus::Unlocked;
					Lock.WriteUnlock();
				}
				break;
			}
		}
	}

	EGlobalLockStatus FGlobalLock::GetLockStatus(EGlobalLockScope Scope)
	{
		return Scope == EGlobalLockScope::Public ? LockStatus : InternalLockStatus.load();
	}

	void FGlobalLock::InternalSharedLock()
	{
		checkf(InternalLockStatus == 
			EGlobalLockStatus::Unlocked, TEXT("Attempting to acquire a global internal TEDS lock while there is already an internal lock."));
		// Get a public shared lock on behalf of all internal locks.
		SharedLock(EGlobalLockScope::Public);
		InternalLockStatus = EGlobalLockStatus::SharedLocked;
	}
	
	void FGlobalLock::InternalSharedUnlock()
	{
		checkf(InternalLockStatus ==
			EGlobalLockStatus::SharedLocked, TEXT("Attempting to release a global internal TEDS lock while there is not internal lock."));
		InternalLockStatus = EGlobalLockStatus::Unlocked;
		Unlock(EGlobalLockScope::Public);
	}


	//
	// FScopedSharedLock
	//

	FScopedSharedLock::FScopedSharedLock(EGlobalLockScope InScope)
		: Scope(InScope)
	{
		FGlobalLock::SharedLock(Scope);
	}

	FScopedSharedLock::~FScopedSharedLock()
	{
		FGlobalLock::Unlock(Scope);
	}


	//
	// FScopedExclusiveLock
	//

	FScopedExclusiveLock::FScopedExclusiveLock(EGlobalLockScope InScope)
		:Scope(InScope)
	{
		FGlobalLock::ExclusiveLock(Scope);
	}

	FScopedExclusiveLock::~FScopedExclusiveLock()
	{
		FGlobalLock::Unlock(Scope);
	}
} // UE::Editor::DataStorage