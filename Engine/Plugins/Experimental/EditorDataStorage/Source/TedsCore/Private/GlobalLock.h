// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/TransactionallySafeRWLock.h"

namespace UE::Editor::DataStorage
{
	enum class EGlobalLockStatus : uint8
	{
		Unlocked,
		SharedLocked,
		ExclusiveLocked
	};

	enum class EGlobalLockScope : uint8
	{
		/** The lock request is coming from a public facing call. */
		Public,
		/** 
		 * The lock is coming from an internal call. These typically have a wider range of
		 * calling threads, but only support shared access. Exclusive locks will check for
		 * validity.
		 */
		Internal,
	};

	/**
	 * Sets a lock for all of TEDS systems. This is used whenever access to any data within TEDS is needed in a thread-safe manner. Shared 
	 * locks will allow multiple thread to safely read data, while exclusive locks provide safe read/write access.
	 * 
	 * The Global lock has an internal lock so shared locks can be freely handed out to requests with an internal scope. Exclusive 
	 * locks with the same scope will assert. This is meant to only be called by TEDS Core to lock the global lock for the duration of
	 * a processing phase. This is the only safe time to call it as all operations within a phase will be processed on all threads before
	 * a phase ends. If this is used for global calls it's possible that such a call is still processing when TEDS Core releases the lock
	 * and another exclusive lock is acquired. Therefore global locks fall back to full lock. Use global locks except from within adaptors
	 * that run in Mass processors.
	 * 
	 * The global lock uses internal reference counting to allow recursive calls from the same thread. Further state management also helps
	 * track incorrect requests for an exclusive lock on a thread that already has a shared lock. An assert triggers in this case to prevent
	 * deadlocks.
	 */
	class FGlobalLock
	{
	public:
		static void SharedLock(EGlobalLockScope Scope);
		static void ExclusiveLock(EGlobalLockScope Scope);
		static void Unlock(EGlobalLockScope Scope);
		
		static EGlobalLockStatus GetLockStatus(EGlobalLockScope Scope);

	private:
		static void InternalSharedLock();
		static void InternalSharedUnlock();

		static FTransactionallySafeRWLock Lock;
		/** Keep track of the lock status because FRWLock is non-recursive and with a global lock it's easy to get recursive calls. */
		static thread_local EGlobalLockStatus LockStatus;
		static thread_local uint64 LockCount;
		static std::atomic<EGlobalLockStatus> InternalLockStatus;
	};

	class FScopedSharedLock
	{
	public:
		explicit FScopedSharedLock(EGlobalLockScope InScope);
		~FScopedSharedLock();

	private:
		EGlobalLockScope Scope;
	};

	class FScopedExclusiveLock
	{
	public:
		explicit FScopedExclusiveLock(EGlobalLockScope InScope);
		~FScopedExclusiveLock();

	private:
		EGlobalLockScope Scope;
	};
} // namespace UE::Editor::DataStorage
