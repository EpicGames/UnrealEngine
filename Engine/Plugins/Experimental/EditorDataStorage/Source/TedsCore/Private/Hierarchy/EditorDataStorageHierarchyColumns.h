// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/MapKey.h"

#include "EditorDataStorageHierarchyColumns.generated.h"

USTRUCT(meta = (DisplayName = "EditorDataHierarchyData (Template)", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataHierarchyData_Template final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FTedsRowHandle Parent;

	UPROPERTY()
	TArray<FTedsRowHandle> Children;
};

USTRUCT(meta = (DisplayName = "EditorDataHierarchyParentTag (Template)", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataHierarchyParentTag_Template : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "EditorDataHierarchyChildTag (Template)", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataHierarchyChildTag_Template : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Unresolved Parent (Template)", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataHierarchyUnresolvedParent_Template final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// The Id used to lookup the parent row every frame
	UE::Editor::DataStorage::FMapKey ParentId;
	
	// The mapping domain the ParentID is looked up in
	FName MappingDomain;
	
};

USTRUCT(meta = (DisplayName = "Parent Changed Column (Template)", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataHierarchyParentChangedTag_Template final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};


namespace UE::Editor::DataStorage
{
	using FHierarchyData_Template = FEditorDataHierarchyData_Template;
	using FHierarchyParentTag_Template = FEditorDataHierarchyParentTag_Template;
	using FHierarchyChildTag_Template = FEditorDataHierarchyChildTag_Template;
	using FHierarchyUnresolvedParentColumn_Template = FEditorDataHierarchyUnresolvedParent_Template;
}
