// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDataStorageHierarchyImplementation.h"

#include "EditorDataStorageHierarchyColumns.h"

namespace UE::Editor::DataStorage
{
	// ──────────────────────────────────────────────────────────────────────────
	// Concrete hierarchy backend declarations (private to this translation unit)
	// ──────────────────────────────────────────────────────────────────────────

	/**
	 * Adjacency-list (legacy) hierarchy backend.
	 * Stores parent/child data in columns directly on participant rows.
	 */
	class FAdjacencyListBackedHierarchyAccess final : public FTedsHierarchyAccessInterface
	{
	public:
		struct FConstructParams
		{
			const UScriptStruct* ChildTag = nullptr;
			const UScriptStruct* ParentTag = nullptr;
			const UScriptStruct* HierarchyData = nullptr;
			const UScriptStruct* UnresolvedParentColumn = nullptr;
			const UScriptStruct* ParentChangedColumn = nullptr;
		};

		explicit FAdjacencyListBackedHierarchyAccess(const FConstructParams& Params);

		const UScriptStruct* GetChildTagType() const override;
		const UScriptStruct* GetParentTagType() const override;
		const UScriptStruct* GetHierarchyDataColumnType() const override;
		const UScriptStruct* GetUnresolvedParentColumnType() const override;
		const UScriptStruct* GetParentChangedColumnType() const override;

		void SetParentRow(ICoreProvider* CoreProvider, RowHandle Target, RowHandle Parent) const override;
		void SetUnresolvedParent(ICoreProvider* CoreProvider, RowHandle Target, FMapKey ParentId, FName MappingDomain) const override;
		RowHandle GetParentRow(const ICoreProvider* CoreProvider, RowHandle Target) const override;

		bool HasChildren(const ICoreProvider& Context, RowHandle Row) const override;
		void WalkDepthFirst(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)> OnVisitedFn,
			ICoreProvider::ETraversalOrder TraversalOrder) const override;
		bool IterateChildren(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> OnVisitedFn) const override;

		TFunction<RowHandle(const void*, const UScriptStruct*)> CreateParentExtractionFunction() const override;

	protected:
		RowHandle GetParent(const ICommonQueryContext& Context, RowHandle Target) const override;

	private:
		const UScriptStruct* ChildTag = nullptr;
		const UScriptStruct* ParentTag = nullptr;
		const UScriptStruct* HierarchyDataColumnType = nullptr;
		const UScriptStruct* UnresolvedParentColumnType = nullptr;
		const UScriptStruct* ParentChangedColumn = nullptr;
	};

	/**
	 * Relations-backed hierarchy backend.
	 * Stores hierarchy data on dedicated relation rows, using interval encoding for O(1) descent queries.
	 */
	class FTedsRelationBackedHierarchyAccess final : public FTedsHierarchyAccessInterface
	{
	public:
		FTedsRelationBackedHierarchyAccess(RelationTypeHandle InRelationType,
			const UScriptStruct* InUnresolvedParentColumnType,
			const UScriptStruct* InParentChangedColumn = nullptr);

		// Relations backend doesn't use tag/data columns on participant rows.
		const UScriptStruct* GetChildTagType() const override { return nullptr; }
		const UScriptStruct* GetParentTagType() const override { return nullptr; }
		const UScriptStruct* GetHierarchyDataColumnType() const override { return nullptr; }
		const UScriptStruct* GetUnresolvedParentColumnType() const override { return UnresolvedParentColumnType; }
		const UScriptStruct* GetParentChangedColumnType() const override { return ParentChangedColumnType; }

		void SetParentRow(ICoreProvider* CoreProvider, RowHandle Target, RowHandle Parent) const override;
		void SetUnresolvedParent(ICoreProvider* CoreProvider, RowHandle Target, FMapKey ParentId, FName MappingDomain) const override;
		RowHandle GetParentRow(const ICoreProvider* CoreProvider, RowHandle Target) const override;

		bool HasChildren(const ICoreProvider& Context, RowHandle Row) const override;
		void WalkDepthFirst(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)> OnVisitedFn,
			ICoreProvider::ETraversalOrder TraversalOrder) const override;
		bool IterateChildren(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> OnVisitedFn) const override;

		TFunction<RowHandle(const void*, const UScriptStruct*)> CreateParentExtractionFunction() const override;

	private:
		RelationTypeHandle HierarchyRelationType;
		const UScriptStruct* UnresolvedParentColumnType = nullptr;
		const UScriptStruct* ParentChangedColumnType = nullptr;
	};

	// ──────────────────────────────────────────────────────────────────────────

	// Interpretation for the opaque FHierarchyHandle.
	// Backend field distinguishes Legacy (column-on-row) from Relations (relation-row) handles.
	struct FHierarchyHandleInterpretation
	{
		int32 Index;
		bool IsSet;
		uint8 Backend;
	};

	static int32 GetHandleIndex(const FHierarchyHandle& Handle)
	{
		const FHierarchyHandleInterpretation& Interp = reinterpret_cast<const FHierarchyHandleInterpretation&>(Handle);
		return Interp.IsSet ? Interp.Index : INDEX_NONE;
	}

	static EHierarchyBackend GetHandleBackend(const FHierarchyHandle& Handle)
	{
		const FHierarchyHandleInterpretation& Interp = reinterpret_cast<const FHierarchyHandleInterpretation&>(Handle);
		return Interp.IsSet ? static_cast<EHierarchyBackend>(Interp.Backend) : EHierarchyBackend::Legacy;
	}

	static FHierarchyHandle CreateHandle(int32 Index, EHierarchyBackend Backend = EHierarchyBackend::Legacy)
	{
		FHierarchyHandle Handle;

		FHierarchyHandleInterpretation& Interp = reinterpret_cast<FHierarchyHandleInterpretation&>(Handle);
		new (&Interp) FHierarchyHandleInterpretation();
		Interp.Index = Index;
		Interp.IsSet = true;
		Interp.Backend = static_cast<uint8>(Backend);

		return Handle;
	}

	static_assert(sizeof(FHierarchyHandleInterpretation) <= sizeof(FHierarchyHandle));
	static_assert(alignof(FHierarchyHandleInterpretation) <= alignof(FHierarchyHandle));
	static_assert(std::is_trivially_copyable_v<FHierarchyHandle>);
	static_assert(std::is_trivially_copyable_v<FHierarchyHandleInterpretation>);
	
	const FTedsHierarchyAccessInterface* FTedsHierarchyRegistrar::GetAccessInterface(FHierarchyHandle Handle) const
	{
		if (int32 Index = GetHandleIndex(Handle); Index != INDEX_NONE)
		{
			return RegisteredHierarchies[Index].AccessInterface.Get();
		}
		return nullptr;
	}

	void FTedsHierarchyRegistrar::ListHierarchyNames(TFunctionRef<void(const FName& HierarchyName)> Callback) const
	{
		for (int32 HierarchyIndex = 0; HierarchyIndex < RegisteredHierarchies.Num(); ++HierarchyIndex)
		{
			Callback(RegisteredHierarchies[HierarchyIndex].Name);
		}
	}

	void FTedsHierarchyRegistrar::ResolveHierarchy(ICoreProvider& CoreProvider, FHierarchyHandle HierarchyHandle)
	{
		using namespace UE::Editor::DataStorage::Queries;
	
		if (int32 Index = GetHandleIndex(HierarchyHandle); Index != INDEX_NONE)
		{
			if (FTedsHierarchyAccessInterface* AccessInterface = RegisteredHierarchies[Index].AccessInterface.Get())
			{
				// Lazily register this query when ResolveHierarchy is called for this hierarchy handle the first time
				if (RegisteredHierarchies[Index].UnresolvedChildRowsQueryHandle == InvalidQueryHandle)
				{
					RegisteredHierarchies[Index].UnresolvedChildRowsQueryHandle = CoreProvider.RegisterQuery(
						Select()
						.Where()
							.All(AccessInterface->GetUnresolvedParentColumnType())
						.Compile());
				}
				
				FRowHandleArray CollectedRows;
				
				CoreProvider.RunQuery(RegisteredHierarchies[Index].UnresolvedChildRowsQueryHandle, CreateDirectQueryCallbackBinding(
					[&CollectedRows](IDirectQueryContext& Context, const RowHandle*)
					{
						CollectedRows.Append(Context.GetRowHandles());
					}));
				
				// Iterate over each row and resolve the hierarchy if the parent is found now
				for (RowHandle UnresolvedChildRow : CollectedRows.GetRows())
				{
					if (const void* UnresolvedParentColumnData = CoreProvider.GetColumnData(UnresolvedChildRow, AccessInterface->GetUnresolvedParentColumnType()))
					{
						const FEditorDataHierarchyUnresolvedParent_Template* UnresolvedParentColumn = 
							static_cast<const FEditorDataHierarchyUnresolvedParent_Template*>(UnresolvedParentColumnData);
						
						RowHandle ParentRow = CoreProvider.LookupMappedRow(UnresolvedParentColumn->MappingDomain, UnresolvedParentColumn->ParentId);
						
						if (CoreProvider.IsRowAvailable(ParentRow))
						{
							AccessInterface->SetParentRow(&CoreProvider, UnresolvedChildRow, ParentRow);
						}
					}
				}
			}
		}
		
	}

	FTedsHierarchyRegistrar::~FTedsHierarchyRegistrar()
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			for (int32 HierarchyIndex = 0; HierarchyIndex < RegisteredHierarchies.Num(); ++HierarchyIndex)
			{
				Storage->UnregisterQuery(RegisteredHierarchies[HierarchyIndex].UnresolvedChildRowsQueryHandle);
			}
		}
	}

	FHierarchyHandle FTedsHierarchyRegistrar::RegisterHierarchy(ICoreProvider* InProvider, const FHierarchyRegistrationParams& RegistrationParams)
	{
		if (RegistrationParams.Backend == FHierarchyRegistrationParams::EBackend::Relations)
		{
			return RegisterHierarchyRelations(InProvider, RegistrationParams);
		}

		return RegisterHierarchyLegacy(InProvider, RegistrationParams);
	}

	FHierarchyHandle FTedsHierarchyRegistrar::RegisterHierarchyLegacy(ICoreProvider* InProvider, const FHierarchyRegistrationParams& RegistrationParams)
	{
		// Generate the tags and HierarchyData column per Hierarchy type
		const UScriptStruct* HierarchyData = [InProvider, &RegistrationParams]()
		{
			FDynamicColumnDescription Desc
			{
				.TemplateType = FEditorDataHierarchyData_Template::StaticStruct(),
				.Identifier = RegistrationParams.Name
			};
			return InProvider->GenerateDynamicColumn(Desc);
		}();

		const UScriptStruct* ParentTag = [InProvider, &RegistrationParams]()
		{
			FDynamicColumnDescription Desc
			{
				.TemplateType = FEditorDataHierarchyParentTag_Template::StaticStruct(),
				.Identifier = RegistrationParams.Name
			};
			return InProvider->GenerateDynamicColumn(Desc);
		}();

		const UScriptStruct* ChildTag = [InProvider, &RegistrationParams]()
		{
			FDynamicColumnDescription Desc
			{
				.TemplateType = FEditorDataHierarchyChildTag_Template::StaticStruct(),
				.Identifier = RegistrationParams.Name
			};
			return InProvider->GenerateDynamicColumn(Desc);
		}();

		const UScriptStruct* UnresolvedParentColumn = [InProvider, &RegistrationParams]()
		{
			FDynamicColumnDescription Desc
			{
				.TemplateType = FEditorDataHierarchyUnresolvedParent_Template::StaticStruct(),
				.Identifier = RegistrationParams.Name
			};
			return InProvider->GenerateDynamicColumn(Desc);
		}();

		const UScriptStruct* ParentChangedColumn = nullptr;

		// If the old deprecated ParentChangedColumn arg was provided, keep using it for backwards compatibility
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (RegistrationParams.ParentChangedColumn)
		{
			ParentChangedColumn = RegistrationParams.ParentChangedColumn;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		else if (RegistrationParams.bEnableParentChangedColumn)
		{
			ParentChangedColumn = [InProvider, &RegistrationParams]()
			{
				FDynamicColumnDescription Desc
				{
					.TemplateType = FEditorDataHierarchyParentChangedTag_Template::StaticStruct(),
					.Identifier = RegistrationParams.Name
				};
				return InProvider->GenerateDynamicColumn(Desc);
			}();
		}

		TUniquePtr<FTedsHierarchyAccessInterface> AccessInterface = [&]()
		{
			FAdjacencyListBackedHierarchyAccess::FConstructParams Params
			{
				.ChildTag = ChildTag,
				.ParentTag = ParentTag,
				.HierarchyData = HierarchyData,
				.UnresolvedParentColumn = UnresolvedParentColumn,
				.ParentChangedColumn = ParentChangedColumn
			};
			return MakeUnique<FAdjacencyListBackedHierarchyAccess>(Params);
		}();

		FRegisteredHierarchy RegisteredHierarchy
		{
			.Name = RegistrationParams.Name,
			.ChildTag = ChildTag,
			.ParentTag = ParentTag,
			.HierarchyData = HierarchyData,
			.UnresolvedParentColumn = UnresolvedParentColumn,
			.ParentChangedColumn = ParentChangedColumn,
			.AccessInterface = MoveTemp(AccessInterface),
			.Backend = EHierarchyBackend::Legacy
		};

		int32 Index = RegisteredHierarchies.Num();
		RegisteredHierarchies.Emplace(MoveTemp(RegisteredHierarchy));

		FHierarchyHandle HierarchyHandle = CreateHandle(Index, EHierarchyBackend::Legacy);

		RegisterObservers(*InProvider, HierarchyHandle);
		RegisterProcessors(*InProvider, HierarchyHandle);

		return HierarchyHandle;
	}

	FHierarchyHandle FTedsHierarchyRegistrar::RegisterHierarchyRelations(ICoreProvider* InProvider, const FHierarchyRegistrationParams& RegistrationParams)
	{
		// Register a hierarchical relation type with exclusive Object (one parent per child)
		FRelationRegistrationParams RelationParams;
		RelationParams.Name = RegistrationParams.Name;
		RelationParams.Traits.HierarchyMode = EHierarchyMode::IntervalEncoded;
		RelationParams.Traits.Subject.bExclusive = false; // A child can only have one parent, but this is enforced by Object exclusivity
		RelationParams.Traits.Object.bExclusive = true;   // Each child has at most one parent
		// CleanUp: only the relation row is destroyed when a parent row is removed.
		// Children are NOT cascade-destroyed — they become roots, matching legacy hierarchy behaviour.
		RelationParams.Traits.Object.DestructionPolicy = FTedsRelationRoleTraits::EDestructionPolicy::CleanUp;
		// Propagate the parent-changed column opt-in: Subject = child, so bEnableSubjectChangedColumn
		// is the correct side -- the child row gets stamped when its parent relation changes.
		RelationParams.bEnableSubjectChangedColumn = RegistrationParams.bEnableParentChangedColumn;

		const RelationTypeHandle RelationType = InProvider->RegisterRelationType(RelationParams);
		if (RelationType == InvalidRelationTypeHandle)
		{
			return FHierarchyHandle();
		}

		// If requested, retrieve the generated subject-changed column to use as ParentChangedColumn.
		const UScriptStruct* ParentChangedColumn =
			RegistrationParams.bEnableParentChangedColumn
				? InProvider->GetRelationSubjectChangedColumn(RelationType)
				: nullptr;

		// Generate a dynamic unresolved parent column for deferred relation creation.
		// Same pattern as Legacy — column on Subject row, auto-cleaned on row destruction.
		const UScriptStruct* UnresolvedParentColumn = [InProvider, &RegistrationParams]()
		{
			FDynamicColumnDescription Desc
			{
				.TemplateType = FEditorDataHierarchyUnresolvedParent_Template::StaticStruct(),
				.Identifier = RegistrationParams.Name
			};
			return InProvider->GenerateDynamicColumn(Desc);
		}();

		// Create a Relations-backed access interface, forwarding the ParentChangedColumn.
		TUniquePtr<FTedsHierarchyAccessInterface> AccessInterface =
			MakeUnique<FTedsRelationBackedHierarchyAccess>(RelationType, UnresolvedParentColumn, ParentChangedColumn);

		FRegisteredHierarchy RegisteredHierarchy
		{
			.Name = RegistrationParams.Name,
			.ChildTag = nullptr,
			.ParentTag = nullptr,
			.HierarchyData = nullptr,
			.UnresolvedParentColumn = UnresolvedParentColumn,
			.ParentChangedColumn = ParentChangedColumn,
			.AccessInterface = MoveTemp(AccessInterface),
			.Backend = EHierarchyBackend::Relations,
			.HierarchyRelationType = RelationType
		};

		int32 Index = RegisteredHierarchies.Num();
		RegisteredHierarchies.Emplace(MoveTemp(RegisteredHierarchy));

		FHierarchyHandle HierarchyHandle = CreateHandle(Index, EHierarchyBackend::Relations);

		// Register processors for unresolved parent resolution.
		// No observers needed — the relation system handles lifecycle.
		RegisterProcessors(*InProvider, HierarchyHandle);

		return HierarchyHandle;
	}

	FHierarchyHandle FTedsHierarchyRegistrar::FindHierarchyByName(const FName& Name) const
	{
		for (int32 HierarchyIndex = 0; HierarchyIndex < RegisteredHierarchies.Num(); ++HierarchyIndex)
		{
			if (RegisteredHierarchies[HierarchyIndex].Name == Name)
			{
				return CreateHandle(HierarchyIndex, RegisteredHierarchies[HierarchyIndex].Backend);
			}
		}
		return FHierarchyHandle();
	}

	auto Hack_FindRowIndexInChunk = [](const ICommonQueryContext& Context, RowHandle Row) -> int32
	{
		int32 RowIndex = INDEX_NONE;
		TConstArrayView<RowHandle> RowHandes = Context.GetRowHandles();
		const int32 RowCount = RowHandes.Num();
		for (int32 CandidateRowIndex = 0; CandidateRowIndex < RowCount; ++CandidateRowIndex)
		{
			if (RowHandes[CandidateRowIndex] == Row)
			{
				RowIndex = CandidateRowIndex;
				break;
			}
		}
		return RowIndex;
	};

	static FEditorDataHierarchyData_Template* GetHierarchyDataComponentHelperInProcessor(ICommonQueryContext& Context, RowHandle Row, const UScriptStruct* ColumnType)
	{
		const int RowIndexInChunk = Hack_FindRowIndexInChunk(Context, Row);
		check(RowIndexInChunk != INDEX_NONE);
		void* HierarchyDataComponentArrayStart = Context.GetMutableColumn(ColumnType);
		uint8* HierarchyDataComponentObjectAddress = static_cast<uint8*>(HierarchyDataComponentArrayStart) + (RowIndexInChunk * ColumnType->GetStructureSize());
		return reinterpret_cast<FEditorDataHierarchyData_Template*>(HierarchyDataComponentObjectAddress);
	};

	void FTedsHierarchyRegistrar::RegisterObservers(ICoreProvider& DataStorage, const FHierarchyHandle& Handle)
	{
		using namespace UE::Editor::DataStorage::Queries;

		const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle);
		checkf(Hierarchy, TEXT("Couldn't find hierarchy that was just registered, observers for the hierarchy could not be registered successfully"));

		QueryHandle Subquery = DataStorage.RegisterQuery(
				Select()
					.ReadWrite(Hierarchy->HierarchyData)
				.Compile());

		DataStorage.RegisterQuery(
		Select(
			TEXT("Remove child from parent's child array"),
			FObserver(FObserver::EEvent::Remove, Hierarchy->ChildTag),
			[this, Handle](IQueryContext& Context, RowHandle Row)
			{
				const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle);
				
				FEditorDataHierarchyData_Template* ChildHierarchyDataComponent = GetHierarchyDataComponentHelperInProcessor(Context, Row, Hierarchy->HierarchyData);

				if (ChildHierarchyDataComponent->Parent == InvalidRowHandle)
				{
					return;
				}
				// Remove the ParentTag from the parent row if it has no more children
				bool bRemoveParentTag = false;
				bool bRemoveParentsDataColumn = false;
				Context.RunSubquery(0, ChildHierarchyDataComponent->Parent, CreateSubqueryCallbackBinding([Row, &bRemoveParentTag, &bRemoveParentsDataColumn, Hierarchy](ISubqueryContext& SubQueryContext, RowHandle ParentRow)
				{
					FEditorDataHierarchyData_Template* ParentHierarchyDataComponent = GetHierarchyDataComponentHelperInProcessor(SubQueryContext, ParentRow, Hierarchy->HierarchyData);
					
					ParentHierarchyDataComponent->Children.RemoveSwap(FTedsRowHandle(Row));
					if (ParentHierarchyDataComponent->Children.IsEmpty())
					{
						bRemoveParentTag = true;
						if (ParentHierarchyDataComponent->Parent == InvalidRowHandle)
						{
							bRemoveParentsDataColumn = true;
						}
					}
				}));
				if (bRemoveParentsDataColumn)
				{
					Context.RemoveColumns(ChildHierarchyDataComponent->Parent, {Hierarchy->HierarchyData});
				}
				if (bRemoveParentTag)
				{
					Context.RemoveColumns(ChildHierarchyDataComponent->Parent, {Hierarchy->ParentTag});
				}
				
				ChildHierarchyDataComponent->Parent = InvalidRowHandle;
			}
		)
		.ReadWrite(Hierarchy->HierarchyData)
		.DependsOn().SubQuery(Subquery)
		.Compile());

		DataStorage.RegisterQuery(
		Select(
			TEXT("Remove parent reference from children of destroyed parent"),
			FObserver(FObserver::EEvent::Remove, Hierarchy->ParentTag),
			[this, Handle](IQueryContext& Context, RowHandle ParentRow)
			{
				const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle);

				FEditorDataHierarchyData_Template* ParentHierarchyDataComponent = GetHierarchyDataComponentHelperInProcessor(Context, ParentRow, Hierarchy->HierarchyData);
					
				for (int32 ChildIndex = 0; ChildIndex < ParentHierarchyDataComponent->Children.Num(); ++ChildIndex)
				{
					RowHandle ChildRow = ParentHierarchyDataComponent->Children[ChildIndex];
					Context.RunSubquery(0, ChildRow, CreateSubqueryCallbackBinding([ParentRow, Hierarchy](ISubqueryContext& SubQueryContext, RowHandle ChildRow)
					{
						FEditorDataHierarchyData_Template* ChildHierarchyDataComponent = GetHierarchyDataComponentHelperInProcessor(SubQueryContext, ChildRow, Hierarchy->HierarchyData);

						ChildHierarchyDataComponent->Parent = InvalidRowHandle;
						if (ChildHierarchyDataComponent->Children.IsEmpty())
						{
							SubQueryContext.RemoveColumns(ChildRow, {Hierarchy->HierarchyData});
						}
					}));
					
					Context.RemoveColumns(ChildRow, {Hierarchy->ChildTag});
					
					if (Hierarchy->ParentChangedColumn)
					{
						Context.AddColumns(ChildRow, {Hierarchy->ParentChangedColumn});
					}
				}

				ParentHierarchyDataComponent->Children.Reset();
				if (ParentHierarchyDataComponent->Parent == InvalidRowHandle)
				{
					Context.RemoveColumns(ParentRow, {Hierarchy->HierarchyData});
				}
			}
		)
		.ReadWrite(Hierarchy->HierarchyData)
		.DependsOn().SubQuery(Subquery)
		.Compile());

		if (Hierarchy->ParentChangedColumn)
		{
			DataStorage.RegisterQuery(
				Select(
					TEXT("Remove Parent Changed Tag at the end of the frame"),
					FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
					[this, Handle](IQueryContext& Context, const RowHandle* Rows)
					{
						if (const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle))
						{
							Context.RemoveColumns(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()), {Hierarchy->ParentChangedColumn});
						}
					}
				)
				.Where()
					.All(Hierarchy->ParentChangedColumn)
				.Compile());
		}
	}

	void FTedsHierarchyRegistrar::RegisterProcessors(ICoreProvider& CoreProvider, const FHierarchyHandle& Handle)
	{
		using namespace UE::Editor::DataStorage::Queries;

		const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle);
		checkf(Hierarchy, TEXT("Couldn't find hierarchy that was just registered, processors for the hierarchy could not be registered successfully"));

		auto ProcessorQuery = Select(
			TEXT("Resolve hierarchy rows"),
			FProcessor(EQueryTickPhase::PrePhysics, CoreProvider.GetQueryTickGroupName(EQueryTickGroups::Default)),
			[this, Handle](IQueryContext& Context, RowHandle Row)
			{
				const FRegisteredHierarchy* Hierarchy = FindRegisteredHierarchy(Handle);

				FEditorDataHierarchyUnresolvedParent_Template* UnresolvedParentColumn;
				{
					const int RowIndexInChunk = Hack_FindRowIndexInChunk(Context, Row);
					check(RowIndexInChunk != INDEX_NONE);
					void* HierarchyDataComponentArrayStart = Context.GetMutableColumn(Hierarchy->UnresolvedParentColumn);
					uint8* HierarchyDataComponentObjectAddress = static_cast<uint8*>(HierarchyDataComponentArrayStart) + (RowIndexInChunk * Hierarchy->UnresolvedParentColumn->GetStructureSize());
					UnresolvedParentColumn = reinterpret_cast<FEditorDataHierarchyUnresolvedParent_Template*>(HierarchyDataComponentObjectAddress);
				}

				RowHandle ParentRow = Context.LookupMappedRow(UnresolvedParentColumn->MappingDomain, UnresolvedParentColumn->ParentId);
				if (Context.IsRowAvailable(ParentRow))
				{
					// Routes through HierarchyAccessInterface -- each backend's SetParentRow
					// handles the correct storage (Legacy: adjacency-list column; Relations: CreateRelation).
					Context.SetParentRow(Row, ParentRow);
					Context.RemoveColumns(Row, {Hierarchy->UnresolvedParentColumn});
				}
			});

		// AccessesHierarchy wires HierarchyAccessInterface into the query context so
		// Context.SetParentRow dispatches through the correct backend implementation.
		// SetupHierarchyDependencies now skips null HierarchyDataColumnType, so this
		// works for both Legacy and Relations backends without adding invalid requirements.
		ProcessorQuery.AccessesHierarchy(Hierarchy->Name);

		CoreProvider.RegisterQuery(
			ProcessorQuery
			.ReadWrite(Hierarchy->UnresolvedParentColumn)
			.Compile());
	}

	const FTedsHierarchyRegistrar::FRegisteredHierarchy* FTedsHierarchyRegistrar::FindRegisteredHierarchy(const FHierarchyHandle& Handle)
	{
		int32 Index = GetHandleIndex(Handle);
		if (Index != INDEX_NONE)
		{
			return &RegisteredHierarchies[Index];
		}
		return nullptr;
	}

	FAdjacencyListBackedHierarchyAccess::FAdjacencyListBackedHierarchyAccess(const FConstructParams& Params)
		: ChildTag(Params.ChildTag)
		, ParentTag(Params.ParentTag)
		, HierarchyDataColumnType(Params.HierarchyData)
		, UnresolvedParentColumnType(Params.UnresolvedParentColumn)
		, ParentChangedColumn(Params.ParentChangedColumn)
	{
		// HierarchyData may be nullptr if GenerateDynamicColumn fails; the check below guards against that.
		check(!HierarchyDataColumnType || HierarchyDataColumnType->IsChildOf(FEditorDataHierarchyData_Template::StaticStruct()));
	}

	const UScriptStruct* FAdjacencyListBackedHierarchyAccess::GetChildTagType() const
	{
		return ChildTag;
	}

	const UScriptStruct* FAdjacencyListBackedHierarchyAccess::GetParentTagType() const
	{
		return ParentTag;
	}

	const UScriptStruct* FAdjacencyListBackedHierarchyAccess::GetHierarchyDataColumnType() const
	{
		return HierarchyDataColumnType;
	}

	const UScriptStruct* FAdjacencyListBackedHierarchyAccess::GetUnresolvedParentColumnType() const
	{
		return UnresolvedParentColumnType;
	}

	const UScriptStruct* FAdjacencyListBackedHierarchyAccess::GetParentChangedColumnType() const
	{
		return ParentChangedColumn;
	}

	bool FAdjacencyListBackedHierarchyAccess::HasChildren(const ICoreProvider& Context, RowHandle Row) const
	{
		const FEditorDataHierarchyData_Template* Hierarchy = static_cast<const FEditorDataHierarchyData_Template*>(Context.GetColumnData(Row, HierarchyDataColumnType));
		return Hierarchy && !Hierarchy->Children.IsEmpty();
	}

	void FAdjacencyListBackedHierarchyAccess::WalkDepthFirst(
		const ICoreProvider& Context,
		RowHandle Row,
		TFunction<void(const ICoreProvider& Context, RowHandle Owner,RowHandle Target)> OnVisitedFn,
		ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		if (!Context.IsRowAvailable(Row))
		{
			return;
		}
		// Recursively walk depth first to visit each child
		auto WalkDepthFirstImpl = [this, TraversalOrder](const ICoreProvider& Context_, RowHandle Row, decltype(OnVisitedFn)& OnVisitedFn_, auto& WalkDepthFirstImplRef) -> void
		{
			const FEditorDataHierarchyData_Template* HierarchyData = static_cast<const FEditorDataHierarchyData_Template*>(Context_.GetColumnData(Row, HierarchyDataColumnType));
			if (HierarchyData)
			{
				for (RowHandle ChildRow : HierarchyData->Children)
				{
					if (TraversalOrder == ICoreProvider::ETraversalOrder::PreOrder)
					{
						OnVisitedFn_(Context_, Row, ChildRow);
					}
					
					WalkDepthFirstImplRef(Context_, ChildRow, OnVisitedFn_, WalkDepthFirstImplRef);

					if (TraversalOrder == ICoreProvider::ETraversalOrder::PostOrder)
					{
						OnVisitedFn_(Context_, Row, ChildRow);
					}
				}
			}
		};

		if (TraversalOrder == ICoreProvider::ETraversalOrder::PreOrder)
		{
			// Call the top level object
			OnVisitedFn(Context, InvalidRowHandle /*no parent*/, Row);
		}

		WalkDepthFirstImpl(Context, Row, OnVisitedFn, WalkDepthFirstImpl);

		if (TraversalOrder == ICoreProvider::ETraversalOrder::PostOrder)
		{
			// Call the top level object
			OnVisitedFn(Context, InvalidRowHandle /*no parent*/, Row);
		}
	}

	bool FAdjacencyListBackedHierarchyAccess::IterateChildren(const ICoreProvider& Context, RowHandle Row, 
		TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> OnVisitedFn) const
	{
		if (!Context.IsRowAvailable(Row))
		{
			return false;
		}

		const FEditorDataHierarchyData_Template* HierarchyData = static_cast<const FEditorDataHierarchyData_Template*>(Context.GetColumnData(Row, HierarchyDataColumnType));
		if (HierarchyData)
		{
			for (RowHandle ChildRow : HierarchyData->Children)
			{
				if (!OnVisitedFn(Context, ChildRow))
				{
					return false;
				}
			}
		}
		return true;
	}

	void FAdjacencyListBackedHierarchyAccess::SetParentRow(ICoreProvider* CoreProvider, RowHandle Target, RowHandle Parent) const
	{
		// The target is not assigned (ie. neither assigned to a table nor reserved)
		// It will never be a valid row so appease TEDS philosophy of not crashing with invalid rows
		if (!CoreProvider->IsRowAvailable(Target))
		{
			return;
		}

		CoreProvider->RemoveColumn(Target, UnresolvedParentColumnType);

		auto AddHierarchyDataColumn = [this](ICoreProvider* CoreProvider, RowHandle Row, RowHandle ParentRow)
		{
			CoreProvider->AddColumnData(
				Row,
				HierarchyDataColumnType,
				[ParentRow](void* Dest, const UScriptStruct& ColumnType)
				{
					ColumnType.InitializeStruct(Dest);
					FEditorDataHierarchyData_Template* HierarchyDataColumn = static_cast<FEditorDataHierarchyData_Template*>(Dest);
					check(HierarchyDataColumn->Parent == InvalidRowHandle);
					HierarchyDataColumn->Parent = ParentRow;
				},
				[](const UScriptStruct& ColumnType, void* Destination, void* Source)
				{
					// We could get away with defining this as the move operator of the base class as
					// it should be the same as whatever is in HierarchyDataColumnType
					// However, it is technically slicing.  
					new (Destination) FEditorDataHierarchyData_Template(MoveTemp(*static_cast<FEditorDataHierarchyData_Template*>(Source)));
				}
			);
		};

		RowHandle PreviousParent = InvalidRowHandle;
		FEditorDataHierarchyData_Template* HierarchyDataColumn_Target;
		if (void* TargetHierarchyDataColumnAddress = CoreProvider->GetColumnData(Target, HierarchyDataColumnType); TargetHierarchyDataColumnAddress == nullptr)
		{
			AddHierarchyDataColumn(CoreProvider, Target, Parent);
			TargetHierarchyDataColumnAddress = CoreProvider->GetColumnData(Target, HierarchyDataColumnType);
			check(TargetHierarchyDataColumnAddress);
			HierarchyDataColumn_Target = static_cast<FEditorDataHierarchyData_Template*>(TargetHierarchyDataColumnAddress);
		}
		else
		{
			HierarchyDataColumn_Target = static_cast<FEditorDataHierarchyData_Template*>(TargetHierarchyDataColumnAddress);
			PreviousParent = HierarchyDataColumn_Target->Parent;
		}
		
		if (CoreProvider->IsRowAvailable(Parent))
		{
			// No updates needed if the parent is the same as before
			if (Parent != PreviousParent)
			{
				FEditorDataHierarchyData_Template* HierarchyDataColumn_PreviousParent = nullptr;
				if (void* PreviousParentHierarchyDataColumnAddress = CoreProvider->GetColumnData(PreviousParent, HierarchyDataColumnType))
				{
					HierarchyDataColumn_PreviousParent = static_cast<FEditorDataHierarchyData_Template*>(PreviousParentHierarchyDataColumnAddress);
				}
				
				// In the case the parent changed, we have to also update the previous parent right now since any observers will act on the new parent
				if (HierarchyDataColumn_PreviousParent)
				{
					HierarchyDataColumn_PreviousParent->Children.RemoveSwap(FTedsRowHandle(Target));

					if (HierarchyDataColumn_PreviousParent->Children.IsEmpty())
					{
						if (HierarchyDataColumn_PreviousParent->Parent == InvalidRowHandle)
						{
							CoreProvider->RemoveColumn(PreviousParent, HierarchyDataColumnType);
						}
						CoreProvider->RemoveColumn(PreviousParent, ParentTag);
					}
				}

				// Update the target with the new information
				HierarchyDataColumn_Target->Parent = Parent;
				CoreProvider->AddColumn(Target, ChildTag);

				FEditorDataHierarchyData_Template* HierarchyDataColumn_Parent;
				if (void* ParentHierarchyDataColumnAddress = CoreProvider->GetColumnData(Parent, HierarchyDataColumnType); ParentHierarchyDataColumnAddress == nullptr)
				{
					AddHierarchyDataColumn(CoreProvider, Parent, InvalidRowHandle);
					ParentHierarchyDataColumnAddress = CoreProvider->GetColumnData(Parent, HierarchyDataColumnType);
					check(ParentHierarchyDataColumnAddress);
					HierarchyDataColumn_Parent = static_cast<FEditorDataHierarchyData_Template*>(ParentHierarchyDataColumnAddress);
				}
				else
				{
					HierarchyDataColumn_Parent = static_cast<FEditorDataHierarchyData_Template*>(ParentHierarchyDataColumnAddress);
				}

				// Update the new parent with the new information
				HierarchyDataColumn_Parent->Children.Add(FTedsRowHandle(Target));
				CoreProvider->AddColumn(Parent, ParentTag);
			}
		}
		else
		{
			CoreProvider->RemoveColumns(Target, {ChildTag});

			// When the child is removed from a parent (not including when the parent changes to another row), an observer handles all this
			// Commented out just to make it clear what should happen
			
			// HierarchyDataColumn_Target->Parent = InvalidRowHandle;
			// HierarchyDataColumn_Parent->Children.Remove(FTedsRowHandle(Target));
			// if (HierarchyDataColumn_Parent->Children.IsEmpty())
			// {
			// 	CoreProvider->RemoveColumns(Parent, {ParentTag});
			// }
		}

		if (ParentChangedColumn && Parent != PreviousParent)
		{
			CoreProvider->AddColumn(Target, ParentChangedColumn);
		}
	}

	void FAdjacencyListBackedHierarchyAccess::SetUnresolvedParent(ICoreProvider* CoreProvider, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		const UScriptStruct* UnresolvedParentColumn = GetUnresolvedParentColumnType();
		
		CoreProvider->AddColumnData(
				Target,
				UnresolvedParentColumn,
				[ParentId, MappingDomain](void* Dest, const UScriptStruct& ColumnType)
				{
					ColumnType.InitializeStruct(Dest);
					FEditorDataHierarchyUnresolvedParent_Template* UnresolvedParentColumn = static_cast<FEditorDataHierarchyUnresolvedParent_Template*>(Dest);
					UnresolvedParentColumn->ParentId = ParentId;
					UnresolvedParentColumn->MappingDomain = MappingDomain;
				},
				[](const UScriptStruct& ColumnType, void* Destination, void* Source)
				{
					ColumnType.CopyScriptStruct(Destination, Source);
				}
			);
	}

	RowHandle FAdjacencyListBackedHierarchyAccess::GetParentRow(const ICoreProvider* CoreProvider, RowHandle Target) const
	{
		const UScriptStruct* HierarchyDataColumn = GetHierarchyDataColumnType();
		const void* ColumnData = CoreProvider->GetColumnData(Target, HierarchyDataColumn);
		if (ColumnData != nullptr)
		{
			const FEditorDataHierarchyData_Template* Column = static_cast<const FEditorDataHierarchyData_Template*>(ColumnData);
			return Column->Parent.RowHandle;
		}
		return InvalidRowHandle;
	}

	TFunction<RowHandle(const void*, const UScriptStruct*)> FAdjacencyListBackedHierarchyAccess::CreateParentExtractionFunction() const
	{
		return [this](const void* ColumnData, const UScriptStruct* ColumnType)->RowHandle
		{
			check(ColumnType == GetHierarchyDataColumnType());
			const FEditorDataHierarchyData_Template* HierarchyDataColumn = static_cast<const FEditorDataHierarchyData_Template*>(ColumnData);
			return HierarchyDataColumn->Parent.RowHandle;
		};
	}

	// Deferred command to set the parent of a target row
	// Setting a parent may result in the Target and Parent to change archetypes due to the addition of data and tags
	// to store the relationships if they are not already established
	struct FSetParentCommand
	{
		RowHandle Parent;
		RowHandle Target;
		const FTedsHierarchyAccessInterface* HierarchyAccessInterface = nullptr;
		ICoreProvider* CoreProvider = nullptr;

		void operator()()
		{
			if ((Parent == InvalidRowHandle || CoreProvider->IsRowAssigned(Parent)) && CoreProvider->IsRowAssigned(Target))
			{
				HierarchyAccessInterface->SetParentRow(CoreProvider, Target, Parent);
			}
		}
	};

	// Deferred command to set the unresolved parent of a target row
	struct FSetUnresolvedParentCommand
	{
		RowHandle Target;
		FMapKey ParentId;
		FName MappingDomain;
		const FTedsHierarchyAccessInterface* HierarchyAccessInterface = nullptr;
		ICoreProvider* CoreProvider = nullptr;

		void operator()() const
		{
			HierarchyAccessInterface->SetUnresolvedParent(CoreProvider, Target, ParentId, MappingDomain);
		}
	};

	
	void FTedsHierarchyAccessInterface::SetParent(ICommonQueryContext& Context, RowHandle Target, RowHandle Parent) const
	{
		FSetParentCommand Command;
		Command.Parent = Parent;
		Command.Target = Target;
		Command.HierarchyAccessInterface = this;
		Command.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		Context.PushCommand(MoveTemp(Command));
	}
	

	void FTedsHierarchyAccessInterface::SetUnresolvedParent(ICommonQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		FSetUnresolvedParentCommand Command;
		Command.ParentId = ParentId;
		Command.Target = Target;
		Command.MappingDomain = MappingDomain;
		Command.HierarchyAccessInterface = this;
		Command.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		Context.PushCommand(MoveTemp(Command));	
	}

	RowHandle FTedsHierarchyAccessInterface::GetParent(const ICommonQueryContext& /*Context*/, RowHandle /*Target*/) const
	{
		return InvalidRowHandle;
	}

	RowHandle FAdjacencyListBackedHierarchyAccess::GetParent(const ICommonQueryContext& Context, RowHandle Target) const
	{
		// NOTE: It is not clear to the user that they will need to have registered the HierarchyDataColumnType as at least ReadOnly
		// for this function to return the Parent.
		// This can be improved with a function on the the context (and other contexts) to check for the access requirements of a
		// query and warn or ensure about a missing requirement - though this does not currently exist at time of writing.
		const FEditorDataHierarchyData_Template* ParentHierarchyDataComponent;
		{
			const int RowIndexInChunk = Hack_FindRowIndexInChunk(Context, Target);
			if (RowIndexInChunk == INDEX_NONE)
			{
				return InvalidRowHandle;
			}
			const void* HierarchyDataComponentArrayStart = Context.GetColumn(HierarchyDataColumnType);
			if (HierarchyDataComponentArrayStart == nullptr)
			{
				return InvalidRowHandle;
			}
			const uint8* HierarchyDataComponentObjectAddress = static_cast<const uint8*>(HierarchyDataComponentArrayStart) + (RowIndexInChunk * HierarchyDataColumnType->GetStructureSize());
			ParentHierarchyDataComponent = reinterpret_cast<const FEditorDataHierarchyData_Template*>(HierarchyDataComponentObjectAddress);
		}

		return ParentHierarchyDataComponent->Parent;
	}

	void FTedsHierarchyAccessInterface::SetParentRow(IQueryContext& Context, RowHandle Target, RowHandle Parent) const
	{
		SetParent(static_cast<ICommonQueryContext&>(Context), Target, Parent);
	}

	void FTedsHierarchyAccessInterface::SetUnresolvedParent(IQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		SetUnresolvedParent(static_cast<ICommonQueryContext&>(Context), Target, ParentId, MappingDomain);
	}

	RowHandle FTedsHierarchyAccessInterface::GetParentRow(const IQueryContext& Context, RowHandle Target) const
	{
		return GetParent(static_cast<const ICommonQueryContext&>(Context), Target);
	}

	void FTedsHierarchyAccessInterface::SetParentRow(ISubqueryContext& Context, RowHandle Target, RowHandle Parent) const
	{
		SetParent(static_cast<ICommonQueryContext&>(Context), Target, Parent);
	}

	void FTedsHierarchyAccessInterface::SetUnresolvedParent(ISubqueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		SetUnresolvedParent(static_cast<ICommonQueryContext&>(Context), Target, ParentId, MappingDomain);
	}

	RowHandle FTedsHierarchyAccessInterface::GetParentRow(const ISubqueryContext& Context, RowHandle Target) const
	{
		return GetParent(static_cast<const ICommonQueryContext&>(Context), Target);
	}

	void FTedsHierarchyAccessInterface::SetParentRow(IDirectQueryContext& Context, RowHandle Target, RowHandle Parent) const
	{
		SetParent(static_cast<ICommonQueryContext&>(Context), Target, Parent);
	}

	void FTedsHierarchyAccessInterface::SetUnresolvedParent(IDirectQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		SetUnresolvedParent(static_cast<ICommonQueryContext&>(Context), Target, ParentId, MappingDomain);
	}

	RowHandle FTedsHierarchyAccessInterface::GetParentRow(const IDirectQueryContext& Context, RowHandle Target) const
	{
		return GetParent(static_cast<const ICommonQueryContext&>(Context), Target);
	}

	//
	// FTedsRelationBackedHierarchyAccess
	//

	FTedsRelationBackedHierarchyAccess::FTedsRelationBackedHierarchyAccess(
		RelationTypeHandle InRelationType,
		const UScriptStruct* InUnresolvedParentColumnType,
		const UScriptStruct* InParentChangedColumn)
		: HierarchyRelationType(InRelationType)
		, UnresolvedParentColumnType(InUnresolvedParentColumnType)
		, ParentChangedColumnType(InParentChangedColumn)
	{
	}

	void FTedsRelationBackedHierarchyAccess::SetParentRow(ICoreProvider* CoreProvider, RowHandle Target, RowHandle Parent) const
	{
		check(CoreProvider != nullptr);

		// Destroy existing parent relation first (C1: explicit destroy-before-create for re-parenting)
		RowHandle CurrentParent = CoreProvider->GetRelationObject(HierarchyRelationType, Target);
		if (CurrentParent != InvalidRowHandle)
		{
			CoreProvider->DestroyRelation(HierarchyRelationType, Target, CurrentParent);
		}

		// Create new parent relation (or orphan if Parent == InvalidRowHandle)
		if (Parent != InvalidRowHandle)
		{
			CoreProvider->CreateRelation(HierarchyRelationType, Target, Parent);
		}
	}

	void FTedsRelationBackedHierarchyAccess::SetUnresolvedParent(
		ICoreProvider* CoreProvider, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
	{
		check(CoreProvider != nullptr);

		// Try to resolve immediately
		RowHandle ParentRow = CoreProvider->LookupMappedRow(MappingDomain, ParentId);
		if (CoreProvider->IsRowAvailable(ParentRow))
		{
			SetParentRow(CoreProvider, Target, ParentRow);
			return;
		}

		// Store unresolved parent data as a column on the Target (child) row.
		// The column auto-cleans when the Target row is destroyed.
		// ResolveHierarchy picks these up each frame and attempts resolution.
		const UScriptStruct* UnresolvedColumn = GetUnresolvedParentColumnType();
		if (UnresolvedColumn)
		{
			CoreProvider->AddColumn(Target, UnresolvedColumn);
			if (void* ColumnData = CoreProvider->GetColumnData(Target, UnresolvedColumn))
			{
				FEditorDataHierarchyUnresolvedParent_Template* Unresolved =
					static_cast<FEditorDataHierarchyUnresolvedParent_Template*>(ColumnData);
				Unresolved->ParentId = ParentId;
				Unresolved->MappingDomain = MappingDomain;
			}
		}
	}

	RowHandle FTedsRelationBackedHierarchyAccess::GetParentRow(const ICoreProvider* CoreProvider, RowHandle Target) const
	{
		check(CoreProvider != nullptr);
		return CoreProvider->GetRelationObject(HierarchyRelationType, Target);
	}

	bool FTedsRelationBackedHierarchyAccess::HasChildren(const ICoreProvider& Context, RowHandle Row) const
	{
		return Context.HasRelationSubject(HierarchyRelationType, Row);
	}

	void FTedsRelationBackedHierarchyAccess::WalkDepthFirst(
		const ICoreProvider& Context,
		RowHandle Row,
		TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)> OnVisitedFn,
		ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		if (Row == InvalidRowHandle || !OnVisitedFn)
		{
			return;
		}

		// Match Legacy behavior: visit the root node as well as all descendants.
		// Legacy's WalkDepthFirst visits root first (PreOrder) or last (PostOrder).
		if (TraversalOrder == ICoreProvider::ETraversalOrder::PreOrder)
		{
			OnVisitedFn(Context, InvalidRowHandle /*no parent*/, Row);
		}

		Context.TraverseDescendants(HierarchyRelationType, Row,
			[&Context, &OnVisitedFn](RowHandle Current, RowHandle Parent, int32 /*Depth*/)
			{
				OnVisitedFn(Context, Parent, Current);
			},
			TraversalOrder);

		if (TraversalOrder == ICoreProvider::ETraversalOrder::PostOrder)
		{
			OnVisitedFn(Context, InvalidRowHandle /*no parent*/, Row);
		}
	}

	bool FTedsRelationBackedHierarchyAccess::IterateChildren(
		const ICoreProvider& Context,
		RowHandle Row,
		TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> OnVisitedFn) const
	{
		if (!Context.IsRowAvailable(Row))
		{
			return false;
		}

		TArray<RowHandle> Children;
		Context.GetRelationSubjects(HierarchyRelationType, Row, Children);
		for (RowHandle Child : Children)
		{
			if (!OnVisitedFn(Context, Child))
			{
				return false;
			}
		}
		return true;
	}

	TFunction<RowHandle(const void*, const UScriptStruct*)> FTedsRelationBackedHierarchyAccess::CreateParentExtractionFunction() const
	{
		// Relations backend doesn't store parent data in a column on the participant row.
		// Return a no-op function that always returns InvalidRowHandle.
		return [](const void*, const UScriptStruct*) -> RowHandle
		{
			return InvalidRowHandle;
		};
	}
}
