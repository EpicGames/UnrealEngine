// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage
{
	enum class EHierarchyBackend : uint8
	{
		Legacy = 0,
		Relations = 1,
	};

	class FTedsHierarchyAccessInterface;
	class FTedsHierarchyRegistrar final
	{
	public:

		~FTedsHierarchyRegistrar();
		/**
		 * Registers a hierarchy type with TEDS
		 * This sets up the internal columns and processors to kee the bidirectional relationship in sync
		 */
		FHierarchyHandle RegisterHierarchy(ICoreProvider* InProvider, const FHierarchyRegistrationParams& Params);
		/**
		 * Finds a registered hierarchy by name
		 */
		FHierarchyHandle FindHierarchyByName(const FName& Name) const;
		/**
		 * Gets an interface to add and remove rows from a hierarchy and walk the hierarchy
		 */
		const FTedsHierarchyAccessInterface* GetAccessInterface(FHierarchyHandle) const;

		/**
		 * Iterate over all registered hierarchyes
		 */
		void ListHierarchyNames(TFunctionRef<void(const FName& HierarchyName)> Callback) const;

		/**
		 * Try to resolve all child rows with unresolved parent columns
		 */
		void ResolveHierarchy(ICoreProvider& CoreProvider, FHierarchyHandle HierarchyHandle);

	private:
		FHierarchyHandle RegisterHierarchyLegacy(ICoreProvider* InProvider, const FHierarchyRegistrationParams& Params);
		FHierarchyHandle RegisterHierarchyRelations(ICoreProvider* InProvider, const FHierarchyRegistrationParams& Params);
		void RegisterObservers(ICoreProvider& CoreProvider, const FHierarchyHandle& Handle);
		void RegisterProcessors(ICoreProvider& CoreProvider, const FHierarchyHandle& Handle);

		struct FRegisteredHierarchy
		{
			FName Name;
			const UScriptStruct* ChildTag;
			const UScriptStruct* ParentTag;
			const UScriptStruct* HierarchyData;
			const UScriptStruct* UnresolvedParentColumn;
			const UScriptStruct* ParentChangedColumn;
			TUniquePtr<FTedsHierarchyAccessInterface> AccessInterface;
			QueryHandle UnresolvedChildRowsQueryHandle = InvalidQueryHandle;
			EHierarchyBackend Backend = EHierarchyBackend::Legacy;
			RelationTypeHandle HierarchyRelationType = InvalidRelationTypeHandle;
		};

		const FRegisteredHierarchy* FindRegisteredHierarchy(const FHierarchyHandle& Handle);

	private:
		TArray<FRegisteredHierarchy> RegisteredHierarchies;
	};

	/**
	 * Abstract interface for accessing hierarchy operations.
	 * Concrete backends (adjacency-list and relations) are defined in the .cpp.
	 *
	 * The non-virtual context-dispatch overloads are generic and live here because
	 * they delegate to the virtual ICoreProvider* overloads via context unwrapping.
	 */
	class FTedsHierarchyAccessInterface
	{
	public:
		virtual ~FTedsHierarchyAccessInterface() = default;

		virtual const UScriptStruct* GetChildTagType() const = 0;
		virtual const UScriptStruct* GetParentTagType() const = 0;
		virtual const UScriptStruct* GetHierarchyDataColumnType() const = 0;
		virtual const UScriptStruct* GetUnresolvedParentColumnType() const = 0;
		virtual const UScriptStruct* GetParentChangedColumnType() const = 0;

		virtual void SetParentRow(ICoreProvider* CoreProvider, RowHandle Target, RowHandle Parent) const = 0;
		virtual void SetUnresolvedParent(ICoreProvider* CoreProvider, RowHandle Target, FMapKey ParentId, FName MappingDomain) const = 0;
		virtual RowHandle GetParentRow(const ICoreProvider* CoreProvider, RowHandle Target) const = 0;

		virtual bool HasChildren(const ICoreProvider& Context, RowHandle Row) const = 0;
		virtual void WalkDepthFirst(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunction<void(const ICoreProvider& Context, RowHandle Owner, RowHandle Target)> OnVisitedFn,
			ICoreProvider::ETraversalOrder TraversalOrder = ICoreProvider::ETraversalOrder::PreOrder) const = 0;

		virtual bool IterateChildren(
			const ICoreProvider& Context,
			RowHandle Row,
			TFunctionRef<bool(const ICoreProvider& Context, RowHandle Child)> OnVisitedFn) const
		{
			return false;
		}

		virtual TFunction<RowHandle(const void*, const UScriptStruct*)> CreateParentExtractionFunction() const = 0;

		// Non-virtual context-dispatch overloads — generic for all backends.
		void SetParentRow(IQueryContext& Context, RowHandle Target, RowHandle Parent) const;
		void SetUnresolvedParent(IQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const;
		RowHandle GetParentRow(const IQueryContext& Context, RowHandle Target) const;
		void SetParentRow(ISubqueryContext& Context, RowHandle Target, RowHandle Parent) const;
		void SetUnresolvedParent(ISubqueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const;
		RowHandle GetParentRow(const ISubqueryContext& Context, RowHandle Target) const;
		void SetParentRow(IDirectQueryContext& Context, RowHandle Target, RowHandle Parent) const;
		void SetUnresolvedParent(IDirectQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const;
		RowHandle GetParentRow(const IDirectQueryContext& Context, RowHandle Target) const;

	protected:
		// Generic context-dispatch helpers shared by all backends.
		void SetParent(ICommonQueryContext& Context, RowHandle Target, RowHandle Parent) const;
		void SetUnresolvedParent(ICommonQueryContext& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const;

		// Context-aware parent lookup. Default returns InvalidRowHandle; override in column-backed backends.
		virtual RowHandle GetParent(const ICommonQueryContext& Context, RowHandle Target) const;
	};

} // namespace UE::Editor::DataStorage
