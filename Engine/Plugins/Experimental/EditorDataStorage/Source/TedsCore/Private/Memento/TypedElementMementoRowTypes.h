// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "TypedElementMementoRowTypes.generated.h"

/**
 * @file MementoRowTypes.h
 * Column/Tags used internally for the memento system
 */

/**
 * MementoTag denotes that the row is a memento
 */
USTRUCT()
struct FTypedElementMementoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
