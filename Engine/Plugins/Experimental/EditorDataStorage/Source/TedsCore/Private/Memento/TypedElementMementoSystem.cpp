// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementMementoSystem.h"

#include "DataStorage/Debug/Log.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GlobalLock.h"
#include "Memento/TypedElementMementoTranslators.h"
#include "TypedElementMementoRowTypes.h"
#include "TypedElementDatabase.h"

namespace UE::Editor::DataStorage
{
	FMementoSystem::FMementoSystem(ICoreProvider& InDataStorage, FTableManager& TableManager)
		: DataStorage(InDataStorage)
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		// Register tables that will be used by reinstancing
		MementoRowBaseTable = TableManager.Register({ FTypedElementMementoTag::StaticStruct() }, TEXT("MementoRowBaseTable"));

		// Discover all MementoTranslators
		{
			constexpr bool bIncludeDerived = true;
			constexpr EObjectFlags ExcludeFlags = EObjectFlags::RF_NoFlags;
			ForEachObjectOfClass(UTedsMementoTranslatorBase::StaticClass(),
				[this](UObject* Object)
				{
					const UTedsMementoTranslatorBase* TranslatorCandidate = Cast<UTedsMementoTranslatorBase>(Object);
					// Exclude abstract classes
					if (TranslatorCandidate->GetClass()->GetClassFlags() & EClassFlags::CLASS_Abstract)
					{
						return;
					}
					MementoTranslators.Add(TranslatorCandidate);
				},
				bIncludeDerived, ExcludeFlags);
		}
	}

	RowHandle FMementoSystem::CreateMemento(RowHandle SourceRow)
	{
		FScopedSharedLock Lock(EGlobalLockScope::Public);

		RowHandle MementoRow = DataStorage.AddRow(MementoRowBaseTable);
		CreateMementoInternal(MementoRow, SourceRow);
		return MementoRow;
	}

	void FMementoSystem::CreateMemento(RowHandle ReservedMementoRow, RowHandle SourceRow)
	{
		FScopedSharedLock Lock(EGlobalLockScope::Public);
	
		DataStorage.AddRow(ReservedMementoRow, MementoRowBaseTable);
		CreateMementoInternal(ReservedMementoRow, SourceRow);
	}

	void FMementoSystem::CreateMementoInternal(RowHandle MementoRow, RowHandle SourceRow)
	{	
		for (const UTedsMementoTranslatorBase* Translator : MementoTranslators)
		{
			if (void* SourceColumn = DataStorage.GetColumnData(SourceRow, Translator->GetColumnType()))
			{
				const UScriptStruct* MementoType = Translator->GetMementoType();
				DataStorage.AddColumnData(MementoRow, MementoType,
					[Translator, SourceColumn](void* MementoColumn, const UScriptStruct& ColumnType)
					{
						ColumnType.InitializeStruct(MementoColumn);
						FScopedSharedLock Lock(EGlobalLockScope::Public);
						Translator->TranslateColumnToMemento(SourceColumn, MementoColumn);
					},
					[](const UScriptStruct& ColumnType, void* Destination, void* Source)
					{
						ColumnType.CopyScriptStruct(Destination, Source);
					});

				UE_LOGF(LogEditorDataStorage, VeryVerbose,
					"Column->Memento: %llu -> %llu", SourceRow, MementoRow);
			}
		}
	}

	void FMementoSystem::RestoreMemento(RowHandle MementoRow, RowHandle TargetRow)
	{
		FScopedSharedLock Lock(EGlobalLockScope::Public);

		for (const UTedsMementoTranslatorBase* Translator : MementoTranslators)
		{
			if (void* MementoColumn = DataStorage.GetColumnData(MementoRow, Translator->GetMementoType()))
			{
				const UScriptStruct* TargetType = Translator->GetColumnType();
				DataStorage.AddColumnData(TargetRow, TargetType,
					[Translator, MementoColumn](void* TargetColumn, const UScriptStruct& ColumnType)
					{
						ColumnType.InitializeStruct(TargetColumn);
						FScopedSharedLock Lock(EGlobalLockScope::Public);
						Translator->TranslateMementoToColumn(MementoColumn, TargetColumn);
					},
					[](const UScriptStruct& ColumnType, void* Destination, void* Source)
					{
						ColumnType.CopyScriptStruct(Destination, Source);
					});
				
				if (const UScriptStruct* Tag = Translator->GetColumnToAddOnRestore())
				{
					DataStorage.AddColumn(TargetRow, Tag);
				}

				UE_LOGF(LogEditorDataStorage, VeryVerbose, 
					"Memento->Column: %llu -> %llu", MementoRow, TargetRow);
			}
		}
	}

	void FMementoSystem::DestroyMemento(RowHandle MementoRow)
	{
		// No need to lock this as no internal data is used.

		checkf(DataStorage.IsRowAvailable(MementoRow) && DataStorage.HasColumns<FTypedElementMementoTag>(MementoRow),
			TEXT("Deleting memento row that's not marked as such."));
		DataStorage.RemoveRow(MementoRow);
	}
} // namespace UE::Editor::DataStorage
