// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/Array.h"

class UTedsMementoTranslatorBase;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class FTableManager;

	class FMementoSystem
	{
	public:
		FMementoSystem(ICoreProvider& InDataStorage, FTableManager& TableManager);

		RowHandle CreateMemento(RowHandle SourceRow);
		void CreateMemento(RowHandle ReservedMementoRow, RowHandle SourceRow);
		void RestoreMemento(RowHandle MementoRow, RowHandle TargetRow);
		void DestroyMemento(RowHandle MementoRow);

	private:
		void CreateMementoInternal(RowHandle MementoRow, RowHandle SourceRow);

		TArray<const UTedsMementoTranslatorBase*> MementoTranslators;
		TableHandle MementoRowBaseTable;
		ICoreProvider& DataStorage;
	};
} // namespace UE::Editor::DataStorage
