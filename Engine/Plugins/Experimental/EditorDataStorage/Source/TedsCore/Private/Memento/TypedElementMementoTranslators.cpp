// Copyright Epic Games, Inc. All Rights Reserved.

#include "Memento/TypedElementMementoTranslators.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "StructUtils/PropertyBag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementMementoTranslators)

const UScriptStruct* UTedsMementoTranslatorBase::GetColumnToAddOnRestore() const
{
	return ColumnToAddOnRestore;
}

const UScriptStruct* UTedsDefaultMementoTranslator::GetMementoType() const
{
	return MementoType;
}

void UTedsDefaultMementoTranslator::PostInitProperties()
{
	Super::PostInitProperties();
	
	const UScriptStruct* SourceColumnType = GetColumnType();
	if (SourceColumnType == nullptr)
	{
		return;
	}

	// Create a new runtime generated struct as the memento from the ColumnType based on the exposed UPROPERTIES
	// and generate a mapping between the properties of the ColumnType to the Memento
	// This mapping will be used to populate the memento and columns during translation
	
	TArray<FPropertyBagPropertyDesc> PropertyDescs;
	for (FProperty* Property = SourceColumnType->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		PropertyDescs.Add(FPropertyBagPropertyDesc(Property->GetFName(), Property));
	}
	
	const FString GeneratedMementoStructName = FString::Printf(TEXT("%s_Memento"), *SourceColumnType->GetName());
	const UPropertyBag* PropertyBag = UPropertyBag::GetOrCreateFromDescs(PropertyDescs, *GeneratedMementoStructName);
	MementoType = PropertyBag;

	// Need to change the type to a FColumn to appease TEDS/Mass
	const_cast<UPropertyBag*>(PropertyBag)->SetSuperStruct(UE::Editor::DataStorage::FColumn::StaticStruct());

	// Create the property mapping
	for (FProperty* SourceProperty = SourceColumnType->PropertyLink; SourceProperty; SourceProperty = SourceProperty->PropertyLinkNext)
	{
		const FProperty* DestinationProperty = PropertyBag->FindPropertyByName(SourceProperty->GetFName());
		if (DestinationProperty && DestinationProperty->SameType(SourceProperty))
		{
			MementoizedColumnProperties.Add(SourceProperty);
			MementoProperties.Add(DestinationProperty);
		}
	}
}

void UTedsDefaultMementoTranslator::TranslateColumnToMemento(const void* TypeErasedColumn, void* TypeErasedMemento) const
{
	const std::byte* BaseAddressColumn = static_cast<const std::byte*>(TypeErasedColumn);
	std::byte* BaseAddressMemento = static_cast<std::byte*>(TypeErasedMemento);

	check(MementoizedColumnProperties.Num() == MementoProperties.Num());

	for (int32 PropertyIndex = 0, PropertyIndexEnd = MementoizedColumnProperties.Num(); PropertyIndex < PropertyIndexEnd; ++PropertyIndex)
	{
		const FProperty* SourceProperty = MementoizedColumnProperties[PropertyIndex];
		const FProperty* DestinationProperty = MementoProperties[PropertyIndex];
		void* DestinationValueAddress = BaseAddressMemento + DestinationProperty->GetOffset_ForInternal();
		const void* SourceValueAddress = BaseAddressColumn + SourceProperty->GetOffset_ForInternal();
		SourceProperty->CopyCompleteValue(
			DestinationValueAddress,
			SourceValueAddress);
	}	
}

void UTedsDefaultMementoTranslator::TranslateMementoToColumn(const void* TypeErasedMemento,
	void* TypeErasedColumn) const
{
	const std::byte* BaseAddressMemento = static_cast<const std::byte*>(TypeErasedMemento);
	std::byte* BaseAddressColumn = static_cast<std::byte*>(TypeErasedColumn);

	check(MementoizedColumnProperties.Num() == MementoProperties.Num());

	for (int32 PropertyIndex = 0, PropertyIndexEnd = MementoizedColumnProperties.Num(); PropertyIndex < PropertyIndexEnd; ++PropertyIndex)
	{
		const FProperty* SourceProperty = MementoProperties[PropertyIndex];
		const FProperty* DestinationProperty = MementoizedColumnProperties[PropertyIndex];
		void* DestinationValueAddress = BaseAddressColumn + DestinationProperty->GetOffset_ForInternal();
		const void* SourceValueAddress = BaseAddressMemento + SourceProperty->GetOffset_ForInternal();
		SourceProperty->CopyCompleteValue(
			DestinationValueAddress,
			SourceValueAddress);
	}	
}
