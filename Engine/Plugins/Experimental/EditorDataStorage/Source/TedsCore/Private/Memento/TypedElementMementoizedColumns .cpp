// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementMementoizedColumns.h"

#include "Elements/Columns/TypedElementVisibilityColumns.h"


UTedsVisibilityColumnMementoTranslator::UTedsVisibilityColumnMementoTranslator()
{
	// To sync any visibility changes via re-instance back to the world we add the FVisibilityChangedTag to the row
	ColumnToAddOnRestore = FVisibilityChangedTag::StaticStruct();
}

const UScriptStruct* UTedsVisibilityColumnMementoTranslator::GetColumnType() const
{
	return FVisibleInEditorColumn::StaticStruct();
}
