// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Memento/TypedElementMementoTranslators.h"
#include "TypedElementMementoizedColumns.generated.h"

/**
 * Enable SelectionColumn to be mementoized
 */
UCLASS()
class UTedsSelectionColumnMementoTranslator final : public UTedsDefaultMementoTranslator
{
	GENERATED_BODY()
public:
	virtual const UScriptStruct* GetColumnType() const override { return FTypedElementSelectionColumn::StaticStruct(); }
};

/**
 * Enable Visibility to be mementoized
 */
UCLASS()
class UTedsVisibilityColumnMementoTranslator final : public UTedsDefaultMementoTranslator
{
	GENERATED_BODY()
public:

	UTedsVisibilityColumnMementoTranslator();
	virtual const UScriptStruct* GetColumnType() const override;
};

