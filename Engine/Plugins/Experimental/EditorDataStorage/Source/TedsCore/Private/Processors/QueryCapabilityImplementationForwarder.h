// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContract.h"
#include "Elements/Framework/TypedElementQueryFunctionArgumentTypes.h"

namespace UE::Editor::DataStorage::Queries::Private
{
	// Because the original context has a different inheritance hierarchy than the new query context implementations can't
	// be directly used in the various IQueryContext implementation. This wrapper can be used to bridge the two until
	// the new query context can fully replace the original query context.

	template<typename Environment, typename Interface, typename... Implementations> struct TImplementationForwarder
	{
		static_assert(sizeof...(Implementations) < 0,
			"Specialization for query context forwards missing. The most likely reason is the addition of a new query "
			"context capability that wasn't specialized.");
	};

	template<typename Environment, typename Interface>
	struct TImplementationForwarder<Environment, Interface> : Interface
	{
		TImplementationForwarder() = default;

		template<typename... TArgs>
		explicit TImplementationForwarder(TArgs&&... Args)
			: ContextEnvironment(Forward<TArgs>(Args)...)
		{
		}

		Environment ContextEnvironment;
	};

	template<typename Implementation, typename Capability>
	concept ImplementsCapability = std::is_base_of_v<ImplementsContextCapability<Capability>, Implementation>;

#define ArgTypeName(Type, Name) Type Name
#define ArgName(Type, Name) Name
#define ForwardArg(Type, Name) Forward< Type >( Name )
	
#define CallFunction(Capability, ReturnValue, FunctionName, ...) \
	if constexpr (ImplementsCapability<Implementation, Capability>) \
	{ \
		static_assert( requires() \
			{ \
				{ Implementation:: FunctionName (this->ContextEnvironment __VA_OPT__(,) __VA_ARGS__ ) } -> UE::CSameAs< ReturnValue >; \
			}, "Function '" #FunctionName "' in capability '" #Capability "' is missing from the forwarder." ); \
		return Implementation:: FunctionName ( this->ContextEnvironment __VA_OPT__(,) __VA_ARGS__ ); \
	} \
	else \
	{ \
		static_assert(std::is_same_v<void, Capability>, \
			"Function '" #FunctionName "' in capability '" #Capability "' is missing from the implementation."); \
		return UE::MakeDefault< ReturnValue >(); \
	}

#define CapabilityStart(Capability, Flags) \
	template<typename Environment, typename Interface, typename Implementation, typename... Implementations> \
		requires ImplementsCapability<Implementation, Queries:: Capability > \
	struct TImplementationForwarder< Environment, Interface, Implementation, Implementations...> : \
		Implementation, \
		TImplementationForwarder<Environment, Interface, Implementations...> \
	{ \
		using Super = TImplementationForwarder<Environment, Interface, Implementations...>; \
		TImplementationForwarder() = default; \
		template<typename... TArgs> \
		explicit TImplementationForwarder(TArgs&&... Args) \
			: Super(Forward<TArgs>(Args)...) \
		{}

#define Function0(Capability, Return, Function) virtual Return Function() \
	{ CallFunction(Capability, Return, Function ) }
#define Function1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1 ) }
#define Function2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2 ) }
#define Function3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3 ) }
#define Function4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4 ) }

#define ConstFunction0(Capability, Return, Function) virtual Return Function() const \
	{ CallFunction(Capability, Return, Function ) }
#define ConstFunction1(Capability, Return, Function, Arg1) virtual Return Function(ArgTypeName Arg1 ) const override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1 ) }
#define ConstFunction2(Capability, Return, Function, Arg1, Arg2) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 ) const override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2 ) }
#define ConstFunction3(Capability, Return, Function, Arg1, Arg2, Arg3) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 ) const override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3 ) }
#define ConstFunction4(Capability, Return, Function, Arg1, Arg2, Arg3, Arg4) virtual Return Function(ArgTypeName Arg1 , ArgTypeName Arg2 , ArgTypeName Arg3 , ArgTypeName Arg4 ) const override \
	{ CallFunction(Capability, Return, Function, ForwardArg Arg1, ForwardArg Arg2, ForwardArg Arg3, ForwardArg Arg4 ) }

#define CapabilityEnd(Capability) \
	}; \

#define DeprecatedFunction(Version, Msg) UE_DEPRECATED(Version, Msg)

#include "Elements/Framework/TypedElementQueryCapabilities.inl"

#undef ArgTypeName
#undef ArgName
#undef ForwardArg
#undef CallFunction
#undef DeprecatedFunction
#undef Function0
#undef Function1
#undef Function2
#undef Function3
#undef Function4
#undef ConstFunction0
#undef ConstFunction1
#undef ConstFunction2
#undef ConstFunction3
#undef ConstFunction4
#undef CapabilityStart
#undef CapabilityEnd
} // namespace UE::Editor::DataStorage::Queries::Private