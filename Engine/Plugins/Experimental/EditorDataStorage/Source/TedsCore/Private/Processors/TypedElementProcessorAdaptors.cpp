// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/TypedElementProcessorAdaptors.h"

#include "DataStorage/Queries/Types.h"
#include "MassEntityTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "Misc/EnumClassFlags.h"
#include "Queries/TypedElementExtendedQueryStore.h"
#include "Queries/ProcessorContextEnvironment.h"
#include "Processors/QueryCapabilityImplementationForwarder.h"
#include "StructUtils/StructTypeBitSet.h"
#include "Relations/EditorDataStorageRelationsImplementation.h"
#include "TypedElementDatabase.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementUtils.h"
#include "Queries/QueryContextCapabilities/EditorDataStorageDynamicColumnInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementProcessorAdaptors)

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		/** Check whether a query has any relation or hierarchy conditions that need per-entity filtering. */
		static bool HasRelationConditions(const FQueryDescription& Description)
		{
			return !Description.RelationConditions.IsEmpty()
				|| !Description.DescendantOfConditions.IsEmpty()
				|| !Description.AncestorOfConditions.IsEmpty();
		}

		/** Evaluate all relation/hierarchy conditions for a single row. Returns true if the row passes all conditions. */
		static bool PassesRelationConditions(RowHandle Row, const FQueryDescription& Description,
			FMassEntityManager& EntityManager, const FTedsRelationAdapter& RelationAdapter,
			const FTedsHierarchicalRelationManager* HierarchyManager)
		{
			// Check relation conditions (IsSubjectOf / IsObjectOf)
			for (const FQueryDescription::FRelationCondition& Condition : Description.RelationConditions)
			{
				if (Condition.RelationType == InvalidRelationTypeHandle)
				{
					continue;
				}

				if (Condition.Role == FQueryDescription::ERelationRole::Subject)
				{
					if (Condition.TargetRow != InvalidRowHandle)
					{
						// IsSubjectOf(Type, TargetObject) — check if this row is subject with that specific object
						if (!RelationAdapter.HasMassRelationData(Condition.RelationType) ||
								!EntityManager.GetRelationManager().IsSubjectOfRelation(
									RelationAdapter.GetMassHandle(Condition.RelationType),
									FMassEntityHandle::FromNumber(Row),
									FMassEntityHandle::FromNumber(Condition.TargetRow)))
						{
							return false;
						}
					}
					else
					{
						// IsSubjectOf(Type) — check if this row is a subject of any relation of this type
						if (!RelationAdapter.HasMassRelationData(Condition.RelationType) ||
							!EntityManager.GetRelationManager().HasRelationObjects(
								RelationAdapter.GetMassHandle(Condition.RelationType),
								FMassEntityHandle::FromNumber(Row)))
						{
							return false;
						}
					}
				}
				else // ERelationRole::Object
				{
					if (Condition.TargetRow != InvalidRowHandle)
					{
						// IsObjectOf(Type, TargetSubject) — check if this row is object with that specific subject
						if (!RelationAdapter.HasMassRelationData(Condition.RelationType) ||
								!EntityManager.GetRelationManager().IsSubjectOfRelation(
									RelationAdapter.GetMassHandle(Condition.RelationType),
									FMassEntityHandle::FromNumber(Condition.TargetRow),
									FMassEntityHandle::FromNumber(Row)))
						{
							return false;
						}
					}
					else
					{
						// IsObjectOf(Type) — check if this row is an object of any relation of this type
						if (!RelationAdapter.HasMassRelationData(Condition.RelationType) ||
							!EntityManager.GetRelationManager().HasRelationSubjects(
								RelationAdapter.GetMassHandle(Condition.RelationType),
								FMassEntityHandle::FromNumber(Row)))
						{
							return false;
						}
					}
				}
			}

			// Check descendant-of conditions
			if (!Description.DescendantOfConditions.IsEmpty())
			{
				if (!HierarchyManager)
				{
					return false;
				}
				for (const FQueryDescription::FHierarchyCondition& Condition : Description.DescendantOfConditions)
				{
					if (Condition.RelationType == InvalidRelationTypeHandle || Condition.ReferenceRow == InvalidRowHandle)
					{
						continue;
					}
					if (Condition.bIncludeSelf && Row == Condition.ReferenceRow)
					{
						continue; // Passes — self-inclusion
					}
					if (!HierarchyManager->IsDescendantOf(EntityManager, RelationAdapter, Condition.RelationType, Row, Condition.ReferenceRow))
					{
						return false;
					}
				}
			}

			// Check ancestor-of conditions
			if (!Description.AncestorOfConditions.IsEmpty())
			{
				if (!HierarchyManager)
				{
					return false;
				}
				for (const FQueryDescription::FHierarchyCondition& Condition : Description.AncestorOfConditions)
				{
					if (Condition.RelationType == InvalidRelationTypeHandle || Condition.ReferenceRow == InvalidRowHandle)
					{
						continue;
					}
					if (Condition.bIncludeSelf && Row == Condition.ReferenceRow)
					{
						continue; // Passes — self-inclusion
					}
					// IsAncestorOf(Type, Descendant) means: is Descendant a descendant of Row?
					if (!HierarchyManager->IsDescendantOf(EntityManager, RelationAdapter, Condition.RelationType, Condition.ReferenceRow, Row))
					{
						return false;
					}
				}
			}



			return true;
		}

		/**
		 * Evaluate relation conditions for all entities in the current chunk and invoke the callback
		 * once per contiguous sub-range of passing entities. This performs entity-level relation filtering
		 * entirely within the TEDS adaptor layer, without requiring Mass-level EntityCondition support.
		 */
		static void ForEachRelationFilteredSubrange(
			FMassExecutionContext& Context,
			const FQueryDescription& Description,
			FMassEntityManager& EntityManager,
			const FTedsRelationAdapter& RelationAdapter,
			const FTedsHierarchicalRelationManager* HierarchyManager,
			TFunctionRef<void(int32 SubRangeStart, int32 SubRangeLength)> Callback)
		{
			TConstArrayView<FMassEntityHandle> Entities = Context.GetEntities();
			const int32 Num = Entities.Num();

			FMemMark Mark(FMemStack::Get());
			TArray<bool, TMemStackAllocator<>> Passes;
			Passes.SetNumUninitialized(Num);
			for (int32 i = 0; i < Num; ++i)
			{
				Passes[i] = PassesRelationConditions(
					Entities[i].AsNumber(), Description, EntityManager, RelationAdapter, HierarchyManager);
			}

			int32 RangeStart = INDEX_NONE;
			for (int32 i = 0; i <= Num; ++i)
			{
				const bool bPasses = (i < Num) && Passes[i];
				if (bPasses && RangeStart == INDEX_NONE)
				{
					RangeStart = i;
				}
				else if (!bPasses && RangeStart != INDEX_NONE)
				{
					Callback(RangeStart, i - RangeStart);
					RangeStart = INDEX_NONE;
				}
			}
		}
	} // namespace Private

	namespace Processors::Private
	{
		/**
		 * Deferred command to change a row's relation object.
		 * Removes the existing relation (if any) then optionally creates a new one.
		 * Executes after the processor chunk loop, under ICoreProvider's normal write path.
		 */
		struct FSetRelationObjectCommand
		{
			RelationTypeHandle Type = InvalidRelationTypeHandle;
			RowHandle Target        = InvalidRowHandle;
			RowHandle NewObject     = InvalidRowHandle;   // InvalidRowHandle = remove only
			ICoreProvider* CoreProvider = nullptr;

			void operator()()
			{
				if (!CoreProvider || !CoreProvider->IsRowAssigned(Target))
				{
					return;
				}

				// Remove existing relation, if any
				const RowHandle OldObject = CoreProvider->GetRelationObject(Type, Target);
				if (OldObject != InvalidRowHandle)
				{
					CoreProvider->DestroyRelation(Type, Target, OldObject);
				}

				// Create new relation, if requested
				if (NewObject != InvalidRowHandle && CoreProvider->IsRowAssigned(NewObject))
				{
					CoreProvider->CreateRelation(Type, Target, NewObject);
				}
			}
		};

		struct FMassContextCommon
		{
			FMassExecutionContext& Context;
			FEnvironment& Environment;
			int32 SubRangeStart = 0;
			int32 SubRangeLength = -1;

			FMassContextCommon(FMassExecutionContext& InContext, FEnvironment& InEnvironment)
				: Context(InContext)
				, Environment(InEnvironment)
			{}

			void SetSubRange(int32 InStart, int32 InLength)
			{
				SubRangeStart = InStart;
				SubRangeLength = InLength;
			}

			void ClearSubRange()
			{
				SubRangeStart = 0;
				SubRangeLength = -1;
			}

			bool HasSubRange() const { return SubRangeLength >= 0; }

			uint32 GetRowCount() const
			{
				return HasSubRange() ? static_cast<uint32>(SubRangeLength) : Context.GetNumEntities();
			}

			TConstArrayView<RowHandle> GetRowHandles() const
			{
				TConstArrayView<RowHandle> All = MassEntitiesToRowsConversion(Context.GetEntities());
				return HasSubRange() ? All.Mid(SubRangeStart, SubRangeLength) : All;
			}

			const void* GetColumn(const UScriptStruct* ColumnType) const
			{
				const uint8* Base = reinterpret_cast<const uint8*>(Context.GetFragmentView(ColumnType).GetData());
				return HasSubRange() ? Base + SubRangeStart * ColumnType->GetStructureSize() : Base;
			}

			void* GetMutableColumn(const UScriptStruct* ColumnType)
			{
				uint8* Base = reinterpret_cast<uint8*>(Context.GetMutableFragmentView(ColumnType).GetData());
				return HasSubRange() ? Base + SubRangeStart * ColumnType->GetStructureSize() : Base;
			}

			void GetColumns(TArrayView<char*> RetrievedAddresses,
				TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes,
				TConstArrayView<EQueryAccessType> AccessTypes)
			{
				checkf(RetrievedAddresses.Num() == ColumnTypes.Num(), TEXT("Unable to retrieve a batch of columns as the number of addresses "
					"doesn't match the number of requested column."));
				checkf(RetrievedAddresses.Num() == AccessTypes.Num(), TEXT("Unable to retrieve a batch of columns as the number of addresses "
					"doesn't match the number of access types."));

				GetColumnsUnguarded(ColumnTypes.Num(), RetrievedAddresses.GetData(), ColumnTypes.GetData(), AccessTypes.GetData());
			}

			void GetColumnsUnguarded(
				int32 TypeCount,
				char** RetrievedAddresses,
				const TWeakObjectPtr<const UScriptStruct>* ColumnTypes,
				const EQueryAccessType* AccessTypes)
			{
				for (int32 Index = 0; Index < TypeCount; ++Index)
				{
					checkf(ColumnTypes->IsValid(), TEXT("Attempting to retrieve a column that is not available."));
					if (*AccessTypes == EQueryAccessType::ReadWrite)
					{
						uint8* Base = reinterpret_cast<uint8*>(Context.GetMutableFragmentView(ColumnTypes->Get()).GetData());
						*RetrievedAddresses = reinterpret_cast<char*>(HasSubRange() ? Base + SubRangeStart * ColumnTypes->Get()->GetStructureSize() : Base);
					}
					else
					{
						const uint8* Base = reinterpret_cast<const uint8*>(Context.GetFragmentView(ColumnTypes->Get()).GetData());
						*RetrievedAddresses = const_cast<char*>(reinterpret_cast<const char*>(HasSubRange() ? Base + SubRangeStart * ColumnTypes->Get()->GetStructureSize() : Base));
					}

					++RetrievedAddresses;
					++ColumnTypes;
					++AccessTypes;
				}
			}

			bool HasColumn(const UScriptStruct* ColumnType) const
			{
				if (UE::Mass::IsA<FMassTag>(ColumnType))
				{
					return Context.DoesArchetypeHaveTag(ColumnType);
				}
				if (UE::Mass::IsA<FMassFragment>(ColumnType))
				{
					return Context.DoesArchetypeHaveFragment(ColumnType);
				}
				const bool bIsTagOrFragment = false;
				checkf(bIsTagOrFragment, TEXT("Attempting to check for a column type that is not a column or tag."));
				return false;
			}

			bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const
			{
				FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
				FMassEntityManager& Manager = Context.GetEntityManagerChecked();
				FMassArchetypeHandle Archetype = Manager.GetArchetypeForEntity(Entity);
				// Entity was invalid (either never was or was destroyed).  Invalid entities have no columns
				if (!Archetype.IsValid())
				{
					return false;
				}
				
				if (UE::Mass::IsA<FMassTag>(ColumnType) || UE::Mass::IsA<FMassFragment>(ColumnType))
				{
					const FMassArchetypeCompositionDescriptor& Composition = Manager.GetArchetypeComposition(Archetype);
					return Composition.GetElementsBitSet().Contains(ColumnType);
				}
				const bool bIsTagOrFragment = false;
				checkf(bIsTagOrFragment, TEXT("Attempting to check for a column type that is not a column or tag."));
				return false;
			}

			float GetDeltaTimeSeconds() const
			{
				return Context.GetDeltaTimeSeconds();
			}

			void PushCommand(void (*CommandFunction)(void*), void* InCommandData) const
			{
				if (!ensure(CommandFunction))
				{
					return;
				}
				const FEnvironment::FEnvironmentCommand Command
				{
					.CommandFunction = CommandFunction,
					.CommandData = InCommandData
				};
				Environment.PushCommands(MakeConstArrayView(&Command, 1));
			}

			void* EmplaceObjectInScratch(size_t ObjectSize, size_t Alignment, void(* Construct)(void*, void*), void(* Destroy)(void*), void* SourceCommandContext) const
			{
				struct FDestructor final
				{
					using DestroyFnType = decltype(Destroy);
					FDestructor(DestroyFnType InDestroyFn, void* InObjectPtr)
						: DestroyFn(InDestroyFn)
						, ObjectPtr(InObjectPtr)
					{
					}
					~FDestructor()
					{
						DestroyFn(ObjectPtr);
					}
					DestroyFnType DestroyFn;
					void* ObjectPtr;
				};

				FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
				
				void* ObjectMemory = ScratchBuffer.AllocateUninitialized(ObjectSize, Alignment);
				Construct(ObjectMemory, SourceCommandContext);

				// The presence of a Destroy function implies that the objects that was just added to the scratch buffer
				// is not trivially destructible, hence needs its destructor called.
				// The API for the scratch buffer's internal memory allocator needs us to emplace a non-trivially destructible object
				// of some type. FDestructor is used to fulfill that role to destroy the object that was just constructed.
				if (Destroy)
				{
					ScratchBuffer.Emplace<FDestructor>(Destroy, ObjectMemory);	
				}
				return ObjectMemory;
			}
		};

		struct FRowColumnModifications
		{
			FMassTagBitSet AddedTags;
			FMassFragmentBitSet AddedFragments;
			TSet<FDynamicColumnDescription> AddedDynamicColumns;
			FMassTagBitSet RemovedTags;
			FMassFragmentBitSet RemovedFragments;

			bool operator==(const FRowColumnModifications& Other) const
			{
				return AddedTags == Other.AddedTags
					&& AddedFragments == Other.AddedFragments
					&& AddedDynamicColumns.Num() == Other.AddedDynamicColumns.Num()
					&& AddedDynamicColumns.Includes(Other.AddedDynamicColumns)
					&& RemovedTags == Other.RemovedTags
					&& RemovedFragments == Other.RemovedFragments;
			}
		};

		uint32 GetTypeHash(const FRowColumnModifications& InModification)
		{
			uint32 Hash = GetTypeHash(InModification.AddedTags);
			Hash = HashCombine(Hash, GetTypeHash(InModification.AddedFragments));
			Hash = HashCombine(Hash, uint32(InModification.AddedDynamicColumns.Num()));
			Hash = HashCombine(Hash, GetTypeHash(InModification.RemovedTags));
			return HashCombine(Hash, GetTypeHash(InModification.RemovedFragments));
		}

		struct FMassWithEnvironmentContextCommon : public FMassContextCommon
		{
			using Parent = FMassContextCommon;
		protected:
			void TedsColumnsToMassDescriptorIfActiveTable(
				FMassArchetypeCompositionDescriptor& Descriptor,
				TConstArrayView<const UScriptStruct*> ColumnTypes)
			{
				for (const UScriptStruct* ColumnType : ColumnTypes)
				{
					if (UE::Mass::IsA<FMassTag>(ColumnType))
					{
						if (this->Context.DoesArchetypeHaveTag(ColumnType))
						{
							Descriptor.Add(ColumnType);
						}
					}
					else
					{
						checkf(UE::Mass::IsA<FMassFragment>(ColumnType),
							TEXT("Given struct type is not a valid fragment or tag type."));
						if (this->Context.DoesArchetypeHaveFragment(ColumnType))
						{
							Descriptor.Add(ColumnType);
						}
					}
				}
			}

			void TedsColumnsToMassDescriptor(
				FMassArchetypeCompositionDescriptor& Descriptor,
				TConstArrayView<const UScriptStruct*> ColumnTypes)
			{
				for (const UScriptStruct* ColumnType : ColumnTypes)
				{
					if (UE::Mass::IsA<FMassTag>(ColumnType))
					{
						Descriptor.Add(ColumnType);
					}
					else
					{
						checkf(UE::Mass::IsA<FMassFragment>(ColumnType),
							TEXT("Given struct type is not a valid fragment or tag type."));
						Descriptor.Add(ColumnType);

					}
				}
			}

		public:
			using ObjectCopyOrMove = void (*)(const UScriptStruct& TypeInfo, void* Destination, void* Source);
	
			FMassWithEnvironmentContextCommon(FMassExecutionContext& InContext, FEnvironment& InEnvironment, bool bInBatchDeferredCommands)
				: FMassContextCommon(InContext, InEnvironment)
				, bBatchDeferredCommands(bInBatchDeferredCommands)
			{}

			~FMassWithEnvironmentContextCommon()
			{
				struct FRelocator
				{
					ObjectCopyOrMove CopyOrMove = nullptr;
					const UScriptStruct* FragmentType = nullptr;
					FName Identifier;
					void* Object = nullptr;
				};

				struct FRowOperations
				{
					FRowColumnModifications ColumnsChange;
					TArray<FRelocator> Relocators;
				};

				if (bBatchDeferredCommands)
				{
					TMap<FMassEntityHandle, FRowOperations> RowOperations;

					// Process the per row operations
					{
						TSet<TPair<FMassTagBitSet,FMassFragmentBitSet>
							, DefaultKeyFuncs<TPair<FMassTagBitSet, FMassFragmentBitSet>>
							, TInlineSetAllocator<16>> 
							UniqueAdd;
						for (const FAddedColumns* const AddColumns : AddedColumnsQueue)
						{
							UniqueAdd.Add(TPair<FMassTagBitSet, FMassFragmentBitSet>(AddColumns->AddDescriptor.Get<FMassTagBitSet>(), AddColumns->AddDescriptor.Get<FMassFragmentBitSet>()));
						}
						const int32 AddedColumnsRowGuess = UniqueAdd.IsEmpty() ? 0 : AddedColumnsQueue.Num() / UniqueAdd.Num();
						UniqueAdd.Empty();

						TSet<const UScriptStruct*
							, DefaultKeyFuncs<const UScriptStruct*>
							, TInlineSetAllocator<16>>
							UniqueColumns;
						for (const FAddValueColumn* const AddValueColumn : AddedColumnWithRelocationQueue)
						{
							UniqueColumns.Add(AddValueColumn->FragmentType);
						}
						const int32 AddedColumnsWithRelocationRowGuess = UniqueColumns.IsEmpty() ? 0 : AddedColumnWithRelocationQueue.Num() / UniqueColumns.Num();
						UniqueColumns.Empty();

						TSet<FDynamicColumnDescription
							, DefaultKeyFuncs<FDynamicColumnDescription>
							, TInlineSetAllocator<16>> UniqueDynamicColumnsAdd;
						for (const FAddDynamicColumn* const AddDynamicColumn : AddedDynamicColumnWithRelocationQueue)
						{
							UniqueDynamicColumnsAdd.Add(AddDynamicColumn->Description);
						}
						const int32 AddDynamicColumnRowsGuess = UniqueDynamicColumnsAdd.IsEmpty() ? 0 : AddedDynamicColumnWithRelocationQueue.Num() / UniqueDynamicColumnsAdd.Num();
						UniqueDynamicColumnsAdd.Empty();


						int32 AddedColumnsBathRowsCount = 0;
						for (FAddedColumnsBatch* const AddColumns : AddedColumnsBatchQueue)
						{
							AddedColumnsBathRowsCount = FMath::Max(AddColumns->Entities.Num(), AddedColumnsBathRowsCount);
						}

						int32 AddedDynamicColumnsRowCount = 0;
						for (FAddDynamicColumnsBatch* const AddedDynamicColumns : AddedDynamicColumnsBatchQueue)
						{
							AddedDynamicColumnsRowCount += FMath::Max(AddedDynamicColumns->Rows.Num(), AddedDynamicColumnsRowCount);
						}

						int32 RemovedColumnsBatchRowCount = 0;
						for (FRemovedColumnsBatch* const RemovedColumns : RemovedColumnsBatchQueue)
						{
							RemovedColumnsBatchRowCount = FMath::Max(RemovedColumns->Entities.Num(), RemovedColumnsBatchRowCount);
						}


						RowOperations.Reserve(FMath::Max(AddedColumnsRowGuess,
							AddedColumnsBathRowsCount,
							AddedColumnsWithRelocationRowGuess,
							AddDynamicColumnRowsGuess,
							AddedDynamicColumnsRowCount,
							RemovedColumnsQueue.Num(),
							RemovedColumnsBatchRowCount
						));


						for (FAddedColumns* const AddColumns : AddedColumnsQueue)
						{
							FRowOperations& Operations = RowOperations.FindOrAdd(AddColumns->Entity);
							Operations.ColumnsChange.AddedTags = AddColumns->AddDescriptor.Get<FMassTagBitSet>();
							Operations.ColumnsChange.AddedFragments = AddColumns->AddDescriptor.Get<FMassFragmentBitSet>();
						}
						AddedColumnsQueue.Empty();

						for (FAddedColumnsBatch* const AddColumns : AddedColumnsBatchQueue)
						{
							for (const FMassEntityHandle& Row : AddColumns->Entities)
							{
								FRowOperations& Operations = RowOperations.FindOrAdd(Row);
								Operations.ColumnsChange.AddedTags += AddColumns->AddDescriptor.Get<FMassTagBitSet>();
								Operations.ColumnsChange.AddedFragments += AddColumns->AddDescriptor.Get<FMassFragmentBitSet>();
							}
						}
						AddedColumnsBatchQueue.Empty();

						for (FAddValueColumn* const AddColumn : AddedColumnWithRelocationQueue)
						{
							FRowOperations& Operations = RowOperations.FindOrAdd(AddColumn->Entity);
							Operations.ColumnsChange.AddedFragments.Add(AddColumn->FragmentType);
							Operations.Relocators.Add(FRelocator{
									.CopyOrMove = AddColumn->Relocator,
									.FragmentType = AddColumn->FragmentType,
									.Object = AddColumn->Object,
								});
						}
						AddedColumnWithRelocationQueue.Empty();

						for (FAddDynamicColumn* const AddDynamicColumn : AddedDynamicColumnWithRelocationQueue)
						{
							FRowOperations& Operations = RowOperations.FindOrAdd(AddDynamicColumn->Entity);
							Operations.ColumnsChange.AddedDynamicColumns.Add(AddDynamicColumn->Description);
							Operations.Relocators.Add(FRelocator{
									.CopyOrMove = AddDynamicColumn->Relocator,
									.FragmentType = AddDynamicColumn->Description.TemplateType,
									.Identifier = AddDynamicColumn->Description.Identifier,
									.Object = AddDynamicColumn->Object,
								});
						}
						AddedDynamicColumnWithRelocationQueue.Empty();

						for (FAddDynamicColumnsBatch* const AddedDynamicColumns : AddedDynamicColumnsBatchQueue)
						{
							for (const RowHandle& Row : AddedDynamicColumns->Rows)
							{
								FRowOperations& Operations = RowOperations.FindOrAdd(FMassEntityHandle::FromNumber(Row));
								Operations.ColumnsChange.AddedDynamicColumns.Append(AddedDynamicColumns->Descriptions);
							}
						}
						AddedDynamicColumnsBatchQueue.Empty();

						for (FRemovedColumns* const RemovedColumns : RemovedColumnsQueue)
						{
							FRowOperations& Operations = RowOperations.FindOrAdd(RemovedColumns->Entity);
							Operations.ColumnsChange.RemovedTags += RemovedColumns->RemovedColumns.Get<FMassTagBitSet>();
							Operations.ColumnsChange.RemovedFragments += RemovedColumns->RemovedColumns.Get<FMassFragmentBitSet>();
						}
						RemovedColumnsQueue.Empty();

						for (FRemovedColumnsBatch* const RemovedColumns : RemovedColumnsBatchQueue)
						{
							for (const FMassEntityHandle& Row : RemovedColumns->Entities)
							{
								FRowOperations& Operations = RowOperations.FindOrAdd(Row);
								Operations.ColumnsChange.RemovedTags += RemovedColumns->RemovedColumns.Get<FMassTagBitSet>();
								Operations.ColumnsChange.RemovedFragments += RemovedColumns->RemovedColumns.Get<FMassFragmentBitSet>();
							}
						}
						RemovedColumnsBatchQueue.Empty();
					}


					// Convert the per row operations to batched changes
					TMap<FRowColumnModifications, TArray<FMassEntityHandle>> ChangesAndRows;
					for (const TPair<FMassEntityHandle, FRowOperations>& RowOperation : RowOperations)
					{
						ChangesAndRows.FindOrAdd(RowOperation.Value.ColumnsChange).Add(RowOperation.Key);
					}

					// Convert the Add row to batched changes
					TMap<FMassArchetypeHandle, TArray<FMassEntityHandle>> RowsToAdd;
					for (const FAddRow& AddRow : AddedRowsQueue)
					{
						RowsToAdd.FindOrAdd(AddRow.Archetype).Add(AddRow.Entity);
					}

					// Prepare the remove row array for Mass
					TArray<FMassEntityHandle> EntitiesToDestroy;
					EntitiesToDestroy.Reserve(RemovedRowsQueue.Num());
					for (const RowHandle& Row : RemovedRowsQueue)
					{
						EntitiesToDestroy.Add(FMassEntityHandle::FromNumber(Row));
					}
					RemovedRowsQueue.Empty();

					this->Context.Defer().template PushCommand<FMassDeferredChangeCompositionCommand>(
						[InChangesAndRows = MoveTemp(ChangesAndRows),
						InRowOperations = MoveTemp(RowOperations),
						PtrToEnvironment = &Environment,
						InEntitiesToDestroy = MoveTemp(EntitiesToDestroy),
						InRowsToAdd = MoveTemp(RowsToAdd)
						](FMassEntityManager& System) mutable
						{
							{
								TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContext = System.GetOrMakeCreationContext();

								// Adds Rows
								for (TPair<FMassArchetypeHandle, TArray<FMassEntityHandle>>& ArchetypeAndRows : InRowsToAdd)
								{
									FMassArchetypeEntityCollection EntityCollection(FMassArchetypeHandle(), ArchetypeAndRows.Value, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates);
									const FMassArchetypeCompositionDescriptor& ArchetypeComposition = System.GetArchetypeComposition(ArchetypeAndRows.Key);

									FMassArchetypeEntityCollectionWithPayload EntityCollectionWithPayload(MoveTemp(EntityCollection));
									System.BatchBuildEntities(EntityCollectionWithPayload, ArchetypeComposition);
								}

								using EntityHandleArray = TArray<FMassEntityHandle, TInlineAllocator<32>>;
								using EntityArchetypeLookup = TMap<FMassArchetypeHandle, EntityHandleArray, TInlineSetAllocator<32>>;
								using ArchetypeEntityArray = TArray<FMassArchetypeEntityCollection, TInlineAllocator<32>>;

								EntityArchetypeLookup LookupTable;
								ArchetypeEntityArray EntityCollections;

								// Change the entity types
								for (TPair<FRowColumnModifications, TArray<FMassEntityHandle>>& BatchedModification : InChangesAndRows)
								{
									for (const FDynamicColumnDescription& DynamicColumnAdded : BatchedModification.Key.AddedDynamicColumns)
									{
										BatchedModification.Key.AddedFragments.Add(PtrToEnvironment->GenerateDynamicColumn(*DynamicColumnAdded.TemplateType, DynamicColumnAdded.Identifier));
									}

									const FRowColumnModifications& ColumnModification = BatchedModification.Key;
									if (!ColumnModification.AddedFragments.IsEmpty() || !ColumnModification.RemovedFragments.IsEmpty())
									{
										LookupTable.Reset();
										EntityCollections.Reset();

										// Sort rows (entities) into to matching table (archetype) bucket.
										for (FMassEntityHandle EntityHandle : BatchedModification.Value)
										{
											if (System.IsEntityValid(EntityHandle))
											{
												FMassArchetypeHandle Archetype = System.GetArchetypeForEntity(EntityHandle);
												EntityHandleArray& EntityCollection = LookupTable.FindOrAdd(Archetype);
												EntityCollection.Add(EntityHandle);
											}
										}

										// Construct table (archetype) specific row (entity) collections.
										EntityCollections.Reserve(LookupTable.Num());
										for (auto It = LookupTable.CreateConstIterator(); It; ++It)
										{
											// Since we use a map to combine all the operation on a row in one via a map go, we already know there won't be any duplicate
											EntityCollections.Emplace(It.Key(), It.Value(), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
										}

										// This could be improved by adding an operation that would both combine the Fragments and Tags change in one bath operation.
										System.BatchChangeFragmentCompositionForEntities(EntityCollections, ColumnModification.AddedFragments, ColumnModification.RemovedFragments);
									}

									if (!ColumnModification.AddedTags.IsEmpty() || !ColumnModification.RemovedTags.IsEmpty())
									{
										LookupTable.Reset();
										EntityCollections.Reset();

										// Sort rows (entities) into to matching table (archetype) bucket.
										for (FMassEntityHandle EntityHandle : BatchedModification.Value)
										{
											if (System.IsEntityValid(EntityHandle))
											{
												FMassArchetypeHandle Archetype = System.GetArchetypeForEntity(EntityHandle);
												EntityHandleArray& EntityCollection = LookupTable.FindOrAdd(Archetype);
												EntityCollection.Add(EntityHandle);
											}
										}

										// Construct table (archetype) specific row (entity) collections.
										EntityCollections.Reserve(LookupTable.Num());
										for (auto It = LookupTable.CreateConstIterator(); It; ++It)
										{
											// Since we use a map to combine all the operation on a row in one via a map go, we already know there won't be any duplicate
											EntityCollections.Emplace(It.Key(), It.Value(), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
										}

										System.BatchChangeTagsForEntities(EntityCollections, ColumnModification.AddedTags, ColumnModification.RemovedTags);
									}
								}

								// Do the relocation
								for (TPair<FMassEntityHandle, FRowOperations>& PerRowOperations : InRowOperations)
								{
									const FMassEntityHandle Handle = PerRowOperations.Key;
									for (FRelocator& Relocator : PerRowOperations.Value.Relocators)
									{
										if (!Relocator.Identifier.IsNone())
										{
											Relocator.FragmentType = PtrToEnvironment->GenerateDynamicColumn(*Relocator.FragmentType, Relocator.Identifier);
										}

										FStructView Fragment = System.GetFragmentDataStruct(Handle, Relocator.FragmentType);
										Relocator.CopyOrMove(*Relocator.FragmentType, Fragment.GetMemory(), Relocator.Object);
									}
								}
							} // We use a scope here to trigger the notification. We can't delete a row while we are holding a EntityCreationContext.

							// Remove Rows
							if (!InEntitiesToDestroy.IsEmpty())
							{
								System.BatchDestroyEntities(InEntitiesToDestroy);
							}
						});
				}
			}

			uint64 GetUpdateCycleId() const 
			{
				return Environment.GetUpdateCycleId();
			}

			bool IsRowAvailable(RowHandle Row) const
			{
				return Environment.GetMassEntityManager().IsEntityValid(FMassEntityHandle::FromNumber(Row));
			}

			bool IsRowAssigned(RowHandle Row) const
			{
				return Environment.GetMassEntityManager().IsEntityActive(FMassEntityHandle::FromNumber(Row));
			}
	
			void ActivateQueries(FName ActivationName)
			{
				this->Context.Defer().template PushCommand<FMassDeferredCommand<EMassCommandOperationType::None>>(
					[Environment = &this->Environment, ActivationName](FMassEntityManager&)
					{
						Environment->GetQueryStore().ActivateQueries(ActivationName);
					});
			}

			template<typename InputT, typename OutputT>
			void CopyArrayViews(const InputT Input, OutputT Output)
			{
				for (int32 Index = 0, End = Input.Num(); Index < End; ++Index)
				{
					Output[Index] = Input[Index];
				}
			}

			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<UE::Editor::DataStorage::FDynamicColumnDescription> DynamicColumnDescriptions)
			{
				FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
				
				FAddDynamicColumnsBatch* CommandData = ScratchBuffer.Emplace<FAddDynamicColumnsBatch>();
				TArrayView<RowHandle> ScratchRows = 
					ScratchBuffer.AllocateUninitializedArray<RowHandle>(Rows.Num());
				TArrayView<FDynamicColumnDescription> ScratchDescriptions = 
					ScratchBuffer.AllocateZeroInitializedArray<FDynamicColumnDescription>(DynamicColumnDescriptions.Num());
				TArrayView<const UScriptStruct*> ScratchTypes = 
					ScratchBuffer.AllocateUninitializedArray<const UScriptStruct*>(DynamicColumnDescriptions.Num());
				
				CopyArrayViews(Rows, ScratchRows);
				CopyArrayViews(DynamicColumnDescriptions, ScratchDescriptions);

				*CommandData = FAddDynamicColumnsBatch
				{
					.Rows = ScratchRows,
					.Descriptions = ScratchDescriptions,
					.ResolvedTypes = ScratchTypes
				};

				if (bBatchDeferredCommands)
				{
					AddedDynamicColumnsBatchQueue.AddElement(CommandData);
				}
				else
				{ 
					this->Context.Defer().template PushCommand<FMassDeferredAddCommand>(
						[CommandData, this](FMassEntityManager& System)
						{
							for (int32 DynamicTypeIndex = 0, DynamicTypeIndexEnd = CommandData->Descriptions.Num(); DynamicTypeIndex < DynamicTypeIndexEnd; ++DynamicTypeIndex)
							{
								const FDynamicColumnDescription& Description = CommandData->Descriptions[DynamicTypeIndex];
								const UScriptStruct* DynamicColumnType = Environment.GenerateDynamicColumn(*Description.TemplateType, Description.Identifier);
								CommandData->ResolvedTypes[DynamicTypeIndex] = DynamicColumnType;
							}

							FMassArchetypeCompositionDescriptor AddDescriptor;
							TedsColumnsToMassDescriptor(AddDescriptor, CommandData->ResolvedTypes);
						
							for (RowHandle Row : CommandData->Rows)
							{
								FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
								if (System.IsEntityActive(Entity))
								{
									System.AddCompositionToEntity_GetDelta(Entity, AddDescriptor);
								}
							}
						});
				}
			}

			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ObjectType) 
			{
				return AddColumnUninitialized(Row, ObjectType,
					[](const UScriptStruct& TypeInfo, void* Destination, void* Source)
					{
						TypeInfo.CopyScriptStruct(Destination, Source);
					});
			}

			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ObjectType, ObjectCopyOrMove Relocator)
			{
				checkf(UE::Mass::IsA<FMassFragment>(ObjectType), TEXT("Column [%s] can not be a tag"), *ObjectType->GetName());
	
				FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
				void* ColumnData = ScratchBuffer.AllocateUninitialized(ObjectType->GetStructureSize(), ObjectType->GetMinAlignment());
				FAddValueColumn* AddedColumn =
					ScratchBuffer.Emplace<FAddValueColumn>(Relocator, ObjectType, FMassEntityHandle::FromNumber(Row), ColumnData);

				if (bBatchDeferredCommands)
				{
					AddedColumnWithRelocationQueue.AddElement(AddedColumn);
				}
				else
				{
					this->Context.Defer().template PushCommand<FMassDeferredAddCommand>(
						[AddedColumn](FMassEntityManager& System)
						{
							// Check entity before proceeding. It's possible it may have been invalidated before this deferred call fired.
							if (System.IsEntityActive(AddedColumn->Entity))
							{
								// Check before adding.  Mass's AddFragmentToEntity is not idempotent and will assert if adding
								// column to a row that already has one
								FStructView Fragment = System.GetFragmentDataStruct(AddedColumn->Entity, AddedColumn->FragmentType);
								if (!Fragment.IsValid())
								{
									System.AddFragmentToEntity(AddedColumn->Entity, AddedColumn->FragmentType, 
										[AddedColumn](void* Fragment, const UScriptStruct& FragmentType)
										{
											AddedColumn->Relocator(FragmentType, Fragment, AddedColumn->Object);
										});
								}
								else
								{
									AddedColumn->Relocator(*AddedColumn->FragmentType, Fragment.GetMemory(), AddedColumn->Object);
								}
							}
						});
					}
		
				return ColumnData;
			}

			void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& Description)
			{
				return AddColumnUninitialized(Row, Description, [](const UScriptStruct& TypeInfo, void* Destination, void* Source)
				{
					TypeInfo.CopyScriptStruct(Destination, Source);
				});
			}

			void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& Description, ObjectCopyOrMove Relocator)
			{
				FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
				// DynamicColumn types are derivations from their template that add no new members.  The size and alignment will be the same
				void* ColumnData = ScratchBuffer.AllocateUninitialized(Description.TemplateType->GetStructureSize(), Description.TemplateType->GetMinAlignment());
				FAddDynamicColumn* AddedColumn =
					ScratchBuffer.Emplace<FAddDynamicColumn>(Relocator, Description, FMassEntityHandle::FromNumber(Row), ColumnData);
		
				if (bBatchDeferredCommands)
				{
					AddedDynamicColumnWithRelocationQueue.AddElement(AddedColumn);
				}
				else
				{
					this->Context.Defer().template PushCommand<FMassDeferredAddCommand>(
						[AddedColumn, PtrToEnvironment = &Environment](FMassEntityManager& System)
						{
							// Check entity before proceeding. It's possible it may have been invalidated before this deferred call fired.
							if (System.IsEntityActive(AddedColumn->Entity))
							{
								const UScriptStruct* DynamicStructType = PtrToEnvironment->GenerateDynamicColumn(*AddedColumn->Description.TemplateType, AddedColumn->Description.Identifier);

								FStructView Fragment = System.GetFragmentDataStruct(AddedColumn->Entity, DynamicStructType);
								// Check before adding.  Mass's AddFragmentToEntity is not idempotent and will assert if adding
								// column to a row that already has one
								if (!Fragment.IsValid())
								{
									System.AddFragmentToEntity(AddedColumn->Entity, DynamicStructType, 
										[AddedColumn](void* Fragment, const UScriptStruct& FragmentType)
										{
											AddedColumn->Relocator(FragmentType, Fragment, AddedColumn->Object);
										});
								}
								else
								{
									AddedColumn->Relocator(*DynamicStructType, Fragment.GetMemory(), AddedColumn->Object);
								}
							}
						});
				}
		
				return ColumnData;
			}
	
			void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) 
			{
				FAddedColumns* AddedColumns = Environment.GetScratchBuffer().template Emplace<FAddedColumns>();
				TedsColumnsToMassDescriptor(AddedColumns->AddDescriptor, ColumnTypes);
				AddedColumns->Entity = FMassEntityHandle::FromNumber(Row);

				if (bBatchDeferredCommands)
				{ 
					AddedColumnsQueue.AddElement(AddedColumns);
				}
				else
				{ 
					this->Context.Defer().template PushCommand<FMassDeferredAddCommand>(
						[AddedColumns](FMassEntityManager& System)
						{
							if (System.IsEntityActive(AddedColumns->Entity))
							{
								System.AddCompositionToEntity_GetDelta(AddedColumns->Entity, AddedColumns->AddDescriptor);
							}
						});
				}
			}

			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) 
			{
				FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
				FAddedColumnsBatch* AddedColumns = ScratchBuffer.Emplace<FAddedColumnsBatch>();
				TedsColumnsToMassDescriptor(AddedColumns->AddDescriptor, ColumnTypes);
		
				AddedColumns->Entities = ScratchBuffer.AllocateZeroInitializedArray<FMassEntityHandle>(Rows.Num());
				FMassEntityHandle* Entities = AddedColumns->Entities.GetData();
				for (RowHandle Row : Rows)
				{
					*Entities = FMassEntityHandle::FromNumber(Row);
					Entities++;
				}
				
				if (bBatchDeferredCommands)
				{
					AddedColumnsBatchQueue.AddElement(AddedColumns);
				}
				else
				{
				this->Context.Defer().template PushCommand<FMassDeferredAddCommand>(
					[AddedColumns](FMassEntityManager& System)
					{
						for (FMassEntityHandle& Entity : AddedColumns->Entities)
						{
							if (System.IsEntityActive(Entity))
							{
								System.AddCompositionToEntity_GetDelta(Entity, AddedColumns->AddDescriptor);
							}
						}
					});
				}
			}

			void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) 
			{
				FRemovedColumns* RemovedColumns = Environment.GetScratchBuffer().template Emplace<FRemovedColumns>();
				RemovedColumns->RemovedColumns = FMassElementBitSet(ColumnTypes);
				RemovedColumns->Entity = FMassEntityHandle::FromNumber(Row);

				if (bBatchDeferredCommands)
				{
					RemovedColumnsQueue.AddElement(RemovedColumns);
				}
				else
				{
					this->Context.Defer().template PushCommand<FMassDeferredRemoveCommand>(
						[RemovedColumns](FMassEntityManager& System)
						{
							if (FMassArchetypeHandle Archetype = System.GetArchetypeForEntity(RemovedColumns->Entity); Archetype.IsValid())
							{
								FMassArchetypeEntityCollection Collection(Archetype, RemovedColumns->Entity);
								System.BatchChangeCompositionForEntities({ Collection }, {}, RemovedColumns->RemovedColumns);
							}
						});
				}
			}

			void RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) 
			{
				switch (Rows.Num())
				{
				case 0:
					return;
				case 1:
					RemoveColumns(*Rows.GetData(), ColumnTypes);
					return;
				default:
				{
					FScratchBuffer& ScratchBuffer = Environment.GetScratchBuffer();
					FRemovedColumnsBatch* RemovedColumns = ScratchBuffer.Emplace<FRemovedColumnsBatch>();

					RemovedColumns->RemovedColumns = FMassElementBitSet(ColumnTypes);
					RemovedColumns->Entities = ScratchBuffer.EmplaceArray<FMassEntityHandle>(Rows.Num());
					FMassEntityHandle* Entities = RemovedColumns->Entities.GetData();
					for (RowHandle Row : Rows)
					{
						*Entities = FMassEntityHandle::FromNumber(Row);
						Entities++;
					}

					if (bBatchDeferredCommands)
					{
						RemovedColumnsBatchQueue.AddElement(RemovedColumns);
					}
					else
					{
						this->Context.Defer().template PushCommand<FMassDeferredRemoveCommand>(
							[RemovedColumns](FMassEntityManager& System)
							{
								using EntityHandleArray = TArray<FMassEntityHandle, TInlineAllocator<32>>;
								using EntityArchetypeLookup = TMap<FMassArchetypeHandle, EntityHandleArray, TInlineSetAllocator<32>>;
								using EntityCollectionArray = TArray<FMassArchetypeEntityCollection, TInlineAllocator<32>>;

								TArrayView<FMassEntityHandle> Entities = RemovedColumns->Entities;
								EntityArchetypeLookup LookupTable;

								for (FMassEntityHandle Entity : Entities)
								{
									if (FMassArchetypeHandle Archetype = System.GetArchetypeForEntity(Entity); Archetype.IsValid())
									{
										LookupTable.FindOrAdd(Archetype).Add(Entity);
									}
								}

								EntityCollectionArray EntityCollections;
								EntityCollections.Reserve(LookupTable.Num());
								for (EntityArchetypeLookup::TIterator It = LookupTable.CreateIterator(); It; ++It)
								{
									EntityCollections.Emplace(It.Key(), TConstArrayView<FMassEntityHandle>(It.Value()),
										FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates);
								}

								System.BatchChangeCompositionForEntities(EntityCollections, {}, RemovedColumns->RemovedColumns);
							});
					}
				}}
			}
	
			RowHandle AddRow(TableHandle Table) 
			{
				FMassEntityHandle EntityHandle = Environment.GetMassEntityManager().ReserveEntity();
				FMassArchetypeHandle ArchetypeHandle = Environment.GetTableManager().GetArchetype(Table);

				if (!ArchetypeHandle.IsValid())
				{
					return InvalidRowHandle;
				}

				FAddRow AddRowTmp{
						.Entity = EntityHandle,
						.Archetype = MoveTemp(ArchetypeHandle)
					};

				if (bBatchDeferredCommands)
				{
					AddedRowsQueue.AddElement(MoveTemp(AddRowTmp));
				}
				else
				{
					this->Context.Defer().template PushCommand<FMassDeferredCreateCommand>(
						[AddRow = MoveTemp(AddRowTmp)](FMassEntityManager& System)
						{
							const FMassArchetypeSharedFragmentValues SharedFragmentValues;
							System.BuildEntity(AddRow.Entity, AddRow.Archetype, SharedFragmentValues);
						});
				}
		
				const RowHandle TedsRowHandle = EntityHandle.AsNumber();
				return TedsRowHandle;
			}

			void RemoveRow(RowHandle Row) 
			{
				if (bBatchDeferredCommands)
				{
					RemovedRowsQueue.AddElement(Row);
				}
				else
				{
					this->Context.Defer().DestroyEntity(FMassEntityHandle::FromNumber(Row));
				}
			}

			void RemoveRows(TConstArrayView<RowHandle> Rows) 
			{
				if (bBatchDeferredCommands)
				{
					for (const RowHandle& Row : Rows)
					{
						RemovedRowsQueue.AddElement(Row);
					}
				}
				else
				{
					this->Context.Defer().DestroyEntities(RowsToMassEntitiesConversion(Rows));
				}
			}

			template<typename ContextType>
			void SetParent(ContextType& Context, RowHandle Target, RowHandle Parent) const
			{
				if (HierarchyAccessInterface)
				{
					HierarchyAccessInterface->SetParentRow(Context, Target, Parent);
				}
			}

			template<typename ContextType>
			void SetUnresolvedParent(ContextType& Context, RowHandle Target, FMapKey ParentId, FName MappingDomain) const
			{
				if (HierarchyAccessInterface)
				{
					HierarchyAccessInterface->SetUnresolvedParent(Context, Target, ParentId, MappingDomain);
				}
			}

			template<typename ContextType>
			RowHandle GetParent(ContextType& Context, RowHandle Target) const
			{
				if (HierarchyAccessInterface)
				{
					return HierarchyAccessInterface->GetParentRow(Context, Target);
				}
				return InvalidRowHandle;
			}
			
			struct FAddedColumns
			{
				FMassArchetypeCompositionDescriptor AddDescriptor;
				FMassEntityHandle Entity;
			};

			struct FAddedColumnsBatch
			{
				FMassArchetypeCompositionDescriptor AddDescriptor;
				TArrayView<FMassEntityHandle> Entities;
			};

			struct FAddValueColumn
			{
				ObjectCopyOrMove Relocator;
				const UScriptStruct* FragmentType;
				FMassEntityHandle Entity;
				void* Object;

				FAddValueColumn() = default;
				FAddValueColumn(ObjectCopyOrMove InRelocator, const UScriptStruct* InFragmentType, FMassEntityHandle InEntity, void* InObject)
					: Relocator(InRelocator)
					, FragmentType(InFragmentType)
					, Entity(InEntity)
					, Object(InObject)
				{}

				~FAddValueColumn()
				{
					if (this->FragmentType && (this->FragmentType->StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor)) == 0)
					{
						this->FragmentType->DestroyStruct(this->Object);
					}
				}
			};

			struct FAddDynamicColumn
			{
				ObjectCopyOrMove Relocator;
				FDynamicColumnDescription Description;
				FMassEntityHandle Entity;
				void* Object;
				bool bNeedsDestruction;

				FAddDynamicColumn() = default;
				FAddDynamicColumn(ObjectCopyOrMove InRelocator, const FDynamicColumnDescription& InDescription, FMassEntityHandle InEntity, void* InObject)
					: Relocator(InRelocator)
					, Description(InDescription)
					, Entity(InEntity)
					, Object(InObject)
				{
					// Check here and cache off the result to avoid command buffer needing to dereference UScriptStruct to check if anything
					// needs to be done. In many cases, this is expected to be false
					bNeedsDestruction = (this->Description.TemplateType->StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor)) == 0;
				}

				~FAddDynamicColumn()
				{
					if (bNeedsDestruction)
					{
						this->Description.TemplateType->DestroyStruct(this->Object);
					}
				}
			};

			struct FAddDynamicColumnsBatch
			{
				TConstArrayView<RowHandle> Rows;
				TConstArrayView<FDynamicColumnDescription> Descriptions;
				TArrayView<const UScriptStruct*> ResolvedTypes;
			};

			struct FRemovedColumns
			{
				FMassElementBitSet RemovedColumns;
				FMassEntityHandle Entity;
			};

			struct FRemovedColumnsBatch
			{
				FMassElementBitSet RemovedColumns;
				TArrayView<FMassEntityHandle> Entities;
			};

			struct FAddRow
			{
				FMassEntityHandle Entity;
				FMassArchetypeHandle Archetype;
			};

			bool bBatchDeferredCommands = false;
			TChunkedArray<FAddedColumns*> AddedColumnsQueue;
			TChunkedArray<FAddedColumnsBatch*> AddedColumnsBatchQueue;
			TChunkedArray<FAddValueColumn*> AddedColumnWithRelocationQueue;
			TChunkedArray<FAddDynamicColumn*> AddedDynamicColumnWithRelocationQueue;
			TChunkedArray<FAddDynamicColumnsBatch*> AddedDynamicColumnsBatchQueue;
			TChunkedArray<FRemovedColumns*> RemovedColumnsQueue;
			TChunkedArray<FRemovedColumnsBatch*> RemovedColumnsBatchQueue;
			TChunkedArray<FAddRow> AddedRowsQueue;
			TChunkedArray<RowHandle> RemovedRowsQueue;
			const FTedsHierarchyAccessInterface* HierarchyAccessInterface = nullptr;
		};

		using FMassDirectContextForwarderBase = Queries::Private::TImplementationForwarder<
			Queries::FProcessorContextEnvironment,
			IDirectQueryContext,
			Queries::FTableInfoContextCapability,
			Queries::FDynamicColumnInfoContextCapability>;
		struct FMassDirectContextForwarder final : FMassDirectContextForwarderBase
		{
			FMassDirectContextForwarder(FMassExecutionContext& InContext, FEnvironment& InEnvironment, FQueryDescription& Description)
				: FMassDirectContextForwarderBase(InEnvironment, InContext)
				, Implementation(InContext, InEnvironment)
			{
				FHierarchyHandle Handle = InEnvironment.GetHierarchyRegistrar().FindHierarchyByName(Description.Hierarchy);
				HierarchyAccessInterface = InEnvironment.GetHierarchyRegistrar().GetAccessInterface(Handle);
			}
	
			uint32 GetRowCount() const override { return Implementation.GetRowCount(); }
			TConstArrayView<RowHandle> GetRowHandles() const override { return Implementation.GetRowHandles(); }
			const void* GetColumn(const UScriptStruct* ColumnType) const override { return Implementation.GetColumn(ColumnType); }
			void* GetMutableColumn(const UScriptStruct* ColumnType) override { return Implementation.GetMutableColumn(ColumnType); }
			void GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, TConstArrayView<EQueryAccessType> AccessTypes) override { return Implementation.GetColumns(RetrievedAddresses, ColumnTypes, AccessTypes); }
			void GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes, const EQueryAccessType* AccessTypes) override { return Implementation.GetColumnsUnguarded(TypeCount, RetrievedAddresses, ColumnTypes, AccessTypes); }
			bool HasColumn(const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(ColumnType); }
			bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(Row, ColumnType); }
			float GetDeltaTimeSeconds() const override { return Implementation.GetDeltaTimeSeconds(); }
			virtual void SetParentRow(RowHandle Target, RowHandle Parent) override { if (HierarchyAccessInterface) { HierarchyAccessInterface->SetParentRow(*static_cast<IDirectQueryContext*>(this), Target, Parent); } }
			virtual void SetRelation(RelationTypeHandle Type, RowHandle Target, RowHandle NewObject) override
			{
				FSetRelationObjectCommand Command;
				Command.Type = Type;
				Command.Target = Target;
				Command.NewObject = NewObject;
				Command.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				ICommonQueryContext::PushCommand(MoveTemp(Command));
			}
			virtual void SetUnresolvedParent(RowHandle Target, FMapKey ParentId, FName MappingDomain) override
			{
				if (HierarchyAccessInterface)
				{
					HierarchyAccessInterface->SetUnresolvedParent(*static_cast<IDirectQueryContext*>(this), Target, ParentId, MappingDomain);
				}
			}
			virtual RowHandle GetParentRow(RowHandle Target) const override { return HierarchyAccessInterface ? HierarchyAccessInterface->GetParentRow(*static_cast<const IDirectQueryContext*>(this), Target) : InvalidRowHandle; }
			virtual void PushCommand(void(* CommandFunction)(void*), void* InCommandData) override  { return Implementation.PushCommand(CommandFunction, InCommandData); }
		
		protected:
			void* EmplaceObjectInScratch(const FEmplaceObjectParams& Params) override { return Implementation.EmplaceObjectInScratch(Params.ObjectSize, Params.Alignment, Params.Construct, Params.Destroy, Params.SourceObject); }

		public:

			FMassContextCommon Implementation;
			const FTedsHierarchyAccessInterface* HierarchyAccessInterface = nullptr;
		};

		using FMassSubqueryContextForwarderBase = Queries::Private::TImplementationForwarder<
			Queries::FProcessorContextEnvironment,
			ISubqueryContext,
			Queries::FTableInfoContextCapability,
			Queries::FDynamicColumnInfoContextCapability>;
		struct FMassSubqueryContextForwarder : FMassSubqueryContextForwarderBase
		{
			FMassSubqueryContextForwarder(FMassExecutionContext& InContext, FEnvironment& InEnvironment, FQueryDescription& Description)
				: FMassSubqueryContextForwarderBase(InEnvironment, InContext)
				, Implementation(InContext, InEnvironment, Description.bShouldBatchModifications)
			{
				FHierarchyHandle Handle = InEnvironment.GetHierarchyRegistrar().FindHierarchyByName(Description.Hierarchy);
				Implementation.HierarchyAccessInterface = InEnvironment.GetHierarchyRegistrar().GetAccessInterface(Handle);
			}

			~FMassSubqueryContextForwarder() override = default;
			void SetHierarchyAccessInterface(const FTedsHierarchyAccessInterface* InHierarchyAccessInterface)
			{
				Implementation.HierarchyAccessInterface = InHierarchyAccessInterface;
			}
			uint32 GetRowCount() const override { return Implementation.GetRowCount(); }
			TConstArrayView<RowHandle> GetRowHandles() const override { return Implementation.GetRowHandles(); }
			const void* GetColumn(const UScriptStruct* ColumnType) const override { return Implementation.GetColumn(ColumnType); }
			void* GetMutableColumn(const UScriptStruct* ColumnType) override { return Implementation.GetMutableColumn(ColumnType); }
			void GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, TConstArrayView<EQueryAccessType> AccessTypes) override { return Implementation.GetColumns(RetrievedAddresses, ColumnTypes, AccessTypes); }
			void GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes, const EQueryAccessType* AccessTypes) override { return Implementation.GetColumnsUnguarded(TypeCount, RetrievedAddresses, ColumnTypes, AccessTypes); }
			bool HasColumn(const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(ColumnType); }
			bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(Row, ColumnType); }
			uint64 GetUpdateCycleId() const override { return Implementation.GetUpdateCycleId(); }
			bool IsRowAvailable(RowHandle Row) const override { return Implementation.IsRowAvailable(Row); }
			bool IsRowAssigned(RowHandle Row) const override { return Implementation.IsRowAssigned(Row); }
			void ActivateQueries(FName ActivationName) override  { return Implementation.ActivateQueries(ActivationName); }
			RowHandle AddRow(TableHandle Table) override  { return Implementation.AddRow(Table); }
			void RemoveRow(RowHandle Row) override { return Implementation.RemoveRow(Row); }
			void RemoveRows(TConstArrayView<RowHandle> Rows) override  { return Implementation.RemoveRows(Rows); }
			void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override { return Implementation.AddColumns(Row, ColumnTypes); }
			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override  { return Implementation.AddColumns(Rows, ColumnTypes); }
			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<UE::Editor::DataStorage::FDynamicColumnDescription> DynamicColumnDescriptions) override { Implementation.AddColumns(Rows, DynamicColumnDescriptions); }
			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ColumnType) override  { return Implementation.AddColumnUninitialized(Row, ColumnType); }
			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ObjectType, ObjectCopyOrMove Relocator) override { return Implementation.AddColumnUninitialized(Row, ObjectType, Relocator); }
			void* AddColumnUninitialized(RowHandle Row, const UE::Editor::DataStorage::FDynamicColumnDescription& DynamicColumnDescription) override { return Implementation.AddColumnUninitialized(Row, DynamicColumnDescription); }
			void* AddColumnUninitialized(RowHandle Row, const UE::Editor::DataStorage::FDynamicColumnDescription& DynamicColumnDescription, ObjectCopyOrMove Relocator) override { return Implementation.AddColumnUninitialized(Row, DynamicColumnDescription, Relocator); }
			void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override { Implementation.RemoveColumns(Row, ColumnTypes); }
			void RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override { Implementation.RemoveColumns(Rows, ColumnTypes); }
			float GetDeltaTimeSeconds() const override { return Implementation.GetDeltaTimeSeconds(); }
			virtual void SetParentRow(RowHandle Target, RowHandle Parent) override { Implementation.SetParent(*static_cast<ISubqueryContext*>(this), Target, Parent); }
			virtual void SetRelation(RelationTypeHandle Type, RowHandle Target, RowHandle NewObject) override
			{
				FSetRelationObjectCommand Command;
				Command.Type = Type;
				Command.Target = Target;
				Command.NewObject = NewObject;
				Command.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				ICommonQueryContext::PushCommand(MoveTemp(Command));
			}
			virtual void SetUnresolvedParent(RowHandle Target, FMapKey ParentId, FName MappingDomain) override { Implementation.SetUnresolvedParent(*static_cast<ISubqueryContext*>(this), Target, ParentId, MappingDomain); }
			virtual RowHandle GetParentRow(RowHandle Target) const override { return Implementation.GetParent(*static_cast<const ISubqueryContext*>(this), Target); }
			void PushCommand(void (*CommandFunction)(void*), void* CommandData) override { return Implementation.PushCommand(CommandFunction, CommandData); }
		protected:
			void* EmplaceObjectInScratch(const FEmplaceObjectParams& Params) override { return Implementation.EmplaceObjectInScratch(Params.ObjectSize, Params.Alignment, Params.Construct, Params.Destroy, Params.SourceObject); }
			
			FMassWithEnvironmentContextCommon Implementation;
		};

		struct FMassQueryContextImplementation final : FMassWithEnvironmentContextCommon
		{
			FMassQueryContextImplementation(
				FQueryDescription& InQueryDescription,
				FMassExecutionContext& InContext, 
				FExtendedQueryStore& InQueryStore,
				FEnvironment& InEnvironment)
				: FMassWithEnvironmentContextCommon(InContext, InEnvironment, InQueryDescription.bShouldBatchModifications)
				, QueryDescription(InQueryDescription)
				, QueryStore(InQueryStore)
			{
				FHierarchyHandle Handle = InEnvironment.GetHierarchyRegistrar().FindHierarchyByName(InQueryDescription.Hierarchy);
				HierarchyAccessInterface = InEnvironment.GetHierarchyRegistrar().GetAccessInterface(Handle);
			}

			~FMassQueryContextImplementation() = default;

			UObject* GetMutableDependency(const UClass* DependencyClass)
			{
				return Context.GetMutableSubsystem<USubsystem>(const_cast<UClass*>(DependencyClass));
			}

			const UObject* GetDependency(const UClass* DependencyClass)
			{
				return Context.GetSubsystem<USubsystem>(const_cast<UClass*>(DependencyClass));
			}
	
			void GetDependencies(TArrayView<UObject*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UClass>> SubsystemTypes,
				TConstArrayView<EQueryAccessType> AccessTypes)
			{
				checkf(RetrievedAddresses.Num() == SubsystemTypes.Num(), TEXT("Unable to retrieve a batch of subsystem as the number of addresses "
					"doesn't match the number of requested subsystem types."));

				GetDependenciesUnguarded(RetrievedAddresses.Num(), RetrievedAddresses.GetData(), SubsystemTypes.GetData(), AccessTypes.GetData());
			}

			void GetDependenciesUnguarded(int32 SubsystemCount, UObject** RetrievedAddresses, const TWeakObjectPtr<const UClass>* DependencyTypes,
				const EQueryAccessType* AccessTypes)
			{
				for (int32 Index = 0; Index < SubsystemCount; ++Index)
				{
					checkf(DependencyTypes->IsValid(), TEXT("Attempting to retrieve a subsystem that's no longer valid."));
					*RetrievedAddresses = *AccessTypes == EQueryAccessType::ReadWrite
						? Context.GetMutableSubsystem<USubsystem>(const_cast<UClass*>(DependencyTypes->Get()))
						: const_cast<USubsystem*>(Context.GetSubsystem<USubsystem>(const_cast<UClass*>(DependencyTypes->Get())));

					++RetrievedAddresses;
					++DependencyTypes;
					++AccessTypes;
				}
			}
	
			RowHandle LookupMappedRow(const FName& Domain, const FMapKeyView& Key) const
			{
				EGlobalLockScope Scope = FGlobalLock::GetLockStatus(EGlobalLockScope::Internal) == EGlobalLockStatus::Unlocked
					? EGlobalLockScope::Public // There's no internal lock so use a public lock instead.
					: EGlobalLockScope::Internal; // There's an internal lock set so use that.
				return Environment.GetMappingTable().Lookup(Scope, Domain, Key);
			}

			UE::Editor::DataStorage::FQueryResult RunQuery(QueryHandle Query)
			{
				const FExtendedQueryStore::Handle Handle(Query);
				// This can be safely called because there's not callback, which means no columns are accessed, even for select queries.
				return QueryStore.RunQuery(Context.GetEntityManagerChecked(), Handle);
			}

			UE::Editor::DataStorage::FQueryResult RunSubquery(int32 SubqueryIndex)
			{
				return SubqueryIndex < QueryDescription.Subqueries.Num() ?
					RunQuery(QueryDescription.Subqueries[SubqueryIndex]) :
					UE::Editor::DataStorage::FQueryResult{};
			}

			UE::Editor::DataStorage::FQueryResult RunSubquery(int32 SubqueryIndex, UE::Editor::DataStorage::SubqueryCallbackRef Callback)
			{
				if (SubqueryIndex < QueryDescription.Subqueries.Num())
				{
					const QueryHandle SubqueryHandle = QueryDescription.Subqueries[SubqueryIndex];
					const FExtendedQueryStore::Handle StorageHandle(SubqueryHandle);
					return QueryStore.RunQuery(Context.GetEntityManagerChecked(), Environment, Context, StorageHandle, Callback);
				}
				else
				{
					return UE::Editor::DataStorage::FQueryResult{};
				}
			}

			UE::Editor::DataStorage::FQueryResult RunSubquery(int32 SubqueryIndex, RowHandle Row,
				UE::Editor::DataStorage::SubqueryCallbackRef Callback)
			{
				if (SubqueryIndex < QueryDescription.Subqueries.Num())
				{
					const QueryHandle SubqueryHandle = QueryDescription.Subqueries[SubqueryIndex];
					const FExtendedQueryStore::Handle StorageHandle(SubqueryHandle);
					return QueryStore.RunQuery(Context.GetEntityManagerChecked(), Environment, Context, StorageHandle, Row, Callback);
				}
				else
				{
					return UE::Editor::DataStorage::FQueryResult{};
				}
			}
	
			FQueryDescription& QueryDescription;
			FExtendedQueryStore& QueryStore;
		};

		using FMassContextForwarderBase = Queries::Private::TImplementationForwarder<
			Queries::FProcessorContextEnvironment,
			IQueryContext,
			Queries::FTableInfoContextCapability,
			Queries::FDynamicColumnInfoContextCapability>;
		struct FMassContextForwarder final : FMassContextForwarderBase
		{
			FMassContextForwarder(
				FQueryDescription& InQueryDescription,
				FMassExecutionContext& InContext, 
				FExtendedQueryStore& InQueryStore,
				FEnvironment& InEnvironment)
					: FMassContextForwarderBase(InEnvironment, InContext)
					, Implementation(InQueryDescription, InContext, InQueryStore, InEnvironment)
			{
			}
	
			uint32 GetRowCount() const override { return Implementation.GetRowCount(); }
			TConstArrayView<RowHandle> GetRowHandles() const override { return Implementation.GetRowHandles(); }
			const void* GetColumn(const UScriptStruct* ColumnType) const override { return Implementation.GetColumn(ColumnType); }
			void* GetMutableColumn(const UScriptStruct* ColumnType) override { return Implementation.GetMutableColumn(ColumnType); }
			void GetColumns(TArrayView<char*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, TConstArrayView<EQueryAccessType> AccessTypes) override { return Implementation.GetColumns(RetrievedAddresses, ColumnTypes, AccessTypes); }
			void GetColumnsUnguarded(int32 TypeCount, char** RetrievedAddresses, const TWeakObjectPtr<const UScriptStruct>* ColumnTypes, const EQueryAccessType* AccessTypes) override { return Implementation.GetColumnsUnguarded(TypeCount, RetrievedAddresses, ColumnTypes, AccessTypes); }
			bool HasColumn(const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(ColumnType); }
			bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const override { return Implementation.HasColumn(Row, ColumnType); }
			uint64 GetUpdateCycleId() const override { return Implementation.GetUpdateCycleId(); }
			bool IsRowAvailable(RowHandle Row) const override { return Implementation.IsRowAvailable(Row); }
			bool IsRowAssigned(RowHandle Row) const override { return Implementation.IsRowAssigned(Row); }
			void ActivateQueries(FName ActivationName) override  { return Implementation.ActivateQueries(ActivationName); }
			RowHandle AddRow(TableHandle Table) override  { return Implementation.AddRow(Table); }
			void RemoveRow(RowHandle Row) override { return Implementation.RemoveRow(Row); }
			void RemoveRows(TConstArrayView<RowHandle> Rows) override  { return Implementation.RemoveRows(Rows); }
			void AddColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override { return Implementation.AddColumns(Row, ColumnTypes); }
			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override  { return Implementation.AddColumns(Rows, ColumnTypes); }
			void AddColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<UE::Editor::DataStorage::FDynamicColumnDescription> DynamicColumnDescriptions) override { return Implementation.AddColumns(Rows, DynamicColumnDescriptions); }
			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ColumnType) override  { return Implementation.AddColumnUninitialized(Row, ColumnType); }
			void* AddColumnUninitialized(RowHandle Row, const UScriptStruct* ObjectType, ObjectCopyOrMove Relocator) override { return Implementation.AddColumnUninitialized(Row, ObjectType, Relocator); }
			void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& DynamicColumnDescription, ObjectCopyOrMove Relocator) override { return Implementation.AddColumnUninitialized(Row, DynamicColumnDescription, Relocator); }
			void* AddColumnUninitialized(RowHandle Row, const FDynamicColumnDescription& DynamicColumnDescription) override { return Implementation.AddColumnUninitialized(Row, DynamicColumnDescription); };
			void RemoveColumns(RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) override { Implementation.RemoveColumns(Row, ColumnTypes); }
			void RemoveColumns(TConstArrayView<RowHandle> Rows, TConstArrayView<const UScriptStruct*> ColumnTypes) override { Implementation.RemoveColumns(Rows, ColumnTypes); }
			float GetDeltaTimeSeconds() const override { return Implementation.GetDeltaTimeSeconds(); }
			virtual void SetParentRow(RowHandle Target, RowHandle Parent) override { Implementation.SetParent(*static_cast<IQueryContext*>(this), Target, Parent); }
			virtual void SetRelation(RelationTypeHandle Type, RowHandle Target, RowHandle NewObject) override
			{
				FSetRelationObjectCommand Command;
				Command.Type = Type;
				Command.Target = Target;
				Command.NewObject = NewObject;
				Command.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				ICommonQueryContext::PushCommand(MoveTemp(Command));
			}
			virtual void SetUnresolvedParent(RowHandle Target, FMapKey ParentId, FName MappingDomain) override { Implementation.SetUnresolvedParent(*static_cast<IQueryContext*>(this), Target, ParentId, MappingDomain); }
			virtual RowHandle GetParentRow(RowHandle Target) const override { return Implementation.GetParent(*static_cast<const IQueryContext*>(this), Target); }
			void PushCommand(void (*CommandFunction)(void*), void* Context) override { return Implementation.PushCommand(CommandFunction, Context); }
			const UObject* GetDependency(const UClass* DependencyClass) override { return Implementation.GetDependency(DependencyClass); }
			UObject* GetMutableDependency(const UClass* DependencyClass) override { return Implementation.GetMutableDependency(DependencyClass); }
			void GetDependencies(TArrayView<UObject*> RetrievedAddresses, TConstArrayView<TWeakObjectPtr<const UClass>> DependencyTypes, TConstArrayView<EQueryAccessType> AccessTypes) override { return Implementation.GetDependencies(RetrievedAddresses, DependencyTypes, AccessTypes); }
			RowHandle LookupMappedRow(const FName& Domain, const FMapKeyView& Index) const override { return Implementation.LookupMappedRow(Domain, Index); }
			FQueryResult RunQuery(QueryHandle Query) override { return Implementation.RunQuery(Query); }
			FQueryResult RunSubquery(int32 SubqueryIndex) override { return Implementation.RunSubquery(SubqueryIndex); }
			FQueryResult RunSubquery(int32 SubqueryIndex, UE::Editor::DataStorage::SubqueryCallbackRef Callback) override { return Implementation.RunSubquery(SubqueryIndex, Callback); }
			FQueryResult RunSubquery(int32 SubqueryIndex, RowHandle Row, UE::Editor::DataStorage::SubqueryCallbackRef Callback) override { return Implementation.RunSubquery(SubqueryIndex, Row, Callback); }

			void SetSubRange(int32 InStart, int32 InLength) { Implementation.SetSubRange(InStart, InLength); }
			void ClearSubRange() { Implementation.ClearSubRange(); }

		protected:
			void* EmplaceObjectInScratch(const FEmplaceObjectParams& Params) override { return Implementation.EmplaceObjectInScratch(Params.ObjectSize, Params.Alignment, Params.Construct, Params.Destroy, Params.SourceObject); }

		protected:
			FMassQueryContextImplementation Implementation;
		};
	} // namespace Processors::Private

	/**
	 * FPhasePreOrPostAmbleExecutor
	 */
	FPhasePreOrPostAmbleExecutor::FPhasePreOrPostAmbleExecutor(FMassEntityManager& EntityManager, float DeltaTime)
		: Context(EntityManager, DeltaTime)
	{
		Context.SetDeferredCommandBuffer(MakeShared<FMassCommandBuffer>());
	}

	FPhasePreOrPostAmbleExecutor::~FPhasePreOrPostAmbleExecutor()
	{
		Context.FlushDeferred();
	}

	void FPhasePreOrPostAmbleExecutor::ExecuteQuery(
		FQueryDescription& Description,
		FExtendedQueryStore& QueryStore,
		FEnvironment& Environment,
		FMassEntityQuery& NativeQuery,
		QueryCallbackRef Callback)
	{
		if (Description.Callback.Active)
		{
			NativeQuery.ForEachEntityChunk(Context,
				[&Callback, &QueryStore, &Environment, &Description](FMassExecutionContext& ExecutionContext)
				{
					if (FTypedElementQueryProcessorData::PrepareCachedDependenciesOnQuery(Description, ExecutionContext))
					{
						Processors::Private::FMassContextForwarder QueryContext(Description, ExecutionContext, QueryStore, Environment);
						Callback(Description, QueryContext);
					}
				}
			);
		}
	}
} // namespace UE::Editor::DataStorage

/**
 * FTypedElementQueryProcessorData
 */

FTypedElementQueryProcessorData::FTypedElementQueryProcessorData(UMassProcessor& Owner)
	: NativeQuery(Owner)
{
}

bool FTypedElementQueryProcessorData::CommonQueryConfiguration(
	UMassProcessor& InOwner,
	UE::Editor::DataStorage::FExtendedQuery& InQuery,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle InQueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& InQueryStore,
	UE::Editor::DataStorage::FEnvironment& InEnvironment,
	TArrayView<FMassEntityQuery> Subqueries)
{
	using namespace UE::Editor::DataStorage;

	ParentQuery = InQueryHandle;
	QueryStore = &InQueryStore;
	Environment = &InEnvironment;

	if (ensureMsgf(InQuery.Description.Subqueries.Num() <= Subqueries.Num(),
		TEXT("Provided query has too many (%i) subqueries."), InQuery.Description.Subqueries.Num()))
	{
		bool Result = true;
		int32 CurrentSubqueryIndex = 0;
		for (QueryHandle SubqueryHandle : InQuery.Description.Subqueries)
		{
			const FExtendedQueryStore::Handle SubqueryStoreHandle(SubqueryHandle);
			if (const FExtendedQuery* Subquery = InQueryStore.Get(SubqueryStoreHandle))
			{
				if (ensureMsgf(Subquery->NativeQuery.CheckValidity(), TEXT("Provided subquery isn't valid. This can be because it couldn't be "
					"constructed properly or because it's been bound to a callback.")))
				{
					Subqueries[CurrentSubqueryIndex] = Subquery->NativeQuery;
					Subqueries[CurrentSubqueryIndex].RegisterWithProcessor(InOwner);
					++CurrentSubqueryIndex;
				}
				else
				{
					Result = false;
				}
			}
			else
			{
				Result = false;
			}
		}
		return Result;
	}
	return false;
}

EMassProcessingPhase FTypedElementQueryProcessorData::MapToMassProcessingPhase(UE::Editor::DataStorage::EQueryTickPhase Phase)
{
	using namespace UE::Editor::DataStorage;

	switch(Phase)
	{
	case EQueryTickPhase::PrePhysics:
		return EMassProcessingPhase::PrePhysics;
	case EQueryTickPhase::DuringPhysics:
		return EMassProcessingPhase::DuringPhysics;
	case EQueryTickPhase::PostPhysics:
		return EMassProcessingPhase::PostPhysics;
	case EQueryTickPhase::FrameEnd:
		return EMassProcessingPhase::FrameEnd;
	default:
		checkf(false, TEXT("Query tick phase '%i' is unsupported."), static_cast<int>(Phase));
		return EMassProcessingPhase::MAX;
	};
}

FString FTypedElementQueryProcessorData::GetProcessorName() const
{
	using namespace UE::Editor::DataStorage;

	if (const FExtendedQuery* StoredQuery = QueryStore ? QueryStore->Get(ParentQuery) : nullptr)
	{
		return StoredQuery->Description.Callback.Name.ToString();
	}
	else
	{
		return TEXT("<unnamed>");
	}
}

void FTypedElementQueryProcessorData::DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const
{
#if WITH_MASSENTITY_DEBUG
	using namespace UE::Editor::DataStorage;

	if (const FExtendedQuery* StoredQuery = QueryStore ? QueryStore->Get(ParentQuery) : nullptr)
	{
		const FQueryDescription& Description = StoredQuery->Description;
		const FQueryDescription::FCallbackData& Callback = Description.Callback;
		
		if (!Callback.Group.IsNone())
		{
			Ar.Logf(TEXT("\n%*sGroup: %s"), Indent, TEXT(""), *Callback.Group.ToString());
		}
		if (!Callback.BeforeGroups.IsEmpty())
		{
			Ar.Logf(TEXT("\n%*sBefore:"), Indent, TEXT(""));
			int32 Index = 0;
			for (FName BeforeName : Callback.BeforeGroups)
			{
				Ar.Logf(TEXT("\n%*s[%i] %s"), Indent + 4, TEXT(""), Index++, *BeforeName.ToString());
			}
		}
		if (!Callback.AfterGroups.IsEmpty())
		{
			Ar.Logf(TEXT("\n%*sAfter:"), Indent, TEXT(""));
			int32 Index = 0;
			for (FName AfterName : Callback.AfterGroups)
			{
				Ar.Logf(TEXT("\n%*s[%i] %s"), Indent + 4, TEXT(""), Index++, *AfterName.ToString());
			}
		}
		
		if (!Callback.ActivationName.IsNone())
		{
			Ar.Logf(TEXT("\n%*sActivatable: %s"), Indent, TEXT(""), *Callback.ActivationName.ToString());
		}

		if (Callback.MonitoredType)
		{
			Ar.Logf(TEXT("\n%*sMonitored type: %s"), Indent, TEXT(""), *Callback.MonitoredType->GetName());
		}

		switch (Callback.ExecutionMode)
		{
		case EExecutionMode::Default:
			Ar.Logf(TEXT("\n%*sExecution mode: Default"), Indent, TEXT(""));
			break;
		case EExecutionMode::GameThread:
			Ar.Logf(TEXT("\n%*sExecution mode: Game Thread"), Indent, TEXT(""));
			break;
		case EExecutionMode::Threaded:
			Ar.Logf(TEXT("\n%*sExecution mode: Threaded"), Indent, TEXT(""));
			break;
		case EExecutionMode::ThreadedChunks:
			Ar.Logf(TEXT("\n%*sExecution mode: Threaded Chunks"), Indent, TEXT(""));
			break;
		default:
			Ar.Logf(TEXT("\n%*sExecution mode: <Unknown option>"), Indent, TEXT(""));
			break;
		}
	}
#endif // WITH_MASSENTITY_DEBUG
}

bool FTypedElementQueryProcessorData::PrepareCachedDependenciesOnQuery(
	UE::Editor::DataStorage::FQueryDescription& Description, FMassExecutionContext& Context)
{
	using namespace UE::Editor::DataStorage;

	const int32 DependencyCount = Description.DependencyTypes.Num();
	const TWeakObjectPtr<const UClass>* Types = Description.DependencyTypes.GetData();
	const EQueryDependencyFlags* Flags = Description.DependencyFlags.GetData();
	TWeakObjectPtr<UObject>* Caches = Description.CachedDependencies.GetData();

	for (int32 Index = 0; Index < DependencyCount; ++Index)
	{
		checkf(Types->IsValid(), TEXT("Attempting to retrieve a dependency type that's no longer available."));
		
		if (EnumHasAnyFlags(*Flags, EQueryDependencyFlags::AlwaysRefresh) || !Caches->IsValid())
		{
			*Caches = EnumHasAnyFlags(*Flags, EQueryDependencyFlags::ReadOnly)
				? const_cast<USubsystem*>(Context.GetSubsystem<USubsystem>(const_cast<UClass*>(Types->Get())))
				: Context.GetMutableSubsystem<USubsystem>(const_cast<UClass*>(Types->Get()));
			if (*Caches != nullptr)
			{
				++Types;
				++Flags;
				++Caches;
			}
			else
			{
				checkf(false, TEXT("Unable to retrieve instance of dependency '%s'."), *((*Types)->GetName()));
				return false;
			}
		}
	}
	return true;
}

UE::Editor::DataStorage::FQueryResult FTypedElementQueryProcessorData::Execute(
	UE::Editor::DataStorage::DirectQueryCallbackRef& Callback,
	UE::Editor::DataStorage::FQueryDescription& Description,
	FMassEntityQuery& NativeQuery,
	FMassEntityManager& EntityManager,
	UE::Editor::DataStorage::FEnvironment& Environment,
	UE::Editor::DataStorage::EDirectQueryExecutionFlags ExecutionFlags)
{
	using namespace UE::Editor::DataStorage;

	FQueryResult Result;
	Result.Completed = FQueryResult::ECompletion::Fully;

	if (EnumHasAnyFlags(ExecutionFlags, EDirectQueryExecutionFlags::AllowBoundQueries) || !Description.Callback.Function)
	{
		const bool bHasRelationConditions = UE::Editor::DataStorage::Private::HasRelationConditions(Description);
		const FTedsRelationAdapter* RelationAdapter = bHasRelationConditions ? &Environment.GetRelationAdapter() : nullptr;
		const FTedsHierarchicalRelationManager* HierarchyManager = bHasRelationConditions ? &Environment.GetHierarchicalRelationManager() : nullptr;

		auto ExecuteFunction = [&Result, &Callback, &Description, &Environment, &EntityManager,
			bHasRelationConditions, RelationAdapter, HierarchyManager](FMassExecutionContext& Context)
			{
				using namespace Processors::Private;
				FMassDirectContextForwarder QueryContext(Context, Environment, Description);

				if (bHasRelationConditions)
				{
					UE::Editor::DataStorage::Private::ForEachRelationFilteredSubrange(
						Context, Description, EntityManager, *RelationAdapter, HierarchyManager,
						[&](int32 SubStart, int32 SubLength)
						{
							QueryContext.Implementation.SetSubRange(SubStart, SubLength);
							Callback(Description, QueryContext);
							Result.Count += SubLength;
						});
					QueryContext.Implementation.ClearSubRange();
				}
				else
				{
					Callback(Description, QueryContext);
					Result.Count += Context.GetNumEntities();
				}
			};

		FMassExecutionContext Context(EntityManager);
		Context.SetDeferredCommandBuffer(Environment.GetSharedMassCommandBuffer());
		Context.SetFlushDeferredCommands(false);

		if (EnumHasAnyFlags(ExecutionFlags, EDirectQueryExecutionFlags::ParallelizeChunks))
		{
			const FMassEntityQuery::EParallelExecutionFlags Mode =
				EnumHasAnyFlags(ExecutionFlags, EDirectQueryExecutionFlags::AutoBalanceParallelChunkProcessing)
				? FMassEntityQuery::EParallelExecutionFlags::AutoBalance
				: FMassEntityQuery::EParallelExecutionFlags::Default;
			NativeQuery.ParallelForEachEntityChunk(Context, ExecuteFunction, Mode);
		}
		else
		{
			NativeQuery.ForEachEntityChunk(Context, ExecuteFunction);
		}
	}
	else
	{
		Result.Completed = FQueryResult::ECompletion::Unsupported;
	}
	return Result;
}

UE::Editor::DataStorage::FQueryResult FTypedElementQueryProcessorData::Execute(
	UE::Editor::DataStorage::Queries::TQueryFunction<void>& Callback,
	UE::Editor::DataStorage::FQueryDescription& Description,
	FMassEntityQuery& NativeQuery,
	FMassEntityManager& EntityManager,
	UE::Editor::DataStorage::FEnvironment& Environment,
	UE::Editor::DataStorage::ERunQueryFlags ExecutionFlags)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	FQueryResult Result;
	Result.Completed = FQueryResult::ECompletion::Fully;

	// Check compatibility here once instead of on the call itself to avoid doing the same thing for every chunk.
	if (!QueryContext_ProcessorApi::SupportsCapabilities(Callback.CapabilityMask))
	{
		QueryContext_ProcessorApi::StaticReportCompatibility(Callback.CapabilityMask);
		FQueryResult UnsupportedResult;
		UnsupportedResult.Completed = FQueryResult::ECompletion::Unsupported;
		return UnsupportedResult;
	}

	if (EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::AllowBoundQueries) || !Description.Callback.Function)
	{
		FMassExecutionContext Context(EntityManager);
		Context.SetDeferredCommandBuffer(Environment.GetSharedMassCommandBuffer());
		Context.SetFlushDeferredCommands(false);
		auto ExecuteFunction = [&Result, &Callback, &Environment](FMassExecutionContext& Context)
			{
				using namespace Processors::Private;
				FProcessorContextEnvironment MassContextEnvironment(Environment, Context);
				QueryContext_ProcessorApi QueryContext(MassContextEnvironment);
				// This assumes that there already has been a check that the provided function is compatible with the query.
				Callback.Call(QueryContext, MassContextEnvironment);
				Result.Count += Context.GetNumEntities();
			};
		if (EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::ParallelizeChunks))
		{
			FMassEntityQuery::EParallelExecutionFlags Mode =
				EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::AutoBalanceParallelChunkProcessing)
				? FMassEntityQuery::EParallelExecutionFlags::AutoBalance
				: FMassEntityQuery::EParallelExecutionFlags::Default;
			NativeQuery.ParallelForEachEntityChunk(Context, ExecuteFunction, Mode);
		}
		else
		{
			NativeQuery.ForEachEntityChunk(Context, ExecuteFunction);
		}
	}
	else
	{
		Result.Completed = FQueryResult::ECompletion::Unsupported;
	}
	return Result;
}

UE::Editor::DataStorage::FQueryResult FTypedElementQueryProcessorData::Execute(
	const UE::Editor::DataStorage::Queries::TConstQueryFunction<void>& Callback,
	const UE::Editor::DataStorage::FQueryDescription& Description,
	FMassEntityQuery& NativeQuery,
	FMassEntityManager& EntityManager,
	UE::Editor::DataStorage::FEnvironment& Environment,
	UE::Editor::DataStorage::ERunQueryFlags ExecutionFlags)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	// Check compatibility here once instead of on the call itself to avoid doing the same thing for every chunk.
	if (!QueryContext_ProcessorApi::SupportsCapabilities(Callback.CapabilityMask))
	{
		QueryContext_ProcessorApi::StaticReportCompatibility(Callback.CapabilityMask);
		FQueryResult UnsupportedResult;
		UnsupportedResult.Completed = FQueryResult::ECompletion::Unsupported;
		return UnsupportedResult;
	}

	FQueryResult Result;
	Result.Completed = FQueryResult::ECompletion::Fully;

	if (EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::AllowBoundQueries) || !Description.Callback.Function)
	{
		const bool bHasRelationConditions = UE::Editor::DataStorage::Private::HasRelationConditions(Description);
		const FTedsRelationAdapter* RelationAdapter = bHasRelationConditions ? &Environment.GetRelationAdapter() : nullptr;
		const FTedsHierarchicalRelationManager* HierarchyManager = bHasRelationConditions ? &Environment.GetHierarchicalRelationManager() : nullptr;

		auto ExecuteFunction = [&Result, &Callback, &Environment, &Description, &EntityManager,
			bHasRelationConditions, RelationAdapter, HierarchyManager](FMassExecutionContext& Context)
			{
				using namespace Processors::Private;
				FConstProcessorContextEnvironment MassContextEnvironment(Environment, Context);
				QueryContext_ConstProcessorApi QueryContext(MassContextEnvironment);

				if (bHasRelationConditions)
				{
					UE::Editor::DataStorage::Private::ForEachRelationFilteredSubrange(
						Context, Description, EntityManager, *RelationAdapter, HierarchyManager,
						[&](int32 SubStart, int32 SubLength)
						{
							MassContextEnvironment.SetSubRange(SubStart, SubLength);
							Callback.Call(QueryContext, MassContextEnvironment);
							Result.Count += SubLength;
						});
					MassContextEnvironment.ClearSubRange();
				}
				else
				{
					Callback.Call(QueryContext, MassContextEnvironment);
					Result.Count += Context.GetNumEntities();
				}
			};

		FMassExecutionContext Context(EntityManager);
		Context.SetDeferredCommandBuffer(Environment.GetSharedMassCommandBuffer());
		Context.SetFlushDeferredCommands(false);

		if (EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::ParallelizeChunks))
		{
			const FMassEntityQuery::EParallelExecutionFlags Mode =
				EnumHasAnyFlags(ExecutionFlags, ERunQueryFlags::AutoBalanceParallelChunkProcessing)
				? FMassEntityQuery::EParallelExecutionFlags::AutoBalance
				: FMassEntityQuery::EParallelExecutionFlags::Default;
			NativeQuery.ParallelForEachEntityChunk(Context, ExecuteFunction, Mode);
		}
		else
		{
			NativeQuery.ForEachEntityChunk(Context, ExecuteFunction);
		}
	}
	else
	{
		Result.Completed = FQueryResult::ECompletion::Unsupported;
	}
	return Result;
}

UE::Editor::DataStorage::FQueryResult FTypedElementQueryProcessorData::Execute(
	UE::Editor::DataStorage::SubqueryCallbackRef& Callback,
	UE::Editor::DataStorage::FQueryDescription& Description,
	FMassEntityQuery& NativeQuery,
	FMassEntityManager& EntityManager,
	UE::Editor::DataStorage::FEnvironment& Environment,
	FMassExecutionContext& ParentContext)
{
	using namespace UE::Editor::DataStorage;

	FQueryResult Result;
	Result.Completed = FQueryResult::ECompletion::Fully;

	checkf(Description.Callback.ExecutionMode != EExecutionMode::ThreadedChunks,
		TEXT("TEDS Sub-queries do not support parallel chunk processing."));

	FMassExecutionContext Context(EntityManager);
	Context.SetDeferredCommandBuffer(ParentContext.GetSharedDeferredCommandBuffer());
	Context.SetFlushDeferredCommands(false);

	NativeQuery.ForEachEntityChunk(Context,
		[&Result, &Callback, &Description, &Environment](FMassExecutionContext& Context)
		{
			using namespace Processors::Private;
			// No need to cache any subsystem dependencies as these are not accessible from a subquery.
			FMassSubqueryContextForwarder QueryContext(Context, Environment, Description);
			Callback(Description, QueryContext);
			Result.Count += Context.GetNumEntities();
		}
	);

	return Result;
}

UE::Editor::DataStorage::FQueryResult FTypedElementQueryProcessorData::Execute(
	UE::Editor::DataStorage::SubqueryCallbackRef& Callback,
	UE::Editor::DataStorage::FQueryDescription& Description,
	UE::Editor::DataStorage::RowHandle RowHandle,
	FMassEntityQuery& NativeQuery,
	FMassEntityManager& EntityManager,
	UE::Editor::DataStorage::FEnvironment& Environment,
	FMassExecutionContext& ParentContext)
{
	using namespace UE::Editor::DataStorage;

	FQueryResult Result;
	Result.Completed = FQueryResult::ECompletion::Fully;

	FMassEntityHandle NativeEntity = FMassEntityHandle::FromNumber(RowHandle);
	if (EntityManager.IsEntityActive(NativeEntity))
	{
		checkf(Description.Callback.ExecutionMode != EExecutionMode::ThreadedChunks,
			TEXT("TEDS Sub-queries do not support parallel chunk processing."));
		
		FMassArchetypeHandle NativeArchetype = EntityManager.GetArchetypeForEntityUnsafe(NativeEntity);
		FMassExecutionContext Context(EntityManager);
		Context.SetEntityCollection(FMassArchetypeEntityCollection(NativeArchetype, { NativeEntity }, FMassArchetypeEntityCollection::NoDuplicates));
		Context.SetDeferredCommandBuffer(ParentContext.GetSharedDeferredCommandBuffer());
		Context.SetFlushDeferredCommands(false);
		
		const FTedsHierarchyAccessInterface* HierarchyAccessInterface = nullptr;
		if (!Description.Hierarchy.IsNone())
		{
			FHierarchyHandle Handle = Environment.GetHierarchyRegistrar().FindHierarchyByName(Description.Hierarchy);
			HierarchyAccessInterface = Environment.GetHierarchyRegistrar().GetAccessInterface(Handle);
		}

		NativeQuery.ForEachEntityChunk(Context,
			[&Result, &Callback, &Description, &Environment,  HierarchyAccessInterface](FMassExecutionContext& Context)
			{
				using namespace Processors::Private;
				// No need to cache any subsystem dependencies as these are not accessible from a subquery.
				FMassSubqueryContextForwarder QueryContext(Context, Environment, Description);
				QueryContext.SetHierarchyAccessInterface(HierarchyAccessInterface);
				
				Callback(Description, QueryContext);
				Result.Count += Context.GetNumEntities();
			}
		);
		checkf(Result.Count < 2, TEXT("Single row subquery produced multiple results."));
	}
	return Result;
}

void FTypedElementQueryProcessorData::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	using namespace UE::Editor::DataStorage;

	FExtendedQuery* StoredQuery = QueryStore->GetMutable(ParentQuery);

	checkf(StoredQuery, TEXT("A query callback was registered for execution without an associated query. Processor: [%s]"), *GetProcessorName());

	FQueryDescription& Description = StoredQuery->Description;

	const bool bHasRelationConditions = UE::Editor::DataStorage::Private::HasRelationConditions(Description);
	const FTedsRelationAdapter* RelationAdapter = bHasRelationConditions ? &Environment->GetRelationAdapter() : nullptr;
	const FTedsHierarchicalRelationManager* HierarchyManager = bHasRelationConditions ? &Environment->GetHierarchicalRelationManager() : nullptr;

	auto ExecuteFunction = [this, &Description, &EntityManager,
		bHasRelationConditions, RelationAdapter, HierarchyManager](FMassExecutionContext& Context)
		{
			if (PrepareCachedDependenciesOnQuery(Description, Context))
			{
				Processors::Private::FMassContextForwarder QueryContext(Description, Context, *QueryStore, *Environment);

				if (bHasRelationConditions)
				{
					UE::Editor::DataStorage::Private::ForEachRelationFilteredSubrange(
						Context, Description, EntityManager, *RelationAdapter, HierarchyManager,
						[&](int32 SubStart, int32 SubLength)
						{
							QueryContext.SetSubRange(SubStart, SubLength);
							Description.Callback.Function(Description, QueryContext);
						});
					QueryContext.ClearSubRange();
				}
				else
				{
					Description.Callback.Function(Description, QueryContext);
				}
			}
		};

	if (StoredQuery->Description.Callback.ExecutionMode == EExecutionMode::ThreadedChunks)
	{
		NativeQuery.ParallelForEachEntityChunk(Context, ExecuteFunction);
	}
	else
	{
		NativeQuery.ForEachEntityChunk(Context, ExecuteFunction);
	}
}



/**
 * UTypedElementQueryProcessorCallbackAdapterProcessor
 */

UTypedElementQueryProcessorCallbackAdapterProcessorBase::UTypedElementQueryProcessorCallbackAdapterProcessorBase()
	: Data(*this)
{
	bAllowMultipleInstances = true;
	bAutoRegisterWithProcessingPhases = false;
}

FMassEntityQuery& UTypedElementQueryProcessorCallbackAdapterProcessorBase::GetQuery()
{
	return Data.NativeQuery;
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorBase::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, {});
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorBase::ShouldAllowQueryBasedPruning(
	const bool bRuntimeMode) const
{
	// TEDS is much more dynamic with when tables and processors are added and removed
	// Don't prune processors if they have queries where no table is defined, it is possible
	// the table will be dynamically created later.
	return false;
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorBase::ConfigureQueryCallbackData(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment,
	TArrayView<FMassEntityQuery> Subqueries)
{
	using namespace UE::Editor::DataStorage;

	bool Result = Data.CommonQueryConfiguration(*this, Query, QueryHandle, QueryStore, Environment, Subqueries);

	bRequiresGameThreadExecution = Query.Description.Callback.ExecutionMode == EExecutionMode::GameThread;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Editor); 
	ExecutionOrder.ExecuteInGroup = Query.Description.Callback.Group;
	ExecutionOrder.ExecuteBefore = Query.Description.Callback.BeforeGroups;
	ExecutionOrder.ExecuteAfter = Query.Description.Callback.AfterGroups;
	ProcessingPhase = Data.MapToMassProcessingPhase(Query.Description.Callback.Phase);

	Super::PostInitProperties();
	return Result;
}

void UTypedElementQueryProcessorCallbackAdapterProcessorBase::ConfigureQueries(const TSharedRef<FMassEntityManager>&)
{
	// When the extended query information is provided the native query will already be fully configured.
}

void UTypedElementQueryProcessorCallbackAdapterProcessorBase::PostInitProperties()
{
	Super::Super::PostInitProperties();
}

FString UTypedElementQueryProcessorCallbackAdapterProcessorBase::GetProcessorName() const
{
	return Data.GetProcessorName();
}

void UTypedElementQueryProcessorCallbackAdapterProcessorBase::DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const
{
#if WITH_MASSENTITY_DEBUG
	UMassProcessor::DebugOutputDescription(Ar, Indent);
	Ar.Logf(TEXT("\n%*sType: Editor Processor"), Indent, TEXT(""));
	Data.DebugOutputDescription(Ar, Indent);
#endif // WITH_MASSENTITY_DEBUG
}

void UTypedElementQueryProcessorCallbackAdapterProcessorBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	Data.Execute(EntityManager, Context);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith1Subquery::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith2Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith3Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith4Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith5Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith6Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith7Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryProcessorCallbackAdapterProcessorWith8Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}


/**
 * UTypedElementQueryObserverCallbackAdapterProcessor
 */

UTypedElementQueryObserverCallbackAdapterProcessorBase::UTypedElementQueryObserverCallbackAdapterProcessorBase()
	: Data(*this)
{
	bAllowMultipleInstances = true;
	bAutoRegisterWithProcessingPhases = false;
}

FMassEntityQuery& UTypedElementQueryObserverCallbackAdapterProcessorBase::GetQuery()
{
	return Data.NativeQuery;
}

const UScriptStruct* UTypedElementQueryObserverCallbackAdapterProcessorBase::GetObservedType() const
{
	return GetSingleObservedType();
}

EMassObservedOperationFlags UTypedElementQueryObserverCallbackAdapterProcessorBase::GetObservedOperations() const
{
	return static_cast<EMassObservedOperationFlags>(ObservedOperations);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorBase::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, {});
}

bool UTypedElementQueryObserverCallbackAdapterProcessorBase::ConfigureQueryCallbackData(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment, TArrayView<FMassEntityQuery> Subqueries)
{
	using namespace UE::Editor::DataStorage;

	bool Result = Data.CommonQueryConfiguration(*this, Query, QueryHandle, QueryStore, Environment, Subqueries);

	bRequiresGameThreadExecution = Query.Description.Callback.ExecutionMode == EExecutionMode::GameThread;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Editor);

	ObservedTypes.Reset();
	ObservedTypes.Add(Query.Description.Callback.MonitoredType);
	
	switch (Query.Description.Callback.Type)
	{
	case EQueryCallbackType::ObserveAdd:
		ObservedOperations = EMassObservedOperationFlags::Add;
		break;
	case EQueryCallbackType::ObserveRemove:
		ObservedOperations = EMassObservedOperationFlags::Remove;
		break;
	default:
		checkf(false, TEXT("Query type %i is not supported from the observer processor adapter."),
			static_cast<int>(Query.Description.Callback.Type));
		return false;
	}

	Super::PostInitProperties();
	return Result;
}

void UTypedElementQueryObserverCallbackAdapterProcessorBase::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// When the extended query information is provided the native query will already be fully configured.
}

void UTypedElementQueryObserverCallbackAdapterProcessorBase::PostInitProperties()
{
	Super::Super::PostInitProperties();
}

void UTypedElementQueryObserverCallbackAdapterProcessorBase::Register()
{ 
	// Do nothing as this processor will be explicitly registered.
}

FString UTypedElementQueryObserverCallbackAdapterProcessorBase::GetProcessorName() const
{
	return Data.GetProcessorName();
}

void UTypedElementQueryObserverCallbackAdapterProcessorBase::DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const
{
#if WITH_MASSENTITY_DEBUG
	UMassObserverProcessor::DebugOutputDescription(Ar, Indent);
	const uint8 ObservationTypes = static_cast<uint8>(GetObservedOperations());

	if (ObservationTypes)
	{
		for (uint8 OperationIndex = 0; OperationIndex < static_cast<uint8>(EMassObservedOperation::MAX); ++OperationIndex)
		{
			if ((ObservationTypes & (1 << OperationIndex)))
			{
				Ar.Logf(TEXT("\n%*sType: Editor %s Observer"), Indent, TEXT("")
					, *UEnum::GetValueAsString(static_cast<EMassObservedOperation>(OperationIndex)));
			}
		}
	}	
	else
	{
		Ar.Logf(TEXT("\n%*sType: Editor <Unknown> Observer"), Indent, TEXT(""));
	}
	Data.DebugOutputDescription(Ar, Indent);
#endif // WITH_MASSENTITY_DEBUG
}


void UTypedElementQueryObserverCallbackAdapterProcessorBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	Data.Execute(EntityManager, Context);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith1Subquery::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith2Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith3Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith4Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith5Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith6Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith7Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}

bool UTypedElementQueryObserverCallbackAdapterProcessorWith8Subqueries::ConfigureQueryCallback(
	UE::Editor::DataStorage::FExtendedQuery& Query,
	UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
	UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
	UE::Editor::DataStorage::FEnvironment& Environment)
{
	return ConfigureQueryCallbackData(Query, QueryHandle, QueryStore, Environment, NativeSubqueries);
}
