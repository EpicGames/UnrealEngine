// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "MassExecutionContext.h"
#include "MassObserverProcessor.h"
#include "MassProcessor.h"
#include "Queries/TypedElementExtendedQueryStore.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementProcessorAdaptors.generated.h"

namespace UE::Editor::DataStorage
{
	class FEnvironment;
	struct FExtendedQuery;
	class FExtendedQueryStore;

	struct FPhasePreOrPostAmbleExecutor
	{
		FPhasePreOrPostAmbleExecutor(FMassEntityManager& EntityManager, float DeltaTime);
		~FPhasePreOrPostAmbleExecutor();

		void ExecuteQuery(
			FQueryDescription& Description,
			FExtendedQueryStore& QueryStore,
			FEnvironment& Environment,
			FMassEntityQuery& NativeQuery,
			QueryCallbackRef Callback);

		FMassExecutionContext Context;
	};
}

USTRUCT()
struct FTypedElementQueryProcessorData
{
	GENERATED_BODY();

	FTypedElementQueryProcessorData() = default;
	explicit FTypedElementQueryProcessorData(UMassProcessor& Owner);

	bool CommonQueryConfiguration(
		UMassProcessor& InOwner,
		UE::Editor::DataStorage::FExtendedQuery& InQuery,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle InQueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& InQueryStore,
		UE::Editor::DataStorage::FEnvironment& InEnvironment,
		TArrayView<FMassEntityQuery> Subqueries);
	static EMassProcessingPhase MapToMassProcessingPhase(UE::Editor::DataStorage::EQueryTickPhase Phase);
	FString GetProcessorName() const;
	void DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const;

	static UE::Editor::DataStorage::FQueryResult Execute(
		UE::Editor::DataStorage::DirectQueryCallbackRef& Callback,
		UE::Editor::DataStorage::FQueryDescription& Description,
		FMassEntityQuery& NativeQuery, 
		FMassEntityManager& EntityManager,
		UE::Editor::DataStorage::FEnvironment& Environment,
		UE::Editor::DataStorage::EDirectQueryExecutionFlags ExecutionFlags);
	static UE::Editor::DataStorage::FQueryResult Execute(
		UE::Editor::DataStorage::Queries::TQueryFunction<void>& Callback,
		UE::Editor::DataStorage::FQueryDescription& Description,
		FMassEntityQuery& NativeQuery,
		FMassEntityManager& EntityManager,
		UE::Editor::DataStorage::FEnvironment& Environment,
		UE::Editor::DataStorage::ERunQueryFlags ExecutionFlags);
	static UE::Editor::DataStorage::FQueryResult Execute(
		const UE::Editor::DataStorage::Queries::TConstQueryFunction<void>& Callback,
		const UE::Editor::DataStorage::FQueryDescription& Description,
		FMassEntityQuery& NativeQuery,
		FMassEntityManager& EntityManager,
		UE::Editor::DataStorage::FEnvironment& Environment,
		UE::Editor::DataStorage::ERunQueryFlags ExecutionFlags);
	static UE::Editor::DataStorage::FQueryResult Execute(
		UE::Editor::DataStorage::SubqueryCallbackRef& Callback,
		UE::Editor::DataStorage::FQueryDescription& Description,
		FMassEntityQuery& NativeQuery,
		FMassEntityManager& EntityManager,
		UE::Editor::DataStorage::FEnvironment& Environment,
		FMassExecutionContext& ParentContext);
	static UE::Editor::DataStorage::FQueryResult Execute(
		UE::Editor::DataStorage::SubqueryCallbackRef& Callback,
		UE::Editor::DataStorage::FQueryDescription& Description,
		UE::Editor::DataStorage::RowHandle RowHandle,
		FMassEntityQuery& NativeQuery,
		FMassEntityManager& EntityManager,
		UE::Editor::DataStorage::FEnvironment& Environment,
		FMassExecutionContext& ParentContext);
	void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context);

	static bool PrepareCachedDependenciesOnQuery(
		UE::Editor::DataStorage::FQueryDescription& Description, FMassExecutionContext& Context);

	UE::Editor::DataStorage::FExtendedQueryStore::Handle ParentQuery;
	UE::Editor::DataStorage::FExtendedQueryStore* QueryStore{ nullptr };
	UE::Editor::DataStorage::FEnvironment* Environment{ nullptr };
	FMassEntityQuery NativeQuery;
};

/**
 * Adapts processor queries callback for MASS.
 */
UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorBase : public UMassProcessor
{
	GENERATED_BODY()

public:
	UTypedElementQueryProcessorCallbackAdapterProcessorBase();

	FMassEntityQuery& GetQuery();
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment);

	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode) const override;

protected:
	bool ConfigureQueryCallbackData(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment,
		TArrayView<FMassEntityQuery> Subqueries);
	void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override;
	void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& TargetParentQuery) override;
	
	void PostInitProperties() override;
	FString GetProcessorName() const override;
	void DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const override;

private:
	UPROPERTY(transient)
	FTypedElementQueryProcessorData Data;
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessor final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()
};

/**
 * Mass verifies that queries that are used by processors are on the processor themselves. It does this by taking the address of the query 
 * and seeing if it's within the start and end address of the processor. When a dynamic array is used those addresses are going to be 
 * elsewhere, so the two options are to store a single fixed size array on a processor or have multiple instances. With Mass' queries being 
 * not an insignificant size it's preferable to have several variants with queries to allow the choice for the minimal size. Unfortunately 
 * UHT doesn't allow for templates so it had to be done in an explicit way.
 */

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith1Subquery final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[1];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith2Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[2];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith3Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[3];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith4Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[4];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith5Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[5];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith6Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[6];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith7Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[7];
};

UCLASS()
class UTypedElementQueryProcessorCallbackAdapterProcessorWith8Subqueries final : public UTypedElementQueryProcessorCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[8];
};

/**
 * Adapts observer queries callback for MASS.
 */
UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorBase : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UTypedElementQueryObserverCallbackAdapterProcessorBase();

	FMassEntityQuery& GetQuery();
	const UScriptStruct* GetObservedType() const;
	EMassObservedOperationFlags GetObservedOperations() const;
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment);

protected:
	bool ConfigureQueryCallbackData(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment,
		TArrayView<FMassEntityQuery> Subqueries);
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& TargetParentQuery) override;

	virtual void PostInitProperties() override;
	virtual void Register() override;
	virtual FString GetProcessorName() const override;
	virtual void DebugOutputDescription(FOutputDevice& Ar, int32 Indent) const override;

private:
	FTypedElementQueryProcessorData Data;
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessor final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()
};

/**
 * Mass verifies that queries that are used by processors are on the processor themselves. It does this by taking the address of the query
 * and seeing if it's within the start and end address of the processor. When a dynamic array is used those addresses are going to be
 * elsewhere, so the two options are to store a single fixed size array on a processor or have multiple instances. With Mass' queries being
 * not an insignificant size it's preferable to have several variants with queries to allow the choice for the minimal size. Unfortunately
 * UHT doesn't allow for templates so it had to be done in an explicit way.
 */

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith1Subquery final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[1];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith2Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[2];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith3Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[3];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith4Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[4];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith5Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[5];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith6Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[6];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith7Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[7];
};

UCLASS()
class UTypedElementQueryObserverCallbackAdapterProcessorWith8Subqueries final : public UTypedElementQueryObserverCallbackAdapterProcessorBase
{
	GENERATED_BODY()

public:
	virtual bool ConfigureQueryCallback(
		UE::Editor::DataStorage::FExtendedQuery& Query,
		UE::Editor::DataStorage::FExtendedQueryStore::Handle QueryHandle,
		UE::Editor::DataStorage::FExtendedQueryStore& QueryStore,
		UE::Editor::DataStorage::FEnvironment& Environment) override;

private:
	FMassEntityQuery NativeSubqueries[8];
};
