// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySearchers/DefaultPropertySearcherFactory.h"

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "PropertySearchers/StringSearchers.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultPropertySearcherFactory)

void UDefaultPropertySearcherFactory::RegisterPropertySearchers(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Searchers;

	auto RegisterCallback = [&DataStorageUi]<typename Searcher, typename PropertyType>()
	{
		DataStorageUi.RegisterSearcherGeneratorForProperty(PropertyType::StaticClass(),
			[](TWeakObjectPtr<const UScriptStruct> ColumnType, const FProperty& Property)
			{
				return MakeShared<Searcher>(ColumnType, CastField<PropertyType>(&Property));
			});
	};
	
	// Strings
	RegisterCallback.operator()<FStringSearcher, FStrProperty>();
	RegisterCallback.operator()<FTextSearcher, FTextProperty>();
	RegisterCallback.operator()<FNameSearcher, FNameProperty>();
}
