// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "DefaultPropertySearcherFactory.generated.h"

/**
 * Factory used to register searcher for text properties.
 */
UCLASS(Transient)
class UDefaultPropertySearcherFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterPropertySearchers(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};
