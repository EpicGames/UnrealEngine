// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Searchers
{
	template<typename SearchableType, typename PropertyType>
	class TPropertySearcherBase : public FColumnSearcherInterface
	{
	public:
		TPropertySearcherBase(TWeakObjectPtr<const UScriptStruct> ColumnType, const PropertyType* Property);

		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const override;
		
	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const SearchableType& Value) const = 0;

	private:
		const SearchableType* GetValue(const ICoreProvider& Storage, const UScriptStruct* Column, RowHandle Row) const;

		TWeakObjectPtr<const UScriptStruct> ColumnType;
		const PropertyType* Property;
	};
} // namespace UE::Editor::DataStorage::Searchers

#include "PropertySearchers/PropertySearcherBase.inl"