// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::Searchers
{
	template<typename SearchableType, typename PropertyType>
	TPropertySearcherBase<SearchableType, PropertyType>::TPropertySearcherBase(
		TWeakObjectPtr<const UScriptStruct> ColumnType, const PropertyType* Property)
		: ColumnType(MoveTemp(ColumnType))
		, Property(Property)
	{
	}

	template<typename SearchableType, typename PropertyType>
	void TPropertySearcherBase<SearchableType, PropertyType>::GetSearchableString(
		FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const
	{
		if (const UScriptStruct* Column = ColumnType.Get())
		{
			if (const SearchableType* Value = GetValue(Storage, Column, Row))
			{
				GetSearchableString(SearchableString, Storage, *Value);
			}
		}
	}

	template<typename SearchableType, typename PropertyType>
	const SearchableType* TPropertySearcherBase<SearchableType, PropertyType>::GetValue(
		const ICoreProvider& Storage, const UScriptStruct* Column, RowHandle Row) const
	{
		if (const void* Data = Storage.GetColumnData(Row, Column))
		{
			return Property->template ContainerPtrToValuePtr<SearchableType>(Data);
		}
		return nullptr;
	}
} // namespace UE::Editor::DataStorage::Searchers