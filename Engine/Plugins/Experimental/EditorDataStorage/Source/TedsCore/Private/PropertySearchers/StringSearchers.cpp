// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySearchers/StringSearchers.h"

namespace UE::Editor::DataStorage::Searchers
{
	//
	// FStringSearcher
	//

	FStringSearcher::FStringSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FStrProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{}

	void FStringSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FString& Value) const
	{
		SearchableString = Value;
	}

	//
	// FTextSearcher
	//

	FTextSearcher::FTextSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FTextProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{}

	void FTextSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FText& Value) const
	{
		SearchableString = Value.ToString();
	}

	//
	// FNameSearcher
	//

	FNameSearcher::FNameSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FNameProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{}

	void FNameSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FName& Value) const
	{
		SearchableString = Value.ToString();
	}

} // namespace UE::Editor::DataStorage::Searchers
