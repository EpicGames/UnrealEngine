// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "PropertySearchers/PropertySearcherBase.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Searchers
{
	class FStringSearcher final : public TPropertySearcherBase<FString, FStrProperty>
	{
		using BaseType = TPropertySearcherBase<FString, FStrProperty>;
	public:
		FStringSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FStrProperty* Property);

	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FString& Value) const override;
	};


	class FTextSearcher final : public TPropertySearcherBase<FText, FTextProperty>
	{
		using BaseType = TPropertySearcherBase<FText, FTextProperty>;
	public:
		FTextSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FTextProperty* Property);

	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FText& Value) const override;
	};


	class FNameSearcher final : public TPropertySearcherBase<FName, FNameProperty>
	{
		using BaseType = TPropertySearcherBase<FName, FNameProperty>;
	public:
		FNameSearcher(TWeakObjectPtr<const UScriptStruct> ColumnType, const FNameProperty* Property);

	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FName& Value) const override;
	};
	
} // namespace UE::Editor::DataStorage::Searchers