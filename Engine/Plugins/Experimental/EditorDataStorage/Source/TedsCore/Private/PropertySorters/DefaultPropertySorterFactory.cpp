// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySorters/DefaultPropertySorterFactory.h"

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "PropertySorters/NumericSorters.h"
#include "PropertySorters/StringSorters.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DefaultPropertySorterFactory)

void UDefaultPropertySorterFactory::RegisterPropertySorters(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Sorters;

	auto RegisterCallback = [&DataStorageUi]<typename Sorter, typename PropertyType>()
	{
		DataStorageUi.RegisterSorterGeneratorForProperty(PropertyType::StaticClass(),
			[](TWeakObjectPtr<const UScriptStruct> ColumnType, const FProperty& Property)
			{
				return MakeShared<Sorter>(ColumnType, CastField<PropertyType>(&Property));
			});
	};

	auto RegisterNumericCallback = [&DataStorageUi]<typename ValueType, typename PropertyType>()
	{
		DataStorageUi.RegisterSorterGeneratorForProperty(PropertyType::StaticClass(),
			[](TWeakObjectPtr<const UScriptStruct> ColumnType, const FProperty& Property)
			{
				return MakeShared<TNumericSorter<ValueType, PropertyType>>(ColumnType, CastField<PropertyType>(&Property));
			});
	};
	
	// Strings
	RegisterCallback.operator()<FStringSorter, FStrProperty>();
	RegisterCallback.operator()<FTextSorter, FTextProperty>();
	RegisterCallback.operator()<FNameSorter, FNameProperty>();
	
	// Signed number properties
	RegisterNumericCallback.operator()<int64, FInt64Property>();
	RegisterNumericCallback.operator()<int32, FIntProperty>();
	RegisterNumericCallback.operator()<int16, FInt16Property>();
	RegisterNumericCallback.operator()<int8,  FInt8Property>();
	
	// Unsigned number properties
	RegisterNumericCallback.operator()<uint64, FUInt64Property>();
	RegisterNumericCallback.operator()<uint32, FUInt32Property>();
	RegisterNumericCallback.operator()<uint16, FUInt16Property>();
	RegisterNumericCallback.operator()<uint8,  FByteProperty>();

	// Floating point
	RegisterNumericCallback.operator()<float, FFloatProperty>();
	RegisterNumericCallback.operator()<double, FDoubleProperty>();

	// Boolean
	RegisterCallback.operator()<FBooleanSorter, FBoolProperty>();
}
