// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "DefaultPropertySorterFactory.generated.h"

/**
 * Factory used to register sorter for numeric and text properties.
 */
UCLASS(Transient)
class UDefaultPropertySorterFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterPropertySorters(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};
