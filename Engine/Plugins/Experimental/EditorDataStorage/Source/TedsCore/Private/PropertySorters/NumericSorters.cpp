// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySorters/NumericSorters.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::DataStorage::Sorters
{
	FBooleanSorter::FBooleanSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FBoolProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{
	}

	int32 FBooleanSorter::Compare(const bool& Left, const bool& Right) const
	{
		return static_cast<int32>(Left) - static_cast<int32>(Right);
	}

	FPrefixInfo FBooleanSorter::CalculatePrefix(const bool& Value, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, Value);
	}
} // namespace UE::Editor::DataStorage::Sorters