// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "PropertySorters/PropertySorterBase.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FBoolProperty;
class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Sorters
{
	template<typename NumericType, typename NumericPropertyType>
	class TNumericSorter final : 
		public TPropertySorterBase<FColumnSorterInterface::ESortType::FixedSize64, NumericType, NumericPropertyType>
	{
		using BaseType = TPropertySorterBase<FColumnSorterInterface::ESortType::FixedSize64, NumericType, NumericPropertyType>;
	public:
		TNumericSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const NumericPropertyType* Property);

	protected:
		virtual int32 Compare(const NumericType& Left, const NumericType& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const NumericType& Value, uint32 ByteIndex) const override;
	};

	class FBooleanSorter final : public TPropertySorterBase<FColumnSorterInterface::ESortType::FixedSize64, bool, FBoolProperty>
	{
		using BaseType = TPropertySorterBase<FColumnSorterInterface::ESortType::FixedSize64, bool, FBoolProperty>;
	public:
		FBooleanSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FBoolProperty* Property);

	protected:
		virtual int32 Compare(const bool& Left, const bool& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const bool& Value, uint32 ByteIndex) const override;
	};


	//
	// Implementations
	//

	template<typename NumericType, typename NumericPropertyType>
	TNumericSorter<NumericType, NumericPropertyType>::TNumericSorter(
		TWeakObjectPtr<const UScriptStruct> ColumnType, const NumericPropertyType* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{
	}

	template<typename NumericType, typename NumericPropertyType>
	int32 TNumericSorter<NumericType, NumericPropertyType>::Compare(const NumericType& Left, const NumericType& Right) const
	{
		return Left == Right ? 0 : (Left < Right ? -1 : 1);
	}

	template<typename NumericType, typename NumericPropertyType>
	FPrefixInfo TNumericSorter<NumericType, NumericPropertyType>::CalculatePrefix(const NumericType& Value, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, Value);
	}
} // namespace UE::Editor::DataStorage::Sorters