// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Sorters
{
	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	class TPropertySorterBase : public FColumnSorterInterface
	{
	public:
		TPropertySorterBase(TWeakObjectPtr<const UScriptStruct> ColumnType, const PropertyType* Property);

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;
		ESortType GetSortType() const override final;
		FText GetShortName() const override final;

	protected:
		virtual int32 Compare(const CompareType& Left, const CompareType& Right) const = 0;
		virtual FPrefixInfo CalculatePrefix(const CompareType& Value, uint32 ByteIndex) const = 0;

	private:
		const CompareType* GetValue(const ICoreProvider& Storage, const UScriptStruct* Column, RowHandle Row) const;

		TWeakObjectPtr<const UScriptStruct> ColumnType;
		const PropertyType* Property;
	};
} // namespace UE::Editor::DataStorage::Sorters

#include "PropertySorters/PropertySorterBase.inl"