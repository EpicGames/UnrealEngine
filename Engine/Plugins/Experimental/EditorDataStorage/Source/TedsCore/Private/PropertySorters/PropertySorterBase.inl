// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::Sorters
{
	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	TPropertySorterBase<SortType, CompareType, PropertyType>::TPropertySorterBase(
		TWeakObjectPtr<const UScriptStruct> ColumnType, const PropertyType* Property)
		: ColumnType(MoveTemp(ColumnType))
		, Property(Property)
	{
	}

	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	int32 TPropertySorterBase<SortType, CompareType, PropertyType>::Compare(
		const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		if (const UScriptStruct* Column = ColumnType.Get())
		{
			const CompareType* LeftValue = GetValue(Storage, Column, Left);
			const CompareType* RightValue = GetValue(Storage, Column, Right);

			return (LeftValue && RightValue)
				? Compare(*LeftValue, *RightValue)
				: ((LeftValue == nullptr) - (RightValue == nullptr));
		}
		return 0;
	}

	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	FPrefixInfo TPropertySorterBase<SortType, CompareType, PropertyType>::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		if (const UScriptStruct* Column = ColumnType.Get())
		{
			if (const void* Data = Storage.GetColumnData(Row, Column))
			{
				if (const CompareType* Value = Property->template ContainerPtrToValuePtr<CompareType>(Data))
				{
					return CalculatePrefix(*Value, ByteIndex);
				}
			}
		}
		return FPrefixInfo{};
	}

	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	FColumnSorterInterface::ESortType TPropertySorterBase<SortType, CompareType, PropertyType>::GetSortType() const
	{
		return SortType;
	}

	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	FText TPropertySorterBase<SortType, CompareType, PropertyType>::GetShortName() const
	{
		// The property is part of the column type, so if the column type is valid then the property pointer is save to use.
		return ColumnType.IsValid() ? Property->GetDisplayNameText() : FText::GetEmpty();
	}

	template<FColumnSorterInterface::ESortType SortType, typename CompareType, typename PropertyType>
	const CompareType* TPropertySorterBase<SortType, CompareType, PropertyType>::GetValue(
		const ICoreProvider& Storage, const UScriptStruct* Column, RowHandle Row) const
	{
		if (const void* Data = Storage.GetColumnData(Row, Column))
		{
			return Property->template ContainerPtrToValuePtr<CompareType>(Data);
		}
		return nullptr;
	}
} // namespace UE::Editor::DataStorage::Sorters