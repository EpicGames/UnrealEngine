// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySorters/StringSorters.h"

namespace UE::Editor::DataStorage::Sorters
{
	//
	// FStringSorter
	//

	FStringSorter::FStringSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FStrProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{
		const FString& Sortable = Property->GetMetaData(TEXT("Sortable"));
		if (Sortable.Contains(TEXT("FString_CaseSensitive")))
		{
			bCaseSensitive = true;
		}
	}

	int32 FStringSorter::Compare(const FString& Left, const FString& Right) const
	{
		return Left.Compare(Right, bCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase);
	}

	FPrefixInfo FStringSorter::CalculatePrefix(const FString& Value, uint32 ByteIndex) const
	{
		FPrefixInfo Result = bCaseSensitive
			? CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseSensitive{}, Value))
			: CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseInsensitive{}, Value));
		return Result;
	}



	//
	// FTextSorter
	//

	FTextSorter::FTextSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FTextProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{
		const FString& Sortable = Property->GetMetaData(TEXT("Sortable"));
		if (Sortable.Contains(TEXT("FText_CaseSensitive")))
		{
			bCaseSensitive = true;
		}
	}

	int32 FTextSorter::Compare(const FText& Left, const FText& Right) const
	{
		return bCaseSensitive
			? Left.CompareTo(Right)
			: Left.CompareToCaseIgnored(Right);
	}

	FPrefixInfo FTextSorter::CalculatePrefix(const FText& Value, uint32 ByteIndex) const
	{
		return bCaseSensitive 
			? CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseSensitive{}, Value.ToString()))
			: CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseInsensitive{}, Value.ToString()));
	}



	//
	// FNameSorter
	//

	FNameSorter::FNameSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FNameProperty* Property)
		: BaseType(MoveTemp(ColumnType), Property)
	{
		const FString& Sortable = Property->GetMetaData(TEXT("Sortable"));
		if (Sortable.Contains(TEXT("FName_WithNone")))
		{
			SortByFlags |= ESortByNameFlags::WithNone;
		}
		if (Sortable.Contains(TEXT("FName_RemoveLeadingSlash")))
		{
			SortByFlags |= ESortByNameFlags::RemoveLeadingSlash;
		}
	}

	int32 FNameSorter::Compare(const FName& Left, const FName& Right) const
	{
		switch (SortByFlags)
		{
		default:
			// fall through
		case ESortByNameFlags::Default:
			return TSortNameView(TSortByName<>{}, Left).Compare(Right);
		case ESortByNameFlags::WithNone:
			return TSortNameView(TSortByName<ESortByNameFlags::WithNone>{}, Left).Compare(Right);
		case ESortByNameFlags::RemoveLeadingSlash:
			return TSortNameView(TSortByName<ESortByNameFlags::RemoveLeadingSlash>{}, Left).Compare(Right);
		case ESortByNameFlags::WithNone | ESortByNameFlags::RemoveLeadingSlash:
			return TSortNameView(TSortByName<ESortByNameFlags::WithNone | ESortByNameFlags::RemoveLeadingSlash>{}, Left).Compare(Right);
		}
	}

	FPrefixInfo FNameSorter::CalculatePrefix(const FName& Value, uint32 ByteIndex) const
	{
		switch (SortByFlags)
		{
		default:
			// fall through
		case ESortByNameFlags::Default:
			return CreateSortPrefix(ByteIndex, TSortNameView(TSortByName<>{}, Value));
		case ESortByNameFlags::WithNone:
			return CreateSortPrefix(ByteIndex, TSortNameView(TSortByName<ESortByNameFlags::WithNone>{}, Value));
		case ESortByNameFlags::RemoveLeadingSlash:
			return CreateSortPrefix(ByteIndex, TSortNameView(TSortByName<ESortByNameFlags::RemoveLeadingSlash>{}, Value));
		case ESortByNameFlags::WithNone | ESortByNameFlags::RemoveLeadingSlash:
			return CreateSortPrefix(ByteIndex, 
				TSortNameView(TSortByName<ESortByNameFlags::WithNone | ESortByNameFlags::RemoveLeadingSlash>{}, Value));
		}
	}
} // namespace UE::Editor::DataStorage::Sorters
