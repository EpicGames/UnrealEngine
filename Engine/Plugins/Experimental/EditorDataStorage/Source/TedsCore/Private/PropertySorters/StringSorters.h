// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "PropertySorters/PropertySorterBase.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Sorters
{
	class FStringSorter final : public TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FString, FStrProperty>
	{
		using BaseType = TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FString, FStrProperty>;
	public:
		FStringSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FStrProperty* Property);

	protected:
		virtual int32 Compare(const FString& Left, const FString& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const FString& Value, uint32 ByteIndex) const override;

		bool bCaseSensitive = false;
	};


	class FTextSorter final : public TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FText, FTextProperty>
	{
		using BaseType = TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FText, FTextProperty>;
	public:
		FTextSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FTextProperty* Property);

	protected:
		virtual int32 Compare(const FText& Left, const FText& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const FText& Value, uint32 ByteIndex) const override;

		bool bCaseSensitive = false;
	};


	class FNameSorter final : public TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FName, FNameProperty>
	{
		using BaseType = TPropertySorterBase<FColumnSorterInterface::ESortType::HybridSort, FName, FNameProperty>;
	public:
		FNameSorter(TWeakObjectPtr<const UScriptStruct> ColumnType, const FNameProperty* Property);

	protected:
		virtual int32 Compare(const FName& Left, const FName& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const FName& Value, uint32 ByteIndex) const override;

		ESortByNameFlags SortByFlags = ESortByNameFlags::Default;
	};
	
} // namespace UE::Editor::DataStorage::Sorters