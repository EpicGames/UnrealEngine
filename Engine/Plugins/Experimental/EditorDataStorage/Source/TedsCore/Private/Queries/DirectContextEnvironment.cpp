// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/DirectContextEnvironment.h"
#include "TypedElementDatabase.h"

namespace UE::Editor::DataStorage::Queries
{
	//
	// FDirectContextEnvironment
	//

	FDirectContextEnvironment::FDirectContextEnvironment(FEnvironment& Environment, UEditorDataStorage& DirectContext)
		: Environment(&Environment)
		, DirectContext(&DirectContext)
	{
	}

	void FDirectContextEnvironment::SetBatch(FRowHandleArrayView InRows)
	{
		Rows = InRows;
	}

	FEnvironment* FDirectContextEnvironment::GetEnvironment()
	{
		return Environment;
	}

	const FEnvironment* FDirectContextEnvironment::GetEnvironment() const
	{
		return Environment;
	}

	FRowHandleArrayView FDirectContextEnvironment::GetBatchRows() const
	{
		return Rows;
	}

	void FDirectContextEnvironment::GetConstColumns(
		TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const
	{
		if (!Rows.IsEmpty())
		{
			RowHandle Root = *Rows.GetData();
			const void** ColumnsDataIt = ColumnsData.GetData();
			for (const UScriptStruct* ColumnType : ColumnTypes)
			{
				*ColumnsDataIt++ = DirectContext->GetColumnData(Root, ColumnType);
			}
		}
		else
		{
			FMemory::Memzero(ColumnsData.GetData(), ColumnsData.NumBytes());
		}
	}

	void FDirectContextEnvironment::GetMutableColumns(
		TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		if (!Rows.IsEmpty())
		{
			// At the time this is called there are guaranteed to be rows assigned.
			RowHandle Root = *Rows.GetData();
			void** ColumnsDataIt = ColumnsData.GetData();
			for (const UScriptStruct* ColumnType : ColumnTypes)
			{
				*ColumnsDataIt++ = DirectContext->GetColumnData(Root, ColumnType);
			}
		}
		else
		{
			FMemory::Memzero(ColumnsData.GetData(), ColumnsData.NumBytes());
		}
	}

	void FDirectContextEnvironment::SetCurrentRow(RowHandle Row)
	{
		CurrentRow = Row;
	}



	//
	// FConstDirectContextEnvironment
	//

	FConstDirectContextEnvironment::FConstDirectContextEnvironment(const FEnvironment& Environment, const UEditorDataStorage& DirectContext)
		: Environment(&Environment)
		, DirectContext(&DirectContext)
	{
	}

	void FConstDirectContextEnvironment::SetBatch(FRowHandleArrayView InRows)
	{
		Rows = InRows;
	}

	FEnvironment* FConstDirectContextEnvironment::GetEnvironment()
	{
		checkf(false, TEXT("Calling a mutable version of the environment is not supported by a const environment."));
		return nullptr;
	}

	const FEnvironment* FConstDirectContextEnvironment::GetEnvironment() const
	{
		return Environment;
	}

	FRowHandleArrayView FConstDirectContextEnvironment::GetBatchRows() const
	{
		return Rows;
	}

	void FConstDirectContextEnvironment::GetConstColumns(
		TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const
	{
		if (!Rows.IsEmpty())
		{
			RowHandle Root = *Rows.GetData();
			const void** ColumnsDataIt = ColumnsData.GetData();
			for (const UScriptStruct* ColumnType : ColumnTypes)
			{
				*ColumnsDataIt++ = DirectContext->GetColumnData(Root, ColumnType);
			}
		}
		else
		{
			FMemory::Memzero(ColumnsData.GetData(), ColumnsData.NumBytes());
		}
	}

	void FConstDirectContextEnvironment::GetMutableColumns(
		TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		checkf(false, TEXT("Retrieving mutable columns is not supported from a const environment."));
		FMemory::Memzero(ColumnsData.GetData(), ColumnsData.NumBytes());
	}

	void FConstDirectContextEnvironment::SetCurrentRow(RowHandle Row)
	{
		CurrentRow = Row;
	}
} // namespace UE::Editor::DataStorage::Queries