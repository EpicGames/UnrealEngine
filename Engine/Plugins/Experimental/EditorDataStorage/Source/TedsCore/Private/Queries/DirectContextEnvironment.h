// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementQueryContextImplementation.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "TypedElementDatabaseEnvironment.h"
#include "Queries/QueryContextCapabilities/ContextEnvironment.h"
#include "Queries/QueryContextCapabilities/EditorDataStorageDynamicColumnInfo.h"
#include "Queries/QueryContextCapabilities/TableCapabilities.h"
#include "Queries/QueryContextCapabilities/RowBatch.h"
#include "Queries/QueryContextCapabilities/SingleRow.h"

class UEditorDataStorage;

namespace UE::Editor::DataStorage::Queries
{
	struct FDirectContextEnvironment final : IContextEnvironment
	{
		FDirectContextEnvironment() = default;
		FDirectContextEnvironment(FEnvironment& Environment, UEditorDataStorage& DirectContext);

		void SetBatch(FRowHandleArrayView InRows);

		virtual FEnvironment* GetEnvironment() override;
		virtual const FEnvironment* GetEnvironment() const override;

		virtual FRowHandleArrayView GetBatchRows() const override;
		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const override;
		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		virtual void SetCurrentRow(RowHandle Row) override;

		FRowHandleArrayView Rows;
		FEnvironment* Environment = nullptr;
		UEditorDataStorage* DirectContext = nullptr;
		RowHandle CurrentRow = InvalidRowHandle;
	};

	struct FConstDirectContextEnvironment final : IContextEnvironment
	{
		/** The provided rows need to be contiguous in the same table. */
		FConstDirectContextEnvironment() = default;
		FConstDirectContextEnvironment(const FEnvironment& Environment, const UEditorDataStorage& DirectContext);

		void SetBatch(FRowHandleArrayView InRows);

		virtual FEnvironment* GetEnvironment() override;
		virtual const FEnvironment* GetEnvironment() const override;

		virtual FRowHandleArrayView GetBatchRows() const override;
		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const override;
		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		virtual void SetCurrentRow(RowHandle Row) override;

		FRowHandleArrayView Rows;
		const FEnvironment* Environment = nullptr;
		const UEditorDataStorage* DirectContext = nullptr;
		RowHandle CurrentRow = InvalidRowHandle;
	};

	using QueryContext_DirectApi = TQueryContextImpl<false, FDirectContextEnvironment,
		FCurrentTableInfoContextCapability,
		FTableInfoContextCapability,
		FSingleRowInfoContextCapability,
		FRowBatchInfoContextCapability,
		FDynamicColumnInfoContextCapability>;

	using QueryContext_ConstDirectApi = TQueryContextImpl<true, FConstDirectContextEnvironment,
		FCurrentTableInfoContextCapability,
		FTableInfoContextCapability,
		FSingleRowInfoContextCapability,
		FRowBatchInfoContextCapability,
		FDynamicColumnInfoContextCapability>;
} // namespace UE::Editor::DataStorage::Queries
