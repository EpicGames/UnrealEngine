// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/ProcessorContextEnvironment.h"
#include "MassExecutionContext.h"
#include "TypedElementUtils.h"

namespace UE::Editor::DataStorage::Queries
{
	//
	// FProcessorContextEnvironment
	//

	FProcessorContextEnvironment::FProcessorContextEnvironment(FEnvironment& Environment, FMassExecutionContext& MassContext)
		: Environment(Environment)
		, MassContext(MassContext)
	{
	}

	FEnvironment* FProcessorContextEnvironment::GetEnvironment()
	{
		return &Environment;
	}

	const FEnvironment* FProcessorContextEnvironment::GetEnvironment() const
	{
		return &Environment;
	}

	FRowHandleArrayView FProcessorContextEnvironment::GetBatchRows() const
	{
		TConstArrayView<RowHandle> Rows = MassEntitiesToRowsConversion(MassContext.GetEntities());
		if (HasSubRange())
		{
			Rows = Rows.Mid(SubRangeStart, SubRangeLength);
		}
		return FRowHandleArrayView(Rows, FRowHandleArrayView::EFlags::IsUnique);
	}

	void FProcessorContextEnvironment::GetConstColumns(
		TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const
	{
		checkfSlow(ColumnsData.Num() >= ColumnTypes.Num(), TEXT("The list size of requested const column types (%i) doesn't match the "
			"available space for the results (%i)"), ColumnTypes.Num(), ColumnsData.Num());

		const void** ColumnsDataIt = ColumnsData.GetData();
		for (const UScriptStruct* ColumnType : ColumnTypes)
		{
			checkfSlow(UE::Mass::IsA<FMassFragment>(ColumnType),
				TEXT("Unable to retrieve const column data for a column type without data: %s"), *ColumnType->GetFullName());
			const uint8* Base = reinterpret_cast<const uint8*>(MassContext.GetFragmentView(ColumnType).GetData());
			*ColumnsDataIt++ = HasSubRange() ? Base + SubRangeStart * ColumnType->GetStructureSize() : Base;
		}
	}

	void FProcessorContextEnvironment::GetMutableColumns(
		TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		checkfSlow(ColumnsData.Num() >= ColumnTypes.Num(), TEXT("The list size of requested mutable column types (%i) doesn't match the "
			"available space for the results (%i)"), ColumnTypes.Num(), ColumnsData.Num());

		void** ColumnsDataIt = ColumnsData.GetData();
		for (const UScriptStruct* ColumnType : ColumnTypes)
		{
			checkfSlow(UE::Mass::IsA<FMassFragment>(ColumnType),
				TEXT("Unable to retrieve mutable column data for a column type without data: %s"), *ColumnType->GetFullName());
			uint8* Base = reinterpret_cast<uint8*>(MassContext.GetMutableFragmentView(ColumnType).GetData());
			*ColumnsDataIt++ = HasSubRange() ? Base + SubRangeStart * ColumnType->GetStructureSize() : Base;
		}
	}

	void FProcessorContextEnvironment::SetCurrentRow(RowHandle Row)
	{
		CurrentRow = Row;
	}


	//
	// FConstProcessorContextEnvironment
	//

	FConstProcessorContextEnvironment::FConstProcessorContextEnvironment(
		const FEnvironment& Environment, const FMassExecutionContext& MassContext)
		: Environment(Environment)
		, MassContext(MassContext)
	{
	}

	FEnvironment* FConstProcessorContextEnvironment::GetEnvironment()
	{
		checkf(false, TEXT("Calling a mutable version of the environment is not supported by a const environment."));
		return nullptr;
	}

	const FEnvironment* FConstProcessorContextEnvironment::GetEnvironment() const
	{
		return &Environment;
	}

	FRowHandleArrayView FConstProcessorContextEnvironment::GetBatchRows() const
	{
		TConstArrayView<RowHandle> Rows = MassEntitiesToRowsConversion(MassContext.GetEntities());
		if (HasSubRange())
		{
			Rows = Rows.Mid(SubRangeStart, SubRangeLength);
		}
		return FRowHandleArrayView(Rows, FRowHandleArrayView::EFlags::IsUnique);
	}

	void FConstProcessorContextEnvironment::GetConstColumns(
		TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const
	{
		checkfSlow(ColumnsData.Num() >= ColumnTypes.Num(), TEXT("The list size of requested const column types (%i) doesn't match the "
			"available space for the results (%i)"), ColumnTypes.Num(), ColumnsData.Num());

		const void** ColumnsDataIt = ColumnsData.GetData();
		for (const UScriptStruct* ColumnType : ColumnTypes)
		{
			checkfSlow(UE::Mass::IsA<FMassFragment>(ColumnType),
				TEXT("Unable to retrieve const column data for a column type without data: %s"), *ColumnType->GetFullName());
			const uint8* Base = reinterpret_cast<const uint8*>(MassContext.GetFragmentView(ColumnType).GetData());
			*ColumnsDataIt++ = HasSubRange() ? Base + SubRangeStart * ColumnType->GetStructureSize() : Base;
		}
	}

	void FConstProcessorContextEnvironment::GetMutableColumns(
		TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes)
	{
		checkf(false, TEXT("Retrieving mutable columns is not supported from a const environment."));
		FMemory::Memzero(ColumnsData.GetData(), ColumnsData.NumBytes());
	}

	void FConstProcessorContextEnvironment::SetCurrentRow(RowHandle Row)
	{
		CurrentRow = Row;
	}
} // namespace UE::Editor::DataStorage::Queries