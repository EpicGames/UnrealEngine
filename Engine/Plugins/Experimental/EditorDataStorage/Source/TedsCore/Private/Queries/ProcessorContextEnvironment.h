// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementQueryContextImplementation.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "TypedElementDatabaseEnvironment.h"
#include "Queries/QueryContextCapabilities/EditorDataStorageDynamicColumnInfo.h"
#include "Queries/QueryContextCapabilities/TableCapabilities.h"
#include "Queries/QueryContextCapabilities/RowBatch.h"
#include "Queries/QueryContextCapabilities/SingleRow.h"

struct FMassExecutionContext;

namespace UE::Editor::DataStorage::Queries
{
	struct FProcessorContextEnvironment final : IContextEnvironment
	{
		FProcessorContextEnvironment(FEnvironment& Environment, FMassExecutionContext& MassContext);

		virtual FEnvironment* GetEnvironment() override;
		virtual const FEnvironment* GetEnvironment() const override;

		virtual FRowHandleArrayView GetBatchRows() const override;
		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const override;
		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		virtual void SetCurrentRow(RowHandle Row) override;

		void SetSubRange(int32 InStart, int32 InLength) { SubRangeStart = InStart; SubRangeLength = InLength; }
		void ClearSubRange() { SubRangeStart = 0; SubRangeLength = -1; }
		bool HasSubRange() const { return SubRangeLength >= 0; }

		FEnvironment& Environment;
		FMassExecutionContext& MassContext;
		RowHandle CurrentRow = InvalidRowHandle;
		int32 SubRangeStart = 0;
		int32 SubRangeLength = -1;
	};

	struct FConstProcessorContextEnvironment final : IContextEnvironment
	{
		FConstProcessorContextEnvironment(const FEnvironment& Environment, const FMassExecutionContext& MassContext);

		virtual FEnvironment* GetEnvironment() override;
		virtual const FEnvironment* GetEnvironment() const override;

		virtual FRowHandleArrayView GetBatchRows() const override;
		virtual void GetConstColumns(TArrayView<const void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) const override;
		virtual void GetMutableColumns(TArrayView<void*> ColumnsData, TConstArrayView<const UScriptStruct*> ColumnTypes) override;
		virtual void SetCurrentRow(RowHandle Row) override;

		void SetSubRange(int32 InStart, int32 InLength) { SubRangeStart = InStart; SubRangeLength = InLength; }
		void ClearSubRange() { SubRangeStart = 0; SubRangeLength = -1; }
		bool HasSubRange() const { return SubRangeLength >= 0; }

		const FEnvironment& Environment;
		const FMassExecutionContext& MassContext;
		RowHandle CurrentRow = InvalidRowHandle;
		int32 SubRangeStart = 0;
		int32 SubRangeLength = -1;
	};

	using QueryContext_ProcessorApi = TQueryContextImpl<false, FProcessorContextEnvironment,
		FCurrentTableInfoContextCapability,
		FTableInfoContextCapability,
		FSingleRowInfoContextCapability,
		FRowBatchInfoContextCapability,
		FDynamicColumnInfoContextCapability>;
	
	using QueryContext_ConstProcessorApi = TQueryContextImpl<true, FConstProcessorContextEnvironment,
		FCurrentTableInfoContextCapability,
		FTableInfoContextCapability,
		FSingleRowInfoContextCapability,
		FRowBatchInfoContextCapability,
		FDynamicColumnInfoContextCapability>;
} // namespace UE::Editor::DataStorage::Queries
