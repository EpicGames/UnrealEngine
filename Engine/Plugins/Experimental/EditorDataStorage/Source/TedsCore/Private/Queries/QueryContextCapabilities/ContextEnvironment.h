// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"

namespace UE::Editor::DataStorage
{
	class FEnvironment;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::Queries
{
	struct IContextEnvironment : IQueryFunctionResponse
	{
		virtual ~IContextEnvironment() = default;

		virtual FEnvironment* GetEnvironment() = 0;
		virtual const FEnvironment* GetEnvironment() const = 0;
	};
} // namespace UE::Editor::DataStorage::Queries
