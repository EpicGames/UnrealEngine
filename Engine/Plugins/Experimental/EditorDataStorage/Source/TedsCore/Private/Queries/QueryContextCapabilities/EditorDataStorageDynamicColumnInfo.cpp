// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/QueryContextCapabilities/EditorDataStorageDynamicColumnInfo.h"

#include "Queries/QueryContextCapabilities/ContextEnvironment.h"

namespace UE::Editor::DataStorage::Queries
{
	const UScriptStruct* FDynamicColumnInfoContextCapability::FindDynamicColumnType(
		const IContextEnvironment& Environment, const UScriptStruct& TemplateType, const FName& Identifier)
	{
		return Environment.GetEnvironment()->FindDynamicColumn(TemplateType, Identifier);
	}

	void FDynamicColumnInfoContextCapability::ForEachDynamicColumnType(
		const IContextEnvironment& Environment, const UScriptStruct& TemplateType, TFunctionRef<void(const UScriptStruct& Type)> Callback)
	{
		Environment.GetEnvironment()->ForEachDynamicColumn(TemplateType, Callback);
	}
} // namespace UE::Editor::DataStorage::Queries
