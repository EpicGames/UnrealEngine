// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "TypedElementDatabaseEnvironment.h"

class FName;
class UScriptStruct;

namespace UE::Editor::DataStorage::Queries
{
	struct IContextEnvironment;

	/**
	 * Implementation for the DynamicColumnInfo context capability.
	 * This provides read access to the dynamic columns in order to retrieve information about them from any thread.
	 * This class provides an implementation block and is not meant to be directly used but should be inherited from
	 * by a class that's compatible with TQueryContextImpl.
	 */
	class FDynamicColumnInfoContextCapability : public ImplementsContextCapability<DynamicColumnInfo>
	{
	public:
		static const UScriptStruct* FindDynamicColumnType(
			const IContextEnvironment& Environment, const UScriptStruct& TemplateType, const FName& Identifier);
		static void ForEachDynamicColumnType(
			const IContextEnvironment& Environment, const UScriptStruct& TemplateType, TFunctionRef<void(const UScriptStruct& Type)> Callback);
	};
} // namespace UE::Editor::DataStorage::Queries
