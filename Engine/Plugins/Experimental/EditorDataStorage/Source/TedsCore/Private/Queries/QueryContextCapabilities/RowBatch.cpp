// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/QueryContextCapabilities/RowBatch.h"

#include "Queries/QueryContextCapabilities/TableCapabilities.h"
#include "Queries/DirectContextEnvironment.h"
#include "Queries/ProcessorContextEnvironment.h"
#include "TypedElementDatabase.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementUtils.h"

namespace UE::Editor::DataStorage::Queries
{
	//
	// IContextEnvironment
	//

	uint32 FRowBatchInfoContextCapability::GetBatchRowCount(const IContextEnvironment& Environment)
	{
		return Environment.GetBatchRows().Num();
	}

	FRowHandleArrayView FRowBatchInfoContextCapability::GetBatchRowHandles(const IContextEnvironment& Environment)
	{
		return Environment.GetBatchRows();
	}

	bool FRowBatchInfoContextCapability::CurrentBatchTableHasColumns(
		const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}




	//
	// FProcessorContextEnvironment
	// 

	bool FRowBatchInfoContextCapability::CurrentBatchTableHasColumns(
		const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}

	const void* FRowBatchInfoContextCapability::GetColumnBatchAddress(
		const FProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		const uint8* Base = reinterpret_cast<const uint8*>(Environment.MassContext.GetFragmentView(ColumnType).GetData());
		return Environment.HasSubRange() ? Base + Environment.SubRangeStart * ColumnType->GetStructureSize() : Base;
	}

	void* FRowBatchInfoContextCapability::GetMutableColumnBatchAddress(
		FProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		uint8* Base = reinterpret_cast<uint8*>(Environment.MassContext.GetMutableFragmentView(ColumnType).GetData());
		return Environment.HasSubRange() ? Base + Environment.SubRangeStart * ColumnType->GetStructureSize() : Base;
	}



	//
	// FConstProcessorContextEnvironment
	// 

	bool FRowBatchInfoContextCapability::CurrentBatchTableHasColumns(
		const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}

	const void* FRowBatchInfoContextCapability::GetColumnBatchAddress(
		const FConstProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		const uint8* Base = reinterpret_cast<const uint8*>(Environment.MassContext.GetFragmentView(ColumnType).GetData());
		return Environment.HasSubRange() ? Base + Environment.SubRangeStart * ColumnType->GetStructureSize() : Base;
	}

	
	
	//
	// FDirectContextEnvironment
	//

	const void* FRowBatchInfoContextCapability::GetColumnBatchAddress(const FDirectContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		
		return !Environment.Rows.IsEmpty()
			? Environment.DirectContext->GetColumnData(*Environment.Rows.GetData(), ColumnType)
			: nullptr;
	}

	void* FRowBatchInfoContextCapability::GetMutableColumnBatchAddress(FDirectContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		return !Environment.Rows.IsEmpty()
			? Environment.DirectContext->GetColumnData(*Environment.Rows.GetData(), ColumnType)
			: nullptr;
	}


	//
	// FConstDirectContextEnvironment
	//

	const void* FRowBatchInfoContextCapability::GetColumnBatchAddress(const FConstDirectContextEnvironment& Environment, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType), TEXT("Trying to retrieve a data pointer for column '%s' which doesn't have data."),
			ColumnType ? *ColumnType->GetName() : TEXT("<null>"));
		return !Environment.Rows.IsEmpty()
			? Environment.DirectContext->GetColumnData(*Environment.Rows.GetData(), ColumnType)
			: nullptr;
	}
} // namespace UE::Editor::DataStorage::Queries
