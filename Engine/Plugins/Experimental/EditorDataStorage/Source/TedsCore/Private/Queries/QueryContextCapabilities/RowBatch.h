// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Queries/QueryContextCapabilities/ContextEnvironment.h"

namespace UE::Editor::DataStorage::Queries
{
	struct FConstDirectContextEnvironment;
	struct FConstProcessorContextEnvironment;
	struct FDirectContextEnvironment;
	struct FProcessorContextEnvironment;

	class FRowBatchInfoContextCapability : public ImplementsContextCapability<RowBatchInfo>
	{
	public:
		static uint32 GetBatchRowCount(const IContextEnvironment& Environment);
		static FRowHandleArrayView GetBatchRowHandles(const IContextEnvironment& Environment);
		static bool CurrentBatchTableHasColumns(const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);

		static bool CurrentBatchTableHasColumns(const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static const void* GetColumnBatchAddress(const FProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType);
		static void* GetMutableColumnBatchAddress(FProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType);

		static bool CurrentBatchTableHasColumns(const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static const void* GetColumnBatchAddress(const FConstProcessorContextEnvironment& Environment, const UScriptStruct* ColumnType);

		static const void* GetColumnBatchAddress(const FDirectContextEnvironment& Environment, const UScriptStruct* ColumnType);
		static void* GetMutableColumnBatchAddress(FDirectContextEnvironment& Environment, const UScriptStruct* ColumnType);

		static const void* GetColumnBatchAddress(const FConstDirectContextEnvironment& Environment, const UScriptStruct* ColumnType);
	};
} // namespace UE::Editor::DataStorage::Queries