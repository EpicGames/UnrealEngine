// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/QueryContextCapabilities/SingleRow.h"

#include "Queries/QueryContextCapabilities/ContextEnvironment.h"
#include "Queries/QueryContextCapabilities/TableCapabilities.h"
#include "TypedElementDatabaseEnvironment.h"

namespace UE::Editor::DataStorage::Queries
{
	bool FSingleRowInfoContextCapability::CurrentRowHasColumns(
		const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}

	bool FSingleRowInfoContextCapability::CurrentRowHasColumns(
		const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}

	bool FSingleRowInfoContextCapability::CurrentRowHasColumns(
		const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		return FCurrentTableInfoContextCapability::CurrentTableHasColumns(Environment, Columns);
	}
} // namespace UE::Editor::DataStorage::Queries
