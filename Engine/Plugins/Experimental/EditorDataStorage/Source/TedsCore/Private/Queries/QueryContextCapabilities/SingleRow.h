// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/ConvertibleTo.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"

namespace UE::Editor::DataStorage::Queries
{
	struct IContextEnvironment;
	struct FConstProcessorContextEnvironment;
	struct FProcessorContextEnvironment;

	template<typename T>
	concept CHasCurrentRow = requires(const T & Type)
	{
		{ Type.CurrentRow } -> UE::CConvertibleTo<RowHandle>;
	};

	class FSingleRowInfoContextCapability : public ImplementsContextCapability<SingleRowInfo>
	{
	public:
		static bool CurrentRowHasColumns(const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static bool CurrentRowHasColumns(const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static bool CurrentRowHasColumns(const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);

		template<CHasCurrentRow EnvironmentType>
		static RowHandle GetCurrentRow(const EnvironmentType& Environment)
		{ 
			return Environment.CurrentRow;
		}
	};
} // namespace UE::Editor::DataStorage::Queries
