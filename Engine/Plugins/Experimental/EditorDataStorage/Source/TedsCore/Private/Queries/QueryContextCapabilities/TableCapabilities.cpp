// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/QueryContextCapabilities/TableCapabilities.h"

#include "MassArchetypeTypes.h"
#include "MassExecutionContext.h"
#include "Queries/DirectContextEnvironment.h"
#include "Queries/ProcessorContextEnvironment.h"
#include "TypedElementDatabaseEnvironment.h"

namespace UE::Editor::DataStorage::Queries
{
	// 
	// FTableInfoContextCapability
	//

	bool FTableInfoContextCapability::IsTableValid(const IContextEnvironment& Environment, TableHandle Table)
	{
		return Environment.GetEnvironment()->GetTableManager().IsTableValid(Table);
	}

	TableHandle FTableInfoContextCapability::FindTable(const IContextEnvironment& Environment, const FName& TableName)
	{
		return Environment.GetEnvironment()->GetTableManager().Find(TableName);
	}

	TableHandle FTableInfoContextCapability::FindTable(const IContextEnvironment& Environment, RowHandle Row)
	{
		return Environment.GetEnvironment()->GetTableManager().Find(Row);
	}

	void FTableInfoContextCapability::ListTables(
		const IContextEnvironment& Environment, TFunctionRef<void(TableHandle)> Callback)
	{
		return Environment.GetEnvironment()->GetTableManager().List(Callback);
	}

	void FTableInfoContextCapability::ListTables(
		const IContextEnvironment& Environment, ETableType TableType, TFunctionRef<void(TableHandle)> Callback)
	{
		return Environment.GetEnvironment()->GetTableManager().List(TableType, Callback);
	}

	void FTableInfoContextCapability::ListTableColumns(
		const IContextEnvironment& Environment, TableHandle Table, TFunctionRef<bool(const UScriptStruct*)> Callback)
	{
		return Environment.GetEnvironment()->GetTableManager().ListColumns(Table, Callback);
	}

	bool FTableInfoContextCapability::TableHasColumns(
		const IContextEnvironment& Environment, TableHandle Table, TConstArrayView<const UScriptStruct*> Columns)
	{
		return Environment.GetEnvironment()->GetTableManager().HasColumns(Table, Columns);
	}

	FTableInfoView FTableInfoContextCapability::GetTableInfo(const IContextEnvironment& Environment, TableHandle Table)
	{
		return Environment.GetEnvironment()->GetTableManager().GetInfo(Table);
	}

	void FTableInfoContextCapability::ListTableRows(
		const IContextEnvironment& Environment, TableHandle Table, TFunctionRef<void(FRowHandleArrayView)> Callback)
	{
		return Environment.GetEnvironment()->GetTableManager().ListRows(Table, Callback);
	}

	void FTableInfoContextCapability::ListTableForeignKeyDomains(
		const IContextEnvironment& Environment, TableHandle Table, bool bIncludeParents, TFunctionRef<void(const FName&)> Callback)
	{
		return Environment.GetEnvironment()->GetTableManager().ListForeignKeyDomains(Table, bIncludeParents, Callback);
	}

	const UScriptStruct* FTableInfoContextCapability::GetTagForTable(const IContextEnvironment& Environment, TableHandle Table)
	{
		return Environment.GetEnvironment()->GetTableManager().GetTagForTable(Table);
	}



	// 
	// FCurrentTableInfoContextCapability
	//

	TableHandle FCurrentTableInfoContextCapability::GetCurrentTable(const IContextEnvironment& Environment)
	{
		FRowHandleArrayView Rows = Environment.GetBatchRows();
		return !Rows.IsEmpty()
			? FTableInfoContextCapability::FindTable(Environment, Rows.First())
			: InvalidTableHandle;
	}

	TableHandle FCurrentTableInfoContextCapability::GetCurrentTable(const FProcessorContextEnvironment& Environment)
	{
		return Environment.GetEnvironment()->GetTableManager().Find(Environment.MassContext.GetEntityCollection().GetArchetype());
	}

	TableHandle FCurrentTableInfoContextCapability::GetCurrentTable(const FConstProcessorContextEnvironment& Environment)
	{
		return Environment.GetEnvironment()->GetTableManager().Find(Environment.MassContext.GetEntityCollection().GetArchetype());
	}

	bool FCurrentTableInfoContextCapability::CurrentTableHasColumns(
		const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		TableHandle Table = GetCurrentTable(Environment);
		return Table != InvalidTableHandle ? FTableInfoContextCapability::TableHasColumns(Environment, Table, Columns) : false;
	}

	bool FCurrentTableInfoContextCapability::CurrentTableHasColumns(
		const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		const FMassExecutionContext& Context = Environment.MassContext;
		for (const UScriptStruct* Column : Columns)
		{
			if (!Context.DoesArchetypeHaveElement(Column))
			{
				return false;
			}
		}
		return true;
	}

	bool FCurrentTableInfoContextCapability::CurrentTableHasColumns(
		const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns)
	{
		const FMassExecutionContext& Context = Environment.MassContext;
		for (const UScriptStruct* Column : Columns)
		{
			if (!Context.DoesArchetypeHaveElement(Column))
			{
				return false;
			}
		}
		return true;
	}



	// 
	// FTableManagementContextCapability
	//

	TableHandle FTableManagementContextCapability::RegisterTable(IContextEnvironment& Environment,
		TConstArrayView<const UScriptStruct*> Columns, const FName& TableName)
	{
		return Environment.GetEnvironment()->GetTableManager().Register(Columns, TableName);
	}

	TableHandle FTableManagementContextCapability::RegisterTable(IContextEnvironment& Environment,
		TConstArrayView<const UScriptStruct*> Columns, const FName& TableName, FTableRegistrationOptions Options)
	{
		return Environment.GetEnvironment()->GetTableManager().Register(Columns, TableName, Options);
	}

	void FTableManagementContextCapability::RegisterTableForeignKey(IContextEnvironment& Environment,
		TableHandle Table, const FName& Domain, const Queries::TConstQueryFunction<FForeignKey>& KeyConstructor)
	{
		return Environment.GetEnvironment()->GetTableManager().RegisterForeignKey(Table, Domain, KeyConstructor);
	}

	void FTableManagementContextCapability::RegisterTableForeignKey(IContextEnvironment& Environment,
		TableHandle Table, const FName& Domain, Queries::TConstQueryFunction<FForeignKey>&& KeyConstructor)
	{
		return Environment.GetEnvironment()->GetTableManager().RegisterForeignKey(Table, Domain, MoveTemp(KeyConstructor));
	}

	FForeignKey FTableManagementContextCapability::BuildForeignKey(
		const FConstDirectContextEnvironment& Environment, const FName& Domain, RowHandle Row)
	{
		return BuildForeignKey(Environment.GetEnvironment(), Environment.DirectContext, Domain, Row);
	}

	FForeignKey FTableManagementContextCapability::BuildForeignKey(
		const FDirectContextEnvironment& Environment, const FName& Domain, RowHandle Row)
	{
		return BuildForeignKey(Environment.GetEnvironment(), Environment.DirectContext, Domain, Row);
	}

	FForeignKey FTableManagementContextCapability::BuildForeignKey(
		const FEnvironment* Environment, const UEditorDataStorage* DirectContext, const FName& Domain, RowHandle Row)
	{
		checkf(Environment, TEXT("Unable to build foreign key because the TEDS environment doesn't exist."));
		checkf(DirectContext, TEXT("Unable to build foreign key because TEDS Core hasn't been instantiated."));

		FConstDirectContextEnvironment LocalContextEnvironment(*Environment, *DirectContext);
		QueryContext_ConstDirectApi Context(LocalContextEnvironment);
		LocalContextEnvironment.SetBatch(FRowHandleArrayView(&Row, 1, FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));

		FForeignKey Result;
		Environment->GetTableManager().BuildForeignKeys(TArrayView<FForeignKey>(&Result, 1), Domain, Context, LocalContextEnvironment);
		return Result;
	}
} // namespace UE::Editor::DataStorage::Queries
