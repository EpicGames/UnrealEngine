// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Queries/QueryContextCapabilities/ContextEnvironment.h"
#include "Templates/UnrealTypeTraits.h"

class FName;
class UScriptStruct;
class UEditorDataStorage;

namespace UE::Editor::DataStorage::Queries
{
	struct IContextEnvironment;
	struct FConstDirectContextEnvironment;
	struct FConstProcessorContextEnvironment;
	struct FDirectContextEnvironment;
	struct FProcessorContextEnvironment;
	
	class FTableInfoContextCapability : public ImplementsContextCapability<TableInfo>
	{
	public:
		static bool IsTableValid(const IContextEnvironment& Environment, TableHandle Table);
		static TableHandle FindTable(const IContextEnvironment& Environment, const FName& TableName);
		static TableHandle FindTable(const IContextEnvironment& Environment, RowHandle Row);
		static void ListTables(const IContextEnvironment& Environment, TFunctionRef<void(TableHandle)> Callback);
		static void ListTables(const IContextEnvironment& Environment, ETableType TableType, TFunctionRef<void(TableHandle)> Callback);
		static void ListTableColumns(
			const IContextEnvironment& Environment, TableHandle Table, TFunctionRef<bool(const UScriptStruct*)> Callback);

		static bool TableHasColumns(
			const IContextEnvironment& Environment, TableHandle Table, TConstArrayView<const UScriptStruct*> Columns);
		static FTableInfoView GetTableInfo(const IContextEnvironment& Environment, TableHandle Table);
		static void ListTableRows(
			const IContextEnvironment& Environment, TableHandle Table, TFunctionRef<void(FRowHandleArrayView)> Callback);
		
		static void ListTableForeignKeyDomains(
			const IContextEnvironment& Environment, TableHandle Table, bool bIncludeParents, TFunctionRef<void(const FName&)> Callback);

		static const UScriptStruct* GetTagForTable(const IContextEnvironment& Environment, TableHandle Table);
	};

	class FCurrentTableInfoContextCapability : public ImplementsContextCapability<CurrentTableInfo>
	{
	public:
		static TableHandle GetCurrentTable(const IContextEnvironment& Environment);
		static TableHandle GetCurrentTable(const FProcessorContextEnvironment& Environment);
		static TableHandle GetCurrentTable(const FConstProcessorContextEnvironment& Environment);
		
		static bool CurrentTableHasColumns(
			const IContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static bool CurrentTableHasColumns(
			const FProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);
		static bool CurrentTableHasColumns(
			const FConstProcessorContextEnvironment& Environment, TConstArrayView<const UScriptStruct*> Columns);

		template<typename EnvironmentContextType> requires (TIsDerivedFrom<EnvironmentContextType, IContextEnvironment>::Value)
		static const UScriptStruct* GetTagForCurrentTable(const EnvironmentContextType& Environment)
		{
			TableHandle CurrentTable = GetCurrentTable(Environment);
			checkf(CurrentTable != InvalidTableHandle, TEXT("If this function is reachable there should be a valid table."));
			const UScriptStruct* Tag = FTableInfoContextCapability::GetTagForTable(Environment, CurrentTable);
			checkf(Tag, TEXT("The current table doesn't have a tag associated with it."));
			return Tag;
		}
	};
	
	class FTableManagementContextCapability : public ImplementsContextCapability<TableManagement>
	{
	public:
		static TableHandle RegisterTable(IContextEnvironment& Environment,
			TConstArrayView<const UScriptStruct*> Columns, const FName& TableName);
		static TableHandle RegisterTable(IContextEnvironment& Environment,
			TConstArrayView<const UScriptStruct*> Columns, const FName& TableName, FTableRegistrationOptions Options);
		static void RegisterTableForeignKey(IContextEnvironment& Environment,
			TableHandle Table, const FName& Domain, const Queries::TConstQueryFunction<FForeignKey>& KeyConstructor);
		static void RegisterTableForeignKey(IContextEnvironment& Environment,
			TableHandle Table, const FName& Domain, Queries::TConstQueryFunction<FForeignKey>&& KeyConstructor);
		static FForeignKey BuildForeignKey(const FDirectContextEnvironment& Environment, const FName& Domain, RowHandle Row);
		static FForeignKey BuildForeignKey(const FConstDirectContextEnvironment& Environment, const FName& Domain, RowHandle Row);

	private:
		static FForeignKey BuildForeignKey(
			const FEnvironment* Environment, const UEditorDataStorage* DirectContext, const FName& Domain, RowHandle Row);
	};
} // namespace UE::Editor::DataStorage::Queries
