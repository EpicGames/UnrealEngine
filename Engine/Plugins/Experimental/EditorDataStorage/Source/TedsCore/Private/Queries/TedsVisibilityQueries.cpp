// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsVisibilityQueries.h"

#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

void UTedsVisibilityFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove 'visibility change' tag"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FVisibilityChangedTag>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			}
		)
		.Where()
			.All<FVisibilityChangedTag>()
		.Compile());
}
