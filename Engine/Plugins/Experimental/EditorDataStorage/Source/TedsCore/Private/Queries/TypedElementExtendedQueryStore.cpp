// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/TypedElementExtendedQueryStore.h"

#include "MassProcessingPhaseManager.h"
#include "MassProcessor.h"
#include "TypedElementDatabaseEnvironment.h"
#include "DataStorage/Debug/Log.h"
#include "Misc/OutputDevice.h"
#include "Processors/TypedElementProcessorAdaptors.h"

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		bool bUseTedsMatchFunction = true;
		static FAutoConsoleVariableRef CVarUseTedsMatchFunction(
			TEXT("TEDS.Feature.UseTedsMatchFunction"), bUseTedsMatchFunction,
			TEXT("Override the Mass query's match function to use the TEDS match function"),
			ECVF_Default);
	}
	const FQueryDescription FExtendedQueryStore::EmptyDescription{};

	class FEnvironmentQueryConditionCompileContext : public Queries::IQueryConditionCompileContext
	{
	public:
		explicit FEnvironmentQueryConditionCompileContext(FEnvironment& InEnvironment)
			: Environment(InEnvironment)
		{}
		FEnvironmentQueryConditionCompileContext(const FEnvironmentQueryConditionCompileContext&) = delete;
		FEnvironmentQueryConditionCompileContext(FEnvironmentQueryConditionCompileContext&&) = delete;
		FEnvironmentQueryConditionCompileContext& operator=(const FEnvironmentQueryConditionCompileContext&) = delete;
		FEnvironmentQueryConditionCompileContext& operator=(FEnvironmentQueryConditionCompileContext&&) = delete;
		
		virtual const UScriptStruct* GenerateDynamicColumn(const FDynamicColumnDescription& Description) const override
		{
			return Environment.GenerateDynamicColumn(*Description.TemplateType, Description.Identifier);
		}
	private:
		FEnvironment& Environment;
	};

	struct FTedsQueryArchetypeMatchOverride
	{
		bool Match(const FMassArchetypeCompositionDescriptor& CompositionDescriptor) const;
		
		FExtendedQueryStore::Handle QueryHandle;
		FEnvironment& Environment;
	};

	static void FORCENOINLINE PrintQueryCompositionToLog(const FMassArchetypeCompositionDescriptor& CompositionDescriptor)
	{
		TStringBuilder<512> StringBuilder;
		StringBuilder.Append( TEXT("Composition Descriptor:\nColumns: {"));
		{
			int32 FragmentsProcessed = 0;
			const FMassFragmentBitSet FragmentBitSet = CompositionDescriptor.Get<FMassFragmentBitSet>();
			const int32 FragmentCount = FragmentBitSet.CountStoredTypes();
			for(auto IndexIterator = FragmentBitSet.GetIndexIterator(); IndexIterator; ++IndexIterator)
			{
				StringBuilder.Append(FragmentBitSet.GetTypeAtIndex(*IndexIterator)->GetName());
				if (FragmentsProcessed != FragmentCount - 1)
				{
					StringBuilder.Append(TEXT(", "));
				}
				++FragmentsProcessed;
			}
			StringBuilder.Append(TEXT("}\n"));
		}
		{
			int32 TagsProcessed = 0;
			const FMassTagBitSet TagBitSet = CompositionDescriptor.Get<FMassTagBitSet>();
			const int32 TagsCount = TagBitSet.CountStoredTypes();
			StringBuilder.Append(TEXT("Tags: {"));
			for(auto IndexIterator = TagBitSet.GetIndexIterator(); IndexIterator; ++IndexIterator)
			{
				StringBuilder.Append(TagBitSet.GetTypeAtIndex(*IndexIterator)->GetName());
				if (TagsProcessed != TagsCount - 1)
				{
					StringBuilder.Append(TEXT(", "));
				}
				++TagsProcessed;
			}
			StringBuilder.Append(TEXT("}"));
		}
		UE_LOGF(LogEditorDataStorage, Error, "%ls", StringBuilder.ToString());
	}
	
	bool FTedsQueryArchetypeMatchOverride::Match(const FMassArchetypeCompositionDescriptor& CompositionDescriptor) const
	{
		FExtendedQuery* Query = Environment.GetQueryStore().Get(QueryHandle);
		if (UNLIKELY(!ensureAlwaysMsgf(Query != nullptr, TEXT("Invalid query discovered in Match function"))))
		{
			PrintQueryCompositionToLog(CompositionDescriptor);
			// Failing this is likely a result of subqueries of a processor being unregistered without the processor being unregistered.
			return false;
		}
		if (Query->Description.Conditions)
		{
			TArray<TWeakObjectPtr<const UScriptStruct>, TInlineAllocator<64>> ColumnTypes;
			FMassFragmentBitSet FragmentBitSet = CompositionDescriptor.Get<FMassFragmentBitSet>();
			auto FragmentIterator = FragmentBitSet.GetIndexIterator(true);
			while (FragmentIterator)
			{
				const UScriptStruct* FragmentStruct = FragmentBitSet.GetTypeAtIndex(*FragmentIterator);
				ColumnTypes.Add(FragmentStruct);
				++FragmentIterator;
			}
			
			// Check if the archetype matches the selected columns
			for (int32 Index = 0, SelectionTypesCount = Query->Description.SelectionTypes.Num(); Index < SelectionTypesCount; ++Index)
			{
				if (Query->Description.SelectionAccessTypes[Index] != EQueryAccessType::OptionalReadOnly)
				{
					if(!ColumnTypes.Contains(Query->Description.SelectionTypes[Index]))
					{
						// Archetype doesn't have the required selected column, so reject the archetype outright.
						return false;
					}
				}
			}

			// Then ensure it also matches the query conditions
			const FMassTagBitSet TagBitSet = CompositionDescriptor.Get<FMassTagBitSet>();
			auto TagIterator = TagBitSet.GetIndexIterator(true);
			while (TagIterator)
			{
				const UScriptStruct* TagStruct = TagBitSet.GetTypeAtIndex(*TagIterator);
				ColumnTypes.Add(TagStruct);
				++TagIterator;
			}

			constexpr bool bAvailableColumnsAreSorted = false;
			const bool bMatchesQueryConditions = Query->Description.Conditions->Verify(MakeArrayView(ColumnTypes.GetData(),  ColumnTypes.Num()), bAvailableColumnsAreSorted);

			return bMatchesQueryConditions;
		}
		else
		{
			return Query->GetQuery()->DoesArchetypeMatchRequirements(CompositionDescriptor);
		}

	}
	
	FExtendedQueryStore::FExtendedQueryStore(FDynamicColumnGenerator& InDynamicColumnGenerator)
		: DynamicColumnGenerator(&InDynamicColumnGenerator)
	{
	}

	FExtendedQueryStore::Handle FExtendedQueryStore::RegisterQuery(
		FQueryDescription Query,
		FEnvironment& Environment,
		FMassEntityManager& EntityManager,
		FMassProcessingPhaseManager& PhaseManager)
	{
		FExtendedQueryStore::Handle Result = Queries.Emplace();
		FExtendedQuery& StoredQuery = GetMutableChecked(Result);
		StoredQuery.Description = MoveTemp(Query);

		FMassEntityQuery& NativeQuery = SetupNativeQuery(StoredQuery.Description, StoredQuery, Environment);

		bool bContinueSetup =              SetupDynamicColumns(StoredQuery.Description, Environment);
		bContinueSetup = bContinueSetup && SetupHierarchyDependencies(StoredQuery.Description, Environment);
		bContinueSetup = bContinueSetup && SetupSelectedColumns(StoredQuery.Description, NativeQuery);
		bContinueSetup = bContinueSetup && SetupChunkFilters(Result, StoredQuery.Description, Environment, NativeQuery);
		bContinueSetup = bContinueSetup && SetupConditions(StoredQuery.Description, Environment, NativeQuery);
		bContinueSetup = bContinueSetup && SetupDependencies(StoredQuery.Description, NativeQuery);
		bContinueSetup = bContinueSetup && SetupTickGroupDefaults(StoredQuery.Description);
		bContinueSetup = bContinueSetup && SetupProcessors(Result, StoredQuery, Environment, EntityManager, PhaseManager);
		bContinueSetup = bContinueSetup && SetupActivatable(Result, StoredQuery);

		if (!bContinueSetup)
		{
			// This will also make the handle invalid.
			Queries.Remove(Result);
		}
		else
		{
			if (Private::bUseTedsMatchFunction)
			{
				const FTedsQueryArchetypeMatchOverride Context
				{
					.QueryHandle = Result,
					.Environment = Environment
				};

				NativeQuery.SetArchetypeMatchOverride(Context);
			}
		}

		return Result;
	}

	void FExtendedQueryStore::UnregisterQuery(Handle Query, FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager)
	{
		if (FExtendedQuery* QueryData = Get(Query))
		{
			UnregisterQueryData(Query, *QueryData, EntityManager, PhaseManager);
			Queries.Remove(Query);
		}
	}

	void FExtendedQueryStore::Clear(FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager)
	{
		TickGroupDescriptions.Empty();

		Queries.ListAliveEntries([this, &EntityManager, &PhaseManager](Handle Query, FExtendedQuery& QueryData)
			{
				if (QueryData.Processor && QueryData.Processor->IsA<UTypedElementQueryObserverCallbackAdapterProcessorBase>())
				{
					// Observers can't be unregistered at this point, so skip these for now.
					return;
				}
				UnregisterQueryData(Query, QueryData, EntityManager, PhaseManager);
			});
	}

	void FExtendedQueryStore::RegisterTickGroup(FName GroupName, EQueryTickPhase Phase,
		FName BeforeGroup, FName AfterGroup, EExecutionMode ExecutionMode)
	{
		FTickGroupDescription& Group = TickGroupDescriptions.FindOrAdd({ GroupName, Phase });

		if (!BeforeGroup.IsNone() && Group.BeforeGroups.Find(BeforeGroup) == INDEX_NONE)
		{
			Group.BeforeGroups.Add(BeforeGroup);
		}

		if (!AfterGroup.IsNone() && Group.AfterGroups.Find(AfterGroup) == INDEX_NONE)
		{
			Group.AfterGroups.Add(AfterGroup);
		}

		Group.ExecutionMode = ExecutionMode;
	}

	void FExtendedQueryStore::UnregisterTickGroup(FName GroupName, EQueryTickPhase Phase)
	{
		TickGroupDescriptions.Remove({ GroupName, Phase });
	}

	FExtendedQuery* FExtendedQueryStore::Get(Handle Entry)
	{
		return IsAlive(Entry) ? &Queries.Get(Entry) : nullptr;
	}

	FExtendedQuery* FExtendedQueryStore::GetMutable(Handle Entry)
	{
		return IsAlive(Entry) ? &Queries.GetMutable(Entry) : nullptr;
	}

	const FExtendedQuery* FExtendedQueryStore::Get(Handle Entry) const
	{
		return IsAlive(Entry) ? &Queries.Get(Entry) : nullptr;
	}

	FExtendedQuery& FExtendedQueryStore::GetChecked(Handle Entry)
	{
		return Queries.Get(Entry);
	}

	FExtendedQuery& FExtendedQueryStore::GetMutableChecked(Handle Entry)
	{
		return Queries.GetMutable(Entry);
	}

	const FExtendedQuery& FExtendedQueryStore::GetChecked(Handle Entry) const
	{
		return Queries.Get(Entry);
	}

	const FQueryDescription& FExtendedQueryStore::GetQueryDescription(Handle Query) const
	{
		const FExtendedQuery* QueryData = Get(Query);
		return QueryData ? QueryData->Description : EmptyDescription;
	}

	uint64 FExtendedQueryStore::CalculateQueryTablesTopologyHash(Handle Query) const
	{
		const FExtendedQuery* QueryData = Get(Query);
		// Const cast used because CalculateArchetypeVersionHash is not const as it may need to redetermine what
		// archetypes match with the native query. This is however conceptually const from TEDS' point of view.
		return
			QueryData
			? const_cast<FExtendedQuery*>(QueryData)->NativeQuery.CalculateArchetypeVersionHash()
			: 0;
	}

	bool FExtendedQueryStore::IsAlive(Handle Entry) const
	{
		return Queries.IsAlive(Entry);
	}

	void FExtendedQueryStore::ListAliveEntries(const ListAliveEntriesConstCallback& Callback) const
	{
		Queries.ListAliveEntries(Callback);
	}

	void FExtendedQueryStore::UpdateActivatableQueries()
	{
		// Update activatable counts and remove any queries that have completed.
		for (Handle Query : ActiveActivatables)
		{
			FExtendedQuery& QueryData = Queries.GetMutable(Query);
			QueryData.Description.Callback.Active = false;
		}
		ActiveActivatables.Reset();

		// Queue up the next batch of activatables.
		for (Handle Query : PendingActivatables)
		{
			FExtendedQuery& QueryData = Queries.GetMutable(Query);
			if (!QueryData.Description.Callback.Active)
			{
				QueryData.Description.Callback.Active = true;
				ActiveActivatables.Add(Query);

				if (QueryData.Description.Callback.Type != EQueryCallbackType::PhasePreparation &&
					QueryData.Description.Callback.Type != EQueryCallbackType::PhaseFinalization)
				{
					if (ensure(QueryData.Processor.IsValid()))
					{
						QueryData.Processor->MakeOneShot();
					}
				}
			}
		}
		PendingActivatables.Reset();
	}

	void FExtendedQueryStore::ActivateQueries(FName ActivationName)
	{
		for (TMultiMap<FName, Handle>::TKeyIterator QueryIt = ActivatableMapping.CreateKeyIterator(ActivationName); QueryIt; ++QueryIt)
		{
			Handle Query = QueryIt.Value();
			if (Queries.IsAlive(Query))
			{
#if DO_ENSURE
				FExtendedQuery& QueryData = Queries.GetMutable(Query);
				checkf(!QueryData.Description.Callback.ActivationName.IsNone(),
					TEXT("Attempting to enable the query '%s' which isn't activatable."), *QueryData.Description.Callback.Name.ToString());
#endif
				PendingActivatables.AddUnique(Query);
			}
		}
	}


	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager, Handle Query)
	{
		using ActionType = FQueryDescription::EActionType;
		using CompletionType = FQueryResult::ECompletion;

		FQueryResult Result;

		if (FExtendedQuery* QueryData = Get(Query))
		{
			switch (QueryData->Description.Action)
			{
			case ActionType::None:
				Result.Completed = CompletionType::Fully;
				break;
			case ActionType::Select:
				// Fall through: There's nothing to callback to, so only return the total count.
			case ActionType::Count:
				Result.Count = QueryData->NativeQuery.GetNumMatchingEntities();
				Result.Completed = CompletionType::Fully;
				break;
			default:
				Result.Completed = CompletionType::Unsupported;
				break;
			}
		}
		else
		{
			Result.Completed = CompletionType::Unavailable;
		}

		return Result;
	}

	template<typename CallbackReference, typename ExecutionFlagsType>
	FQueryResult FExtendedQueryStore::RunQueryCallbackCommon(
		FMassEntityManager& EntityManager,
		FEnvironment& Environment,
		FMassExecutionContext* ParentContext,
		Handle Query,
		ExecutionFlagsType ExecutionFlags,
		CallbackReference Callback)
	{
		FQueryResult Result;
		if (FExtendedQuery* QueryData = Get(Query))
		{
			switch (QueryData->Description.Action)
			{
			case FQueryDescription::EActionType::None:
				Result.Completed = FQueryResult::ECompletion::Fully;
				break;
			case FQueryDescription::EActionType::Select:
				if (!QueryData->Processor.IsValid())
				{
					if constexpr (std::is_same_v<CallbackReference, DirectQueryCallbackRef>)
					{
						static_assert(std::is_same_v<ExecutionFlagsType, EDirectQueryExecutionFlags>, 
							"Running a direct query requires 'EDirectQueryExecutionFlags' flags.");
						Result = FTypedElementQueryProcessorData::Execute(
							Callback, QueryData->Description, QueryData->NativeQuery, EntityManager, Environment, ExecutionFlags);
					}
					else if constexpr (std::is_same_v<CallbackReference, Queries::TQueryFunction<void>>)
					{
						if (CheckCompatibility(QueryData->Description, Callback))
						{
							static_assert(std::is_same_v<ExecutionFlagsType, ERunQueryFlags>, 
								"Calling run query requires 'ERunQueryFlags' flags.");
							Result = FTypedElementQueryProcessorData::Execute(
								Callback, QueryData->Description, QueryData->NativeQuery, EntityManager, Environment, ExecutionFlags);
						}
						else
						{
							Result.Completed = FQueryResult::ECompletion::Incompatible;
						}	
					}
					else
					{
						Result = FTypedElementQueryProcessorData::Execute(
							Callback, QueryData->Description, QueryData->NativeQuery, EntityManager, Environment, *ParentContext);
					}
				}
				else
				{
					Result.Completed = FQueryResult::ECompletion::Unsupported;
				}
				break;
			case FQueryDescription::EActionType::Count:
				// Only the count is requested so no need to trigger the callback.
				Result.Count = QueryData->NativeQuery.GetNumMatchingEntities();
				Result.Completed = FQueryResult::ECompletion::Fully;
				break;
			default:
				Result.Completed = FQueryResult::ECompletion::Unsupported;
				break;
			}
		}
		else
		{
			Result.Completed = FQueryResult::ECompletion::Unavailable;
		}

		return Result;
	}

	FQueryResult FExtendedQueryStore::RunQueryCallbackCommon(
		FMassEntityManager& EntityManager,
		FEnvironment& Environment,
		FMassExecutionContext* ParentContext,
		Handle Query,
		ERunQueryFlags ExecutionFlags,
		const Queries::TConstQueryFunction<void>& Callback) const
	{
		FQueryResult Result;
		if (const FExtendedQuery* QueryData = Get(Query))
		{
			switch (QueryData->Description.Action)
			{
			case FQueryDescription::EActionType::None:
				Result.Completed = FQueryResult::ECompletion::Fully;
				break;
			case FQueryDescription::EActionType::Select:
				if (!QueryData->Processor.IsValid())
				{
					if (CheckCompatibility(QueryData->Description, Callback))
					{
						Result = FTypedElementQueryProcessorData::Execute(
							Callback, QueryData->Description, QueryData->NativeQuery, EntityManager, Environment, ExecutionFlags);
					}
					else
					{
						Result.Completed = FQueryResult::ECompletion::Incompatible;
					}
				}
				else
				{
					Result.Completed = FQueryResult::ECompletion::Unsupported;
				}
				break;
			case FQueryDescription::EActionType::Count:
				// Only the count is requested so no need to trigger the callback.
				Result.Count = QueryData->NativeQuery.GetNumMatchingEntities();
				Result.Completed = FQueryResult::ECompletion::Fully;
				break;
			default:
				Result.Completed = FQueryResult::ECompletion::Unsupported;
				break;
			}
		}
		else
		{
			Result.Completed = FQueryResult::ECompletion::Unavailable;
		}

		return Result;
	}

	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager,
		FEnvironment& Environment, Handle Query, EDirectQueryExecutionFlags DirectExecutionFlags,
		DirectQueryCallbackRef Callback)
	{
		return RunQueryCallbackCommon(EntityManager, Environment, nullptr, Query, DirectExecutionFlags, Callback);
	}

	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager,
		FEnvironment& Environment, Handle Query, ERunQueryFlags Flags,
		const Queries::TQueryFunction<void>& Callback)
	{
		return RunQueryCallbackCommon(EntityManager, Environment, nullptr, Query, Flags, Callback);
	}

	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager,
		FEnvironment& Environment, Handle Query, ERunQueryFlags Flags,
		const Queries::TConstQueryFunction<void>& Callback) const
	{
		return RunQueryCallbackCommon(EntityManager, Environment, nullptr, Query, Flags, Callback);
	}
	
	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager,
		FEnvironment& Environment, FMassExecutionContext& ParentContext, Handle Query,
		SubqueryCallbackRef Callback)
	{
		return RunQueryCallbackCommon(EntityManager, Environment, &ParentContext, Query, 
			EDirectQueryExecutionFlags::Default, Callback);
	}

	FQueryResult FExtendedQueryStore::RunQuery(FMassEntityManager& EntityManager,
		FEnvironment& Environment, FMassExecutionContext& ParentContext, Handle Query, RowHandle Row,
		SubqueryCallbackRef Callback)
	{
		using ActionType = FQueryDescription::EActionType;
		using CompletionType = FQueryResult::ECompletion;

		FQueryResult Result;

		if (FExtendedQuery* QueryData = Get(Query))
		{
			switch (QueryData->Description.Action)
			{
			case ActionType::None:
				Result.Completed = CompletionType::Fully;
				break;
			case ActionType::Select:
				if (!QueryData->Processor.IsValid())
				{
					Result = FTypedElementQueryProcessorData::Execute(
						Callback, QueryData->Description, Row, QueryData->NativeQuery, EntityManager, Environment, ParentContext);
				}
				else
				{
					Result.Completed = CompletionType::Unsupported;
				}
				break;
			case ActionType::Count:
				// Only the count is requested so no need to trigger the callback.
				Result.Count = 1;
				Result.Completed = CompletionType::Fully;
				break;
			default:
				Result.Completed = CompletionType::Unsupported;
				break;
			}
		}
		else
		{
			Result.Completed = CompletionType::Unavailable;
		}

		return Result;
	}
	void FExtendedQueryStore::RunPhasePreambleQueries(FMassEntityManager& EntityManager,
		FEnvironment& Environment, EQueryTickPhase Phase, float DeltaTime)
	{
		RunPhasePreOrPostAmbleQueries(EntityManager, Environment, Phase, DeltaTime,
			PhasePreparationQueries[static_cast<QueryTickPhaseType>(Phase)]);
	}

	void FExtendedQueryStore::RunPhasePostambleQueries(FMassEntityManager& EntityManager,
		FEnvironment& Environment, EQueryTickPhase Phase, float DeltaTime)
	{
		RunPhasePreOrPostAmbleQueries(EntityManager, Environment, Phase, DeltaTime,
			PhaseFinalizationQueries[static_cast<QueryTickPhaseType>(Phase)]);
	}

	void FExtendedQueryStore::NotifyNewDynamicColumn(const FDynamicColumnGeneratorInfo& GeneratedColumnInfo)
	{
		if (!Private::bUseTedsMatchFunction)
		{
			return;
		}
		// Find all queries that use the template as a condition and add the newly generated type
		// to the query
		Queries.ListAliveEntries([&](QueryStore::Handle Handle, FExtendedQuery& Query)
		{
			if (!Query.Description.Conditions.IsSet())
			{
				// Generic dynamic column template queries only supported for when query conditions used
				return;
			}
			// Handle dynamic Selected columns
			for (int32 Index = 0, End = Query.Description.DynamicSelectionTypes.Num(); Index < End; ++Index)
			{
				// Only consider query descriptions that have a dynamic selection type that is supposed to match all
				// columns 
				const bool bIsTemplateSelectionType = Query.Description.DynamicSelectionTypes[Index].Identifier.IsNone();
				const bool bTemplateMatches = Query.Description.DynamicSelectionTypes[Index].TemplateType == GeneratedColumnInfo.Template;
				if (bIsTemplateSelectionType && bTemplateMatches)
				{
					const UScriptStruct* GeneratedColumnType = GeneratedColumnInfo.Type;
					const EMassFragmentAccess AccessMode = ConvertToNativeAccessType(Query.Description.DynamicSelectionAccessTypes[Index]);
					const EMassFragmentPresence PresenceMode = EMassFragmentPresence::Any;
					if (GeneratedColumnType->IsChildOf<FEditorDataStorageColumn>())
					{
						Query.GetQuery()->AddRequirement(GeneratedColumnType, AccessMode, PresenceMode);
					}
					else if (GeneratedColumnType->IsChildOf<FEditorDataStorageTag>())
					{
						Query.GetQuery()->AddTagRequirement(*GeneratedColumnType, PresenceMode);
					}
				}
			}

			// Handle dynamic conditional columns
			for (int32 Index = 0, End = Query.Description.DynamicConditionDescriptions.Num(); Index < End; ++Index)
			{
				// Only consider query descriptions that have a dynamic selection type that is supposed to match all
				// columns 
				const bool bIsTemplateSelectionType = Query.Description.DynamicConditionDescriptions[Index].Identifier.IsNone();
				const bool bTemplateMatches = Query.Description.DynamicConditionDescriptions[Index].TemplateType == GeneratedColumnInfo.Template;
				if (bIsTemplateSelectionType && bTemplateMatches)
				{
					const UScriptStruct* GeneratedColumnType = GeneratedColumnInfo.Type;
					const EMassFragmentAccess AccessMode = EMassFragmentAccess::None;
					const EMassFragmentPresence PresenceMode = EMassFragmentPresence::Any;

					if (GeneratedColumnType->IsChildOf<FEditorDataStorageColumn>())
					{
						Query.GetQuery()->AddRequirement(GeneratedColumnType, AccessMode, PresenceMode);
					}
					else if (GeneratedColumnType->IsChildOf<FEditorDataStorageTag>())
					{
						Query.GetQuery()->AddTagRequirement(*GeneratedColumnType, PresenceMode);
					}
				}
			}
		});
	}

	void FExtendedQueryStore::DebugPrintQueryCallbacks(FOutputDevice& Output) const
	{
		Output.Log(TEXT("The Typed Elements Data Storage has the following query callbacks:"));
		Queries.ListAliveEntries(
			[&Output](Handle QueryHandle, const FExtendedQuery& Query)
			{
				if (Query.Processor)
				{
					Output.Logf(TEXT("    [%s] %s"),
						IsValid(Query.Processor.Get()) ? TEXT("Valid") : TEXT("Invalid"),
						*(Query.Processor->GetProcessorName()));
				}
			});

		for (QueryTickPhaseType PhaseId = 0; PhaseId < static_cast<QueryTickPhaseType>(MaxTickPhase); ++PhaseId)
		{
			for (Handle QueryHandle : PhasePreparationQueries[PhaseId])
			{
				const FExtendedQuery& QueryData = GetChecked(QueryHandle);
				Output.Logf(TEXT("    [Valid] %s [Editor Phase Preamble]"), *QueryData.Description.Callback.Name.ToString());
			}
			for (Handle QueryHandle : PhaseFinalizationQueries[PhaseId])
			{
				const FExtendedQuery& QueryData = GetChecked(QueryHandle);
				Output.Logf(TEXT("    [Valid] %s [Editor Phase Postamble]"), *QueryData.Description.Callback.Name.ToString());
			}
		}

		Output.Log(TEXT("End of Typed Elements Data Storage query callback list."));
	}

	bool FExtendedQueryStore::SetupDynamicColumns(FQueryDescription& Query, FEnvironment& Environment)
	{
		auto IsDynamicColumnTemplate = [](const UScriptStruct& Struct, const FName& Identifier)->bool
		{
			return Identifier.IsNone();
		};
		
		const int32 SelectionCount = Query.DynamicSelectionTypes.Num();

		for (int32 Index = 0, End = SelectionCount; Index < End; ++Index)
		{
			// Convert the dynamic elements into concrete types
			const FDynamicColumnDescription& Description = Query.DynamicSelectionTypes[Index];
			if (!ensureMsgf(Description.TemplateType, TEXT("Null template type for dynamic column")))
			{
				continue;
			}
			const UScriptStruct* DynamicColumnType = Environment.GenerateDynamicColumn(*Description.TemplateType, Description.Identifier);

			const EQueryAccessType AccessType = Query.DynamicSelectionAccessTypes[Index];
			const FColumnMetaData::EFlags MetadataFlags = Query.DynamicSelectionMetaData[Index];
			if (ensureMsgf(DynamicColumnType, TEXT("Provided query selection type can not be null.")))
			{
				Query.SelectionTypes.Emplace(DynamicColumnType);
				Query.SelectionAccessTypes.Emplace(AccessType);
				Query.SelectionMetaData.Emplace(FColumnMetaData(DynamicColumnType, MetadataFlags));
			}
		}

		for (int32 Index = 0, End = Query.DynamicConditionDescriptions.Num(); Index < End; ++Index)
		{
			const FDynamicColumnDescription& Description = Query.DynamicConditionDescriptions[Index];
			if (!ensureMsgf(Description.TemplateType, TEXT("Null template type for dynamic column")))
			{
				continue;
			}
			if (IsDynamicColumnTemplate(*Description.TemplateType, Description.Identifier))
			{
				FQueryDescription::EOperatorType Operation = Query.DynamicConditionOperations[Index];
				DynamicColumnGenerator->ForEachDynamicColumn(*Description.TemplateType, [&Query, Operation](const FDynamicColumnGeneratorInfo& Info)
				{
					Query.ConditionTypes.Add(Operation);
					Query.ConditionOperators.Add(FQueryDescription::FOperator
					{
						.Type = Info.Type
					});
				});
			}
			else
			{
				const UScriptStruct* DynamicColumnType = DynamicColumnGenerator->GenerateColumn(*Description.TemplateType, Description.Identifier).Type;
				FQueryDescription::EOperatorType Operation = Query.DynamicConditionOperations[Index];
				Query.ConditionTypes.Add(Operation);
				Query.ConditionOperators.Add(FQueryDescription::FOperator
				{
					.Type = DynamicColumnType
				});
			}
		}

		return true;
	}

	bool FExtendedQueryStore::SetupHierarchyDependencies(FQueryDescription& Query, FEnvironment& Environment)
	{
		if (!Query.Hierarchy.IsNone())
		{
			FHierarchyHandle Handle = Environment.GetHierarchyRegistrar().FindHierarchyByName(Query.Hierarchy);
			const FTedsHierarchyAccessInterface* HierarchyAccessInterface = Environment.GetHierarchyRegistrar().GetAccessInterface(Handle);
			ensureMsgf(
				HierarchyAccessInterface,
				TEXT("Hierarchy named '%s' not registered. Query will return no hierarchy information. Register query after registering hierarchy"),
				*Query.Hierarchy.ToString());
			if (HierarchyAccessInterface)
			{
				// Must bind the HierarchyDataColumn for Read to allow for GetParent.
				// Relations-backed hierarchies store data on relation rows rather than
				// participant rows, so GetHierarchyDataColumnType() returns null for them.
				// Skip adding a null type -- the query still works for those backends.
				// Note: Optional used to allow a query to match rows that are not currently participating in a relationship (ie. childless root objects)
				const UScriptStruct* DataColumnType = HierarchyAccessInterface->GetHierarchyDataColumnType();
				if (DataColumnType)
				{
					Query.SelectionTypes.Emplace(DataColumnType);
					Query.SelectionAccessTypes.Emplace(EQueryAccessType::OptionalReadOnly);
					Query.SelectionMetaData.Emplace(FColumnMetaData(DataColumnType, FColumnMetaData::EFlags::None));
				}
			}
		}
		return true;
	}

	FMassEntityQuery& FExtendedQueryStore::SetupNativeQuery(
		FQueryDescription& Query, FExtendedQuery& StoredQuery, FEnvironment& Environment)
	{
		/**
		 * Mass verifies that queries that are used by processors are on the processor themselves. It does this by taking the address of the query
		 * and seeing if it's within the start and end address of the processor. When a dynamic array is used those addresses are going to be
		 * elsewhere, so the two options are to store a single fixed size array on a processor or have multiple instances. With Mass' queries being
		 * not an insignificant size it's preferable to have several variants with queries to allow the choice for the minimal size. Unfortunately
		 * UHT doesn't allow for templates so it had to be done in an explicit way.
		 */

		StoredQuery.QueryReference = nullptr;

		if (Query.Action == FQueryDescription::EActionType::Select)
		{
			switch (Query.Callback.Type)
			{
			case EQueryCallbackType::None:
				break;
			case EQueryCallbackType::Processor:
			{
				UTypedElementQueryProcessorCallbackAdapterProcessorBase* Processor;
				switch (Query.Subqueries.Num())
				{
				case 0:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessor>();
					break;
				case 1:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith1Subquery>();
					break;
				case 2:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith2Subqueries>();
					break;
				case 3:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith3Subqueries>();
					break;
				case 4:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith4Subqueries>();
					break;
				case 5:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith5Subqueries>();
					break;
				case 6:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith6Subqueries>();
					break;
				case 7:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith7Subqueries>();
					break;
				case 8:
					Processor = NewObject<UTypedElementQueryProcessorCallbackAdapterProcessorWith8Subqueries>();
					break;
				default:
					checkf(false, TEXT("The current Typed Elements Data Storage backend doesn't support %i subqueries per processor query."),
						Query.Subqueries.Num());
					StoredQuery.NativeQuery.Initialize(Environment.GetMassEntityManager().AsShared());
					return StoredQuery.NativeQuery;
				}
				StoredQuery.Processor.Reset(Processor);
				StoredQuery.QueryReference = &Processor->GetQuery();
				StoredQuery.QueryReference->Initialize(Environment.GetMassEntityManager().AsShared());
				return Processor->GetQuery();
			}
			case EQueryCallbackType::ObserveAdd:
				// Fall-through
			case EQueryCallbackType::ObserveRemove:
			{
				UTypedElementQueryObserverCallbackAdapterProcessorBase* Observer;
				switch (Query.Subqueries.Num())
				{
				case 0:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessor>();
					break;
				case 1:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith1Subquery>();
					break;
				case 2:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith2Subqueries>();
					break;
				case 3:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith3Subqueries>();
					break;
				case 4:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith4Subqueries>();
					break;
				case 5:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith5Subqueries>();
					break;
				case 6:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith6Subqueries>();
					break;
				case 7:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith7Subqueries>();
					break;
				case 8:
					Observer = NewObject<UTypedElementQueryObserverCallbackAdapterProcessorWith8Subqueries>();
					break;
				default:
					checkf(false, TEXT("The current Typed Elements Data Storage backend doesn't support %i subqueries per observer query."),
						Query.Subqueries.Num());
					return StoredQuery.NativeQuery;
				}
				StoredQuery.Processor.Reset(Observer);
				StoredQuery.QueryReference = &Observer->GetQuery();
				StoredQuery.QueryReference->Initialize(Environment.GetMassEntityManager().AsShared());
				return Observer->GetQuery();
			}
			case EQueryCallbackType::PhasePreparation:
				break;
			case EQueryCallbackType::PhaseFinalization:
				break;
			default:
				checkf(false, TEXT("Unsupported query callback type %i."), static_cast<int>(Query.Callback.Type));
				break;
			}
		}

		StoredQuery.NativeQuery.Initialize(Environment.GetMassEntityManager().AsShared());
		return StoredQuery.NativeQuery;
	}

	bool FExtendedQueryStore::SetupSelectedColumns(FQueryDescription& Query, FMassEntityQuery& NativeQuery)
	{
		switch (Query.Action)
		{
		case FQueryDescription::EActionType::None:
			return true;
		case FQueryDescription::EActionType::Select:
		{
			const int32 SelectionCount = Query.SelectionTypes.Num();
			if (ensureMsgf(SelectionCount == Query.SelectionAccessTypes.Num(),
				TEXT("The number of query selection types (%i) doesn't match the number of selection access types (%i)."),
				SelectionCount, Query.SelectionAccessTypes.Num()))
			{
				for (int SelectionIndex = 0; SelectionIndex < SelectionCount; ++SelectionIndex)
				{
					TWeakObjectPtr<const UScriptStruct>& Type = Query.SelectionTypes[SelectionIndex];
					EQueryAccessType AccessType = Query.SelectionAccessTypes[SelectionIndex];
					if (ensureMsgf(Type.IsValid(), TEXT("Provided query selection type can not be null.")) &&
						ensureMsgf(
							Type->IsChildOf(FColumn::StaticStruct()) ||
							UE::Mass::IsA<FMassFragment>(Type.Get()),
							TEXT("Provided query selection type '%s' is not based on FColumn or another supported base type."),
							*Type->GetStructPathName().ToString()))
					{
						NativeQuery.AddRequirement(Type.Get(), ConvertToNativeAccessType(AccessType), ConvertToNativePresenceType(AccessType));
					}
					else
					{
						return false;
					}
				}
				return true;
			}
			return false;
		}
		case FQueryDescription::EActionType::Count:
		{
			bool bIsSelectionEmpty = Query.SelectionTypes.IsEmpty();
			bool bIsAccessTypesEmpty = Query.SelectionAccessTypes.IsEmpty();
			checkf(bIsSelectionEmpty, TEXT("Count queries for the Typed Elements Data Storage can't have entries for selection."));
			checkf(bIsAccessTypesEmpty, TEXT("Count queries for the Typed Elements Data Storage can't have entries for selection."));
			return bIsSelectionEmpty && bIsAccessTypesEmpty;
		}
		default:
			checkf(false, TEXT("Unexpected query action: %i."), static_cast<int32>(Query.Action));
			return false;
		}
	}

	bool FExtendedQueryStore::SetupConditions(FQueryDescription& Query, FEnvironment& Environment, FMassEntityQuery& NativeQuery)
	{
		if (Private::bUseTedsMatchFunction && Query.Conditions.IsSet())
		{
			// Will ignore the All/Any/None conditions if in this branch, but let the user know via an ensure anyways
			ensureAlwaysMsgf(Query.ConditionTypes.IsEmpty(), TEXT("Queries do not support using both QueryConditions and All/Any/None conditions"));
				
			// Use the QueryConditions to setup the condition requirements for Mass
			Queries::FConditions& Conditions = *Query.Conditions;
			const FEnvironmentQueryConditionCompileContext CompileContext(Environment);
			Conditions.Compile(CompileContext);

			const TConstArrayView<TWeakObjectPtr<const UScriptStruct>>& Columns = Conditions.GetColumns();

			// Note: need to remove duplicates to appease Mass
			TSet<const UScriptStruct*, DefaultKeyFuncs<const UScriptStruct*>, TInlineSetAllocator<128>> DedupedColumns;
			for (const TWeakObjectPtr<const UScriptStruct>& Column : Columns)
			{
				// If any of the columns are Dynamic Column Templates, then expand them
				if (DynamicColumnGenerator->IsDynamicTemplate(*Column.Get()))
				{
					DynamicColumnGenerator->ForEachDynamicColumn(*Column.Get(), [&DedupedColumns ](const FDynamicColumnGeneratorInfo& Info)
					{
						DedupedColumns.Add(Info.Type);
					});
				}
				else
				{
					DedupedColumns.Add(Column.Get());
				}
			}

			// Everything is going to be Any, Mass' internals checks requirements 
			// We override the archetype matching function anyways so Mass will not use the All, Any or None
			const EMassFragmentPresence Presence = EMassFragmentPresence::Any;
			for (const UScriptStruct* Column : DedupedColumns)
			{
				if (Column->IsChildOf(FEditorDataStorageTag::StaticStruct()))
				{
					NativeQuery.AddTagRequirement(*Column, Presence);
				}
				else if (Column->IsChildOf(FEditorDataStorageColumn::StaticStruct()))
				{
					// Check that the requirement wasn't already added as a "Selected" column
					const bool IsSelected = Query.SelectionTypes.Contains(Column);
					if (ensureMsgf(!IsSelected, TEXT("Cannot add '%s' as a condition if it is a selected column"), *Column->GetName()))
					{
						NativeQuery.AddRequirement(Column, EMassFragmentAccess::None, Presence);
					}
					else
					{
						return false;
					}
				}
			}
			return true;
		}
		else
		{
			if (Query.ConditionTypes.IsEmpty())
			{
				return true;
			}
			
			if (ensureMsgf(Query.ConditionTypes.Num() == Query.ConditionOperators.Num(),
				TEXT("The types and operators for a typed element query have gone out of sync.")))
			{
				const FQueryDescription::FOperator* Operand = Query.ConditionOperators.GetData();
				for (FQueryDescription::EOperatorType Type : Query.ConditionTypes)
				{
					const EMassFragmentPresence Presence = ConvertToNativePresenceType(Type);
					
					if (UE::Mass::IsA<FMassTag>(Operand->Type.Get()))
					{
						NativeQuery.AddTagRequirement(*(Operand->Type), Presence);
					}
					else if (UE::Mass::IsA<FMassFragment>(Operand->Type.Get()))
					{
						NativeQuery.AddRequirement(Operand->Type.Get(), EMassFragmentAccess::None, Presence);
					}

					++Operand;
				}
				return true;
			}
			return false;
		}
	}

	bool FExtendedQueryStore::SetupChunkFilters(
		Handle QueryHandle,
		FQueryDescription& Query,
		FEnvironment& Environment,
		FMassEntityQuery& NativeQuery)
	{
		if (Query.ValueTags.IsEmpty())
		{
			return true;
		}

		Algo::SortBy(Query.ValueTags, [](const FQueryDescription::FValueTagData& ValueTagData)
			{
				return ValueTagData.Tag.GetName();
			}, FNameFastLess());
		// Check if there are any duplicate groups. Not yet supported until we can match multiple MatchTags
	FValueTag PreviousTag = Query.ValueTags[0].Tag;
		for (int32 Index = 1, End = Query.ValueTags.Num(); Index < End; ++Index)
		{
			if (Query.ValueTags[Index].Tag == PreviousTag)
			{
				return false;
			}
			PreviousTag = Query.ValueTags[Index].Tag;
		}

		struct FGroupTagPair
		{
			const UScriptStruct* ColumnType;
			FName Value;
		};

		TArray<FGroupTagPair> GroupTagPairsTemp;
		GroupTagPairsTemp.Reserve(Query.ValueTags.Num());
		for (int32 Index = 0, End = Query.ValueTags.Num(); Index < End; ++Index)
		{
			const UScriptStruct* ColumnType = Environment.GenerateColumnType(Query.ValueTags[Index].Tag);
			GroupTagPairsTemp.Emplace(FGroupTagPair
				{
					.ColumnType = ColumnType,
					.Value = Query.ValueTags[Index].MatchValue
				});
		}

		check(!GroupTagPairsTemp.IsEmpty());

		for (const FGroupTagPair& Element : GroupTagPairsTemp)
		{
			NativeQuery.AddConstSharedRequirement(Element.ColumnType);
		}

		auto ChunkFilterFunction = [GroupTagPairs = MoveTemp(GroupTagPairsTemp)](const FMassExecutionContext& MassContext) -> bool
		{
			for (const FGroupTagPair& GroupTagPair : GroupTagPairs)
			{
				const void* SharedFragmentData = MassContext.GetConstSharedFragmentPtr(*GroupTagPair.ColumnType);

				if (SharedFragmentData)
				{
					const FValueTagColumn* TagOverlay = static_cast<const FValueTagColumn*>(SharedFragmentData);
					// NAME_None will match any presence of the shared fragment
					// otherwise, match the specific tag only
					if (GroupTagPair.Value != NAME_None && TagOverlay->Value != GroupTagPair.Value)
					{
						return false;
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		};

		NativeQuery.SetChunkFilter(ChunkFilterFunction);
		return true;
	}

	bool FExtendedQueryStore::SetupDependencies(
		FQueryDescription& Query, FMassEntityQuery& NativeQuery)
	{
		const int32 DependencyCount = Query.DependencyTypes.Num();
		if (ensureMsgf(DependencyCount == Query.DependencyFlags.Num() && DependencyCount == Query.CachedDependencies.Num(),
			TEXT("The number of query dependencies (%i) doesn't match the number of dependency access types (%i) and/or cached dependencies count (%i)."),
			DependencyCount, Query.DependencyFlags.Num(), Query.CachedDependencies.Num()))
		{
			for (int32 DependencyIndex = 0; DependencyIndex < DependencyCount; ++DependencyIndex)
			{
				TWeakObjectPtr<const UClass>& Type = Query.DependencyTypes[DependencyIndex];
				if (ensureMsgf(Type.IsValid(), TEXT("Provided query dependency type can not be null.")) &&
					ensureMsgf(Type->IsChildOf<USubsystem>(), TEXT("Provided query dependency type '%s' is not based on USubSystem."),
						*Type->GetStructPathName().ToString()))
				{
					EQueryDependencyFlags Flags = Query.DependencyFlags[DependencyIndex];
					NativeQuery.AddSubsystemRequirement(
						const_cast<UClass*>(Type.Get()),
						EnumHasAllFlags(Flags, EQueryDependencyFlags::ReadOnly) ? EMassFragmentAccess::ReadOnly : EMassFragmentAccess::ReadWrite,
						EnumHasAllFlags(Flags, EQueryDependencyFlags::GameThreadBound));
				}
				else
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

	bool FExtendedQueryStore::SetupTickGroupDefaults(FQueryDescription& Query)
	{
		const FTickGroupDescription* TickGroup = TickGroupDescriptions.Find({ Query.Callback.Group, Query.Callback.Phase });
		if (TickGroup)
		{
			for (auto It = Query.Callback.BeforeGroups.CreateIterator(); It; ++It)
			{
				if (TickGroup->BeforeGroups.Contains(*It))
				{
					It.RemoveCurrentSwap();
				}
			}
			Query.Callback.BeforeGroups.Append(TickGroup->BeforeGroups);

			for (auto It = Query.Callback.AfterGroups.CreateIterator(); It; ++It)
			{
				if (TickGroup->AfterGroups.Contains(*It))
				{
					It.RemoveCurrentSwap();
				}
			}
			Query.Callback.AfterGroups.Append(TickGroup->AfterGroups);

			if (Query.Callback.ExecutionMode == EExecutionMode::Default)
			{
				Query.Callback.ExecutionMode = TickGroup->ExecutionMode;
			}
		}
		return true;
	}

	bool FExtendedQueryStore::SetupProcessors(Handle QueryHandle, FExtendedQuery& StoredQuery,
		FEnvironment& Environment, FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager)
	{
		// Register Phase processors locally.
		switch (StoredQuery.Description.Callback.Type)
		{
		case EQueryCallbackType::PhasePreparation:
			RegisterPreambleQuery(StoredQuery.Description.Callback.Phase, QueryHandle);
			break;
		case EQueryCallbackType::PhaseFinalization:
			RegisterPostambleQuery(StoredQuery.Description.Callback.Phase, QueryHandle);
			break;
		}

		// Register regular processors and observer with Mass.
		if (StoredQuery.Processor)
		{
			if (StoredQuery.Processor->IsA<UTypedElementQueryProcessorCallbackAdapterProcessorBase>())
			{
				if (static_cast<UTypedElementQueryProcessorCallbackAdapterProcessorBase*>(StoredQuery.Processor.Get())->
					ConfigureQueryCallback(StoredQuery, QueryHandle, *this, Environment))
				{
					PhaseManager.RegisterDynamicProcessor(*StoredQuery.Processor);
				}
				else
				{
					return false;
				}
			}
			else if (StoredQuery.Processor->IsA<UTypedElementQueryObserverCallbackAdapterProcessorBase>())
			{
				if (UTypedElementQueryObserverCallbackAdapterProcessorBase* Observer =
					static_cast<UTypedElementQueryObserverCallbackAdapterProcessorBase*>(StoredQuery.Processor.Get()))
				{
					Observer->ConfigureQueryCallback(StoredQuery, QueryHandle, *this, Environment);
					EntityManager.GetObserverManager().AddObserverInstance(Observer->GetObservedType(), Observer->GetObservedOperations(), Observer);
				}
				else
				{
					return false;
				}
			}
			else
			{
				checkf(false, TEXT("Query processor %s is of unsupported type %s."),
					*StoredQuery.Description.Callback.Name.ToString(), *StoredQuery.Processor->GetSparseClassDataStruct()->GetName());
				return false;
			}
		}
		return true;
	}

	bool FExtendedQueryStore::SetupActivatable(Handle QueryHandle, FExtendedQuery& StoredQuery)
	{
		if (!StoredQuery.Description.Callback.ActivationName.IsNone())
		{
			ActivatableMapping.Add(StoredQuery.Description.Callback.ActivationName, QueryHandle);

			if (StoredQuery.Description.Callback.Type != EQueryCallbackType::PhasePreparation &&
				StoredQuery.Description.Callback.Type != EQueryCallbackType::PhaseFinalization)
			{
				if (ensure(StoredQuery.Processor.IsValid()))
				{
					StoredQuery.Processor->MakeInactive();
				}
			}
		}
		return true;
	}

	EMassFragmentAccess FExtendedQueryStore::ConvertToNativeAccessType(EQueryAccessType AccessType)
	{
		switch (AccessType)
		{
		case EQueryAccessType::ReadOnly:
			// Fall through
		case EQueryAccessType::OptionalReadOnly:
			return EMassFragmentAccess::ReadOnly;
		case EQueryAccessType::ReadWrite:
			return EMassFragmentAccess::ReadWrite;
		default:
			checkf(false, TEXT("Invalid query access type: %i."), static_cast<uint32>(AccessType));
			return EMassFragmentAccess::MAX;
		}
	}

	EMassFragmentPresence FExtendedQueryStore::ConvertToNativePresenceType(EQueryAccessType AccessType)
	{
		switch (AccessType)
		{
		case EQueryAccessType::ReadOnly:
			return EMassFragmentPresence::All;
		case EQueryAccessType::OptionalReadOnly:
			return EMassFragmentPresence::Optional;
		case EQueryAccessType::ReadWrite:
			return EMassFragmentPresence::All;
		default:
			checkf(false, TEXT("Invalid query access type: %i."), static_cast<uint32>(AccessType));
			return EMassFragmentPresence::MAX;
		}
	}

	EMassFragmentPresence FExtendedQueryStore::ConvertToNativePresenceType(FQueryDescription::EOperatorType OperatorType)
	{
		EMassFragmentPresence Presence;
		switch (OperatorType)
		{
		case FQueryDescription::EOperatorType::SimpleAll:
			Presence = EMassFragmentPresence::All;
			break;
		case FQueryDescription::EOperatorType::SimpleAny:
			Presence = EMassFragmentPresence::Any;
			break;
		case FQueryDescription::EOperatorType::SimpleNone:
			Presence = EMassFragmentPresence::None;
			break;
		default:
			Presence = EMassFragmentPresence::Any;
			ensureAlwaysMsgf(false, TEXT("OperatorType '%d' cannot be converted to native type"), static_cast<int32>(OperatorType));
		}
		return Presence;
	}

	void FExtendedQueryStore::RegisterPreambleQuery(EQueryTickPhase Phase, Handle Query)
	{
		PhasePreparationQueries[static_cast<QueryTickPhaseType>(Phase)].Add(Query);
	}

	void FExtendedQueryStore::RegisterPostambleQuery(EQueryTickPhase Phase, Handle Query)
	{
		PhaseFinalizationQueries[static_cast<QueryTickPhaseType>(Phase)].Add(Query);
	}

	void FExtendedQueryStore::UnregisterPreambleQuery(EQueryTickPhase Phase, Handle Query)
	{
		int32 Index;
		if (PhasePreparationQueries[static_cast<QueryTickPhaseType>(Phase)].Find(Query, Index))
		{
			PhasePreparationQueries[static_cast<QueryTickPhaseType>(Phase)].RemoveAt(Index);
		}
	}

	void FExtendedQueryStore::UnregisterPostambleQuery(EQueryTickPhase Phase, Handle Query)
	{
		int32 Index;
		if (PhaseFinalizationQueries[static_cast<QueryTickPhaseType>(Phase)].Find(Query, Index))
		{
			PhaseFinalizationQueries[static_cast<QueryTickPhaseType>(Phase)].RemoveAt(Index);
		}
	}

	void FExtendedQueryStore::RunPhasePreOrPostAmbleQueries(FMassEntityManager& EntityManager,
		FEnvironment& Environment, EQueryTickPhase Phase,
		float DeltaTime, TArray<Handle>& QueryHandles)
	{
		if (!QueryHandles.IsEmpty())
		{
			FPhasePreOrPostAmbleExecutor Executor(EntityManager, DeltaTime);
			for (Handle Query : QueryHandles)
			{
				FExtendedQuery& QueryData = Queries.Get(Query);
				Executor.ExecuteQuery(QueryData.Description, *this, Environment, QueryData.NativeQuery, QueryData.Description.Callback.Function);
			}
		}
	}

	void FExtendedQueryStore::UnregisterQueryData(Handle Query, FExtendedQuery& QueryData, FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager)
	{
		if (!QueryData.Description.Callback.ActivationName.IsNone())
		{
			ActivatableMapping.RemoveSingle(QueryData.Description.Callback.ActivationName, Query);
			ActiveActivatables.RemoveSingleSwap(Query);
			PendingActivatables.RemoveSingleSwap(Query);
		}

		if (QueryData.Processor)
		{
			if (QueryData.Processor->IsA<UTypedElementQueryProcessorCallbackAdapterProcessorBase>())
			{
				PhaseManager.UnregisterDynamicProcessor(*QueryData.Processor);
			}
			else if (QueryData.Processor->IsA<UTypedElementQueryObserverCallbackAdapterProcessorBase>())
			{
				UTypedElementQueryObserverCallbackAdapterProcessorBase* Observer =
					static_cast<UTypedElementQueryObserverCallbackAdapterProcessorBase*>(QueryData.Processor.Get());
				if (ensure(Observer))
				{
					EntityManager.GetObserverManager().RemoveObserverInstance(Observer->GetObservedType(), Observer->GetObservedOperations(), Observer);
				}
			}
			else
			{
				checkf(false, TEXT("Query processor %s is of unsupported type %s."),
					*QueryData.Description.Callback.Name.ToString(), *QueryData.Processor->GetSparseClassDataStruct()->GetName());
			}
		}
		else if (QueryData.Description.Callback.Type == EQueryCallbackType::PhasePreparation)
		{
			UnregisterPreambleQuery(QueryData.Description.Callback.Phase, Query);
		}
		else if (QueryData.Description.Callback.Type == EQueryCallbackType::PhaseFinalization)
		{
			UnregisterPostambleQuery(QueryData.Description.Callback.Phase, Query);
		}
		else
		{
			QueryData.NativeQuery.Clear();
		}
	}

	bool FExtendedQueryStore::CheckCompatibility(const FQueryDescription& QueryDescription, const Queries::TQueryFunction<void> QueryCallback) const
	{
		for (const UScriptStruct* Column : QueryCallback.ConstColumnTypes)
		{
			if (int32 Index = QueryDescription.SelectionTypes.Find(Column);
				Index == INDEX_NONE || QueryDescription.SelectionAccessTypes[Index] == EQueryAccessType::OptionalReadOnly)
			{
				return false;
			}
		}

		for (const UScriptStruct* Column : QueryCallback.MutableColumnTypes)
		{
			if (int32 Index = QueryDescription.SelectionTypes.Find(Column);
				Index == INDEX_NONE || QueryDescription.SelectionAccessTypes[Index] != EQueryAccessType::ReadWrite)
			{
				return false;
			}
		}
		return true;
	}

	bool FExtendedQueryStore::CheckCompatibility(const FQueryDescription& QueryDescription, const Queries::TConstQueryFunction<void> QueryCallback) const
	{
		for (const UScriptStruct* Column : QueryCallback.ConstColumnTypes)
		{
			if (int32 Index = QueryDescription.SelectionTypes.Find(Column);
				Index == INDEX_NONE || QueryDescription.SelectionAccessTypes[Index] == EQueryAccessType::OptionalReadOnly)
			{
				return false;
			}
		}
		return true;
	}
} // namespace UE::Editor::DataStorage
