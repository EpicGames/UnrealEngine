// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "MassEntityQuery.h"
#include "MassRequirements.h"
#include "TypedElementHandleStore.h"

struct FMassEntityManager;
struct FMassProcessingPhaseManager;
class UMassProcessor;
class FOutputDevice;

namespace UE::Editor::DataStorage
{
	class FEnvironment;
	
	class FDynamicColumnGenerator;
	struct FDynamicColumnGeneratorInfo;

	struct FExtendedQuery
	{
		mutable FMassEntityQuery NativeQuery; // Used if there's no processor bound. Mutable because NativeQuery does internal state caching.
		FQueryDescription Description;
		TStrongObjectPtr<UMassProcessor> Processor;
		FMassEntityQuery* QueryReference;
		
		FMassEntityQuery* GetQuery()
		{
			if (Processor)
			{
				return QueryReference;
			}
			else
			{
				return &NativeQuery;
			}
		}
		
	};
	
	/**
	 * Storage and utilities for Typed Element queries after they've been processed by the Data Storage implementation.
	 */
	class FExtendedQueryStore
	{
	private:
		using QueryStore = THandleStore<FExtendedQuery>;
	public:
		using Handle = QueryStore::Handle;
		using ListAliveEntriesConstCallback = QueryStore::ListAliveEntriesConstCallback;

		explicit FExtendedQueryStore(FDynamicColumnGenerator& InDynamicColumnGenerator);

		/**
		 * @section Registration
		 * @description A set of functions to manage the registration of queries.
		 */

		 /** Adds a new query to the store and initializes the query with the provided arguments. */
		Handle RegisterQuery(
			FQueryDescription Query,
			FEnvironment& Environment,
			FMassEntityManager& EntityManager,
			FMassProcessingPhaseManager& PhaseManager);
		/** Removes the query at the given handle if still alive and otherwise does nothing. */
		void UnregisterQuery(Handle Query, FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager);

		/** Removes all data in the query store. */
		void Clear(FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager);

		/** Register the defaults for a tick group. These will be applied on top of any settings provided with a query registration. */
		void RegisterTickGroup(FName GroupName, EQueryTickPhase Phase,
			FName BeforeGroup, FName AfterGroup, EExecutionMode ExecutionMode);
		/** Removes a previously registered set of tick group defaults. */
		void UnregisterTickGroup(FName GroupName, EQueryTickPhase Phase);

		/**
		 * @section Retrieval
		 * @description Functions to retrieve data or information on queries.
		 */

		 /** Retrieves the query at the provided handle, if still alive or otherwise returns a nullptr. */
		FExtendedQuery* Get(Handle Entry);
		/** Retrieves the query at the provided handle, if still alive or otherwise returns a nullptr. */
		FExtendedQuery* GetMutable(Handle Entry);
		/** Retrieves the query at the provided handle, if still alive or otherwise returns a nullptr. */
		const FExtendedQuery* Get(Handle Entry) const;

		/** Retrieves the query at the provided handle, if still alive. It's up to the caller to guarantee the query is still alive. */
		FExtendedQuery& GetChecked(Handle Entry);
		/** Retrieves the query at the provided handle, if still alive. It's up to the caller to guarantee the query is still alive. */
		FExtendedQuery& GetMutableChecked(Handle Entry);
		/** Retrieves the query at the provided handle, if still alive. It's up to the caller to guarantee the query is still alive. */
		const FExtendedQuery& GetChecked(Handle Entry) const;

		/** Gets the original description used to create an extended query or an empty default if the provided query isn't alive. */
		const FQueryDescription& GetQueryDescription(Handle Query) const;

		/**
		 * Calculates a hash based on the topology revision id of the tables associated with the provided query. If this value changes
		 * it indicates that one or more tables has new rows added, had rows removed and/or rows have been reordered.
		 */
		uint64 CalculateQueryTablesTopologyHash(Handle Query) const;

		/** Checks to see if a query is still available or has been removed. */
		bool IsAlive(Handle Entry) const;

		/** Calls the provided callback for each query that's available. */
		void ListAliveEntries(const ListAliveEntriesConstCallback& Callback) const;

		/**
		 * @section activatable queries
		 * @description Functions to manipulate activatable queries
		 */

		 /** Update the active activatable queries. In practice this means decrementing any active queries that automatically decrement. */
		void UpdateActivatableQueries();
		/** Triggers a query to run for a single update cycle. */
		void ActivateQueries(FName ActivationName);

		/**
		 * @section Execution
		 * @description Various functions to run queries.
		 */

		FQueryResult RunQuery(FMassEntityManager& EntityManager, Handle Query);
		FQueryResult RunQuery(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			Handle Query,
			EDirectQueryExecutionFlags DirectExecutionFlags,
			DirectQueryCallbackRef Callback);
		FQueryResult RunQuery(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			Handle Query,
			ERunQueryFlags Flags,
			const Queries::TQueryFunction<void>& Callback);
		FQueryResult RunQuery(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			Handle Query,
			ERunQueryFlags Flags,
			const Queries::TConstQueryFunction<void>& Callback) const;
		FQueryResult RunQuery(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			FMassExecutionContext& ParentContext,
			Handle Query,
			SubqueryCallbackRef Callback);
		FQueryResult RunQuery(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			FMassExecutionContext& ParentContext,
			Handle Query,
			RowHandle Row,
			SubqueryCallbackRef Callback);
		void RunPhasePreambleQueries(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			EQueryTickPhase Phase,
			float DeltaTime);
		void RunPhasePostambleQueries(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			EQueryTickPhase Phase,
			float DeltaTime);

		void NotifyNewDynamicColumn(const FDynamicColumnGeneratorInfo& GeneratedColumnInfo);

		void DebugPrintQueryCallbacks(FOutputDevice& Output) const;

	private:
		using QueryTickPhaseType = std::underlying_type_t<EQueryTickPhase>;
		static constexpr QueryTickPhaseType MaxTickPhase = static_cast<QueryTickPhaseType>(EQueryTickPhase::Max);

		struct FTickGroupId
		{
			FName Name;
			EQueryTickPhase Phase;

			friend inline uint32 GetTypeHash(const FTickGroupId& Id) { return HashCombine(GetTypeHash(Id.Name), GetTypeHash(Id.Phase)); }
			friend inline bool operator==(const FTickGroupId& Lhs, const FTickGroupId& Rhs) { return Lhs.Phase == Rhs.Phase && Lhs.Name == Rhs.Name; }
			friend inline bool operator!=(const FTickGroupId& Lhs, const FTickGroupId& Rhs) { return Lhs.Phase != Rhs.Phase || Lhs.Name != Rhs.Name; }
		};

		struct FTickGroupDescription
		{
			TArray<FName> BeforeGroups;
			TArray<FName> AfterGroups;
			EExecutionMode ExecutionMode = EExecutionMode::Default;
		};

		template<typename CallbackReference, typename ExecutionFlagsType>
		FQueryResult RunQueryCallbackCommon(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			FMassExecutionContext* ParentContext,
			Handle Query,
			ExecutionFlagsType ExecutionFlags,
			CallbackReference Callback);
		FQueryResult RunQueryCallbackCommon(
			FMassEntityManager& EntityManager,
			FEnvironment& Environment,
			FMassExecutionContext* ParentContext,
			Handle Query,
			ERunQueryFlags ExecutionFlags,
			const Queries::TConstQueryFunction<void>& Callback) const;

		FMassEntityQuery& SetupNativeQuery(FQueryDescription& Query, FExtendedQuery& StoredQuery, FEnvironment& Environment);
		bool SetupDynamicColumns(FQueryDescription& Query, FEnvironment& Environment);
		bool SetupHierarchyDependencies(FQueryDescription& Query, FEnvironment& Environment);
		bool SetupSelectedColumns(FQueryDescription& Query, FMassEntityQuery& NativeQuery);
		bool SetupConditions(FQueryDescription& Query, FEnvironment& Environment, FMassEntityQuery& NativeQuery);
		bool SetupChunkFilters(Handle QueryHandle, FQueryDescription& Query, FEnvironment& Environment, FMassEntityQuery& NativeQuery);
		bool SetupDependencies(FQueryDescription& Query, FMassEntityQuery& NativeQuery);
		bool SetupTickGroupDefaults(FQueryDescription& Query);
		bool SetupProcessors(Handle QueryHandle, FExtendedQuery& StoredQuery, FEnvironment& Environment,
			FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager);
		bool SetupActivatable(Handle QueryHandle, FExtendedQuery& StoredQuery);

		EMassFragmentAccess ConvertToNativeAccessType(EQueryAccessType AccessType);
		EMassFragmentPresence ConvertToNativePresenceType(EQueryAccessType AccessType);
		EMassFragmentPresence ConvertToNativePresenceType(FQueryDescription::EOperatorType OperatorType);

		void RegisterPreambleQuery(EQueryTickPhase Phase, Handle Query);
		void RegisterPostambleQuery(EQueryTickPhase Phase, Handle Query);
		void UnregisterPreambleQuery(EQueryTickPhase Phase, Handle Query);
		void UnregisterPostambleQuery(EQueryTickPhase Phase, Handle Query);
		void RunPhasePreOrPostAmbleQueries(FMassEntityManager& EntityManager, FEnvironment& Environment,
			EQueryTickPhase Phase, float DeltaTime, TArray<Handle>& QueryHandles);

		void UnregisterQueryData(Handle Query, FExtendedQuery& QueryData, FMassEntityManager& EntityManager, FMassProcessingPhaseManager& PhaseManager);

		bool CheckCompatibility(const FQueryDescription& QueryDescription, const Queries::TQueryFunction<void> QueryCallback) const;
		bool CheckCompatibility(const FQueryDescription& QueryDescription, const Queries::TConstQueryFunction<void> QueryCallback) const;

		static const FQueryDescription EmptyDescription;

		QueryStore Queries;
		TMultiMap<FName, Handle> ActivatableMapping;
		TMap<FTickGroupId, FTickGroupDescription> TickGroupDescriptions;
		TArray<Handle> PhasePreparationQueries[MaxTickPhase];
		TArray<Handle> PhaseFinalizationQueries[MaxTickPhase];
		TArray<Handle> PendingActivatables;
		TArray<Handle> ActiveActivatables;
		FDynamicColumnGenerator* DynamicColumnGenerator;
	};
} // namespace UE::Editor::DataStorage
