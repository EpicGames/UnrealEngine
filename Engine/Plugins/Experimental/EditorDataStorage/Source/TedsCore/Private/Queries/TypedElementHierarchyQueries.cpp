// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/TypedElementHierarchyQueries.h"

#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementHierarchyQueries)

void UTypedElementHiearchyQueriesFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Resolve hierarchy rows"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default)),
			[](IQueryContext& Context, RowHandle Row, const FUnresolvedTableRowParentColumn& UnresolvedParent)
			{
				RowHandle ParentRow = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, UnresolvedParent.ParentIdKey);
				if (Context.IsRowAvailable(ParentRow))
				{
					Context.RemoveColumns<FUnresolvedTableRowParentColumn>(Row);
					Context.AddColumn(Row, FTableRowParentColumn{ .Parent = ParentRow });
				}
			})
		.Compile());
}
