// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementHierarchyQueries.generated.h"

/**
 * Calls to queries for general hierarchy management.
 */
UCLASS()
class UTypedElementHiearchyQueriesFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTypedElementHiearchyQueriesFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
