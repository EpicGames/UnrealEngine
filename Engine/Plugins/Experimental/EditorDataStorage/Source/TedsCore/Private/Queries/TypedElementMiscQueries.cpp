// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/TypedElementMiscQueries.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementMiscQueries)

void UTypedElementRemoveSyncToWorldTagFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove 'sync to world' tag"),
			FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FTypedElementSyncBackToWorldTag>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			}
		)
		.Where()
			.All<FTypedElementSyncBackToWorldTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove 'sync from world' tag"),
			FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FTypedElementSyncFromWorldTag>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			}
		)
		.Where()
			.All<FTypedElementSyncFromWorldTag>()
		.Compile());
}
