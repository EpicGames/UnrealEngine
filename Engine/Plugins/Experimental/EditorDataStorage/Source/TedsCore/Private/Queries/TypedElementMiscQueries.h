// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementMiscQueries.generated.h"

/**
 * Removes all FTypedElementSyncBackToWorldTags at the end of an update cycle.
 */
UCLASS()
class UTypedElementRemoveSyncToWorldTagFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTypedElementRemoveSyncToWorldTagFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
