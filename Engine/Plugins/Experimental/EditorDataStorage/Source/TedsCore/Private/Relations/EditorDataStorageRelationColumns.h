// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"

#include "EditorDataStorageRelationColumns.generated.h"

/**
 * Template tag column stamped on a Subject row when a relation of a specific type is created or destroyed.
 * Removed at FrameEnd. Use as a per-frame change notification in queries/observers.
 * Each relation type generates its own unique instance via GenerateDynamicColumn.
 */
USTRUCT(meta = (DisplayName = "Relation Subject Changed", EditorDataStorage_DynamicColumnTemplate))
struct FTedsRelationSubjectChanged_Template : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Template tag column stamped on an Object row when a relation of a specific type is created or destroyed.
 * Removed at FrameEnd. Use as a per-frame change notification in queries/observers.
 * Each relation type generates its own unique instance via GenerateDynamicColumn.
 */
USTRUCT(meta = (DisplayName = "Relation Object Changed", EditorDataStorage_DynamicColumnTemplate))
struct FTedsRelationObjectChanged_Template : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/** Hierarchy metadata column for WalkOnly relations. Added to each relation row. */
USTRUCT(meta = (DisplayName = "Walk-Only Hierarchy Metadata"))
struct FWalkOnlyHierarchyMetadata final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	int32 Depth = 0;

	UPROPERTY(meta = (Searchable))
	FTedsRowHandle Root;
};

/** Hierarchy metadata column for IntervalEncoded relations. Added to each relation row. */
USTRUCT(meta = (DisplayName = "Interval-Encoded Hierarchy Metadata"))
struct FIntervalEncodedHierarchyMetadata final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	int32 Depth = 0;

	UPROPERTY(meta = (Searchable))
	uint32 IntervalVersion = 0;

	UPROPERTY(meta = (Sortable))
	int64 IntervalLeft = 0;

	UPROPERTY(meta = (Sortable))
	int64 IntervalRight = 0;

	UPROPERTY(meta = (Searchable))
	FTedsRowHandle Root;
};
