// Copyright Epic Games, Inc. All Rights Reserved.

#include "Relations/EditorDataStorageRelationsImplementation.h"
#include "TableManager.h"
#include "Misc/MemStack.h"
#include "Relations/EditorDataStorageRelationColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "MassEntityManager.h"
#include "MassTypeManager.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::Editor::DataStorage
{
	// TSet allocator backed by FMemStack. All allocations freed in one shot when FMemMark destructs.
	using FMemStackSetAllocator = TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<>, TMemStackAllocator<>>, TMemStackAllocator<>>;

	// Maximum parent-chain walk depth. Guards against infinite loops from corrupt hierarchy cycles.
	constexpr int32 MaxHierarchyWalkDepth = 65536;

	//
	// Interval encoding (nested set model) for O(1) descendant queries.
	// Each node gets an interval [Left, Right] via DFS. Y is a descendant of X iff X.Left < Y.Left && Y.Right < X.Right.
	//

	// Maximum usable interval value. Half of int64 max to leave headroom for arithmetic.
	constexpr int64 MaxInterval = TNumericLimits<int64>::Max() / 2;

	// Minimum gap between adjacent intervals. Below this, rebalancing is triggered.
	constexpr int64 MinimumIntervalGap = 2;

	struct FIntervalResult
	{
		int64 Left = 0;
		int64 Right = 0;
		bool bNeedsRebalance = false;
	};

	static bool IsIntervalDescendant(int64 TestLeft, int64 TestRight, int64 AncestorLeft, int64 AncestorRight)
	{
		return AncestorLeft < TestLeft && TestRight < AncestorRight;
	}

	static FIntervalResult CalculateDescendantInterval(
		int64 AncestorLeft,
		int64 AncestorRight,
		int64 ExistingSiblingRight,
		int64 InitialGap)
	{
		FIntervalResult Result;

		const int64 AvailableStart = ExistingSiblingRight + 1;
		const int64 AvailableEnd = AncestorRight - 1;
		const int64 AvailableSpace = AvailableEnd - AvailableStart;

		if (AvailableSpace < MinimumIntervalGap * 2 + 1)
		{
			Result.bNeedsRebalance = true;
			Result.Left = AvailableStart;
			Result.Right = AvailableStart + 1;
			return Result;
		}

		// Divide available space into three equal parts (gap before Left, node width, gap after Right).
		const int64 DesiredGap = FMath::Min(InitialGap, AvailableSpace / 3);
		Result.Left = AvailableStart + DesiredGap;
		Result.Right = Result.Left + DesiredGap;

		if (Result.Right >= AvailableEnd)
		{
			Result.bNeedsRebalance = true;
			Result.Left = AvailableStart + 1;
			Result.Right = AvailableEnd - 1;
		}

		return Result;
	}


//
// FTedsRelationAdapter
//

RelationTypeHandle FTedsRelationAdapter::RegisterRelationType(
	FMassEntityManager& EntityManager,
	FTableManager& TableManager,
	const FRelationRegistrationParams& Params)
{
	if (const RelationTypeHandle* ExistingHandle = NameToHandle.Find(Params.Name))
	{
		return *ExistingHandle;
	}

	// Create a dynamic UScriptStruct derived from FMassRelation.
	// This follows the DynamicColumnGenerator pattern used elsewhere in TEDS.
	const FName TagName = FName(*FString::Printf(TEXT("TedsRelation_%s"), *Params.Name.ToString()));

	UScriptStruct* NewRelationTag = NewObject<UScriptStruct>(GetTransientPackage(), TagName);
	NewRelationTag->AddToRoot();
	NewRelationTag->SetSuperStruct(FMassRelation::StaticStruct());
	NewRelationTag->Bind();
	NewRelationTag->PrepareCppStructOps();
	NewRelationTag->StaticLink(true);

	UE::Mass::Relations::FRelationTypeTraits MassTraits(NewRelationTag);
	MassTraits.RelationName = Params.Name;
	MassTraits.bHierarchical = (Params.Traits.HierarchyMode != EHierarchyMode::Disabled);

	auto MapRoleTraits = [](const FTedsRelationRoleTraits& TedsRole) -> UE::Mass::Relations::FRoleTraits
	{
		UE::Mass::Relations::FRoleTraits MassRole;
		MassRole.bExclusive = TedsRole.bExclusive;

		// Mass's observer infrastructure and RoleMap (used by GetRelationSubjects/Objects,
		// GatherHierarchy, etc.) require RequiresExternalMapping = Yes. Without it,
		// observers crash when accessing the RoleMap. Setting Element = nullptr avoids
		// adding tags to participants but the RoleMap is still maintained.
		MassRole.Element = nullptr;
		MassRole.RequiresExternalMapping = UE::Mass::Relations::EExternalMappingRequired::Yes;

		switch (TedsRole.DestructionPolicy)
		{
		case FTedsRelationRoleTraits::EDestructionPolicy::CleanUp:
			MassRole.DestructionPolicy = UE::Mass::Relations::ERemovalPolicy::CleanUp;
			break;
		case FTedsRelationRoleTraits::EDestructionPolicy::Cascade:
			MassRole.DestructionPolicy = UE::Mass::Relations::ERemovalPolicy::Destroy;
			break;
		case FTedsRelationRoleTraits::EDestructionPolicy::Orphan:
			// Orphan uses Custom policy; TEDS handles the observer
			MassRole.DestructionPolicy = UE::Mass::Relations::ERemovalPolicy::Custom;
			break;
		}

		return MassRole;
	};

	MassTraits.RoleTraits[static_cast<uint8>(UE::Mass::Relations::ERelationRole::Subject)] =
		MapRoleTraits(Params.Traits.Subject);
	MassTraits.RoleTraits[static_cast<uint8>(UE::Mass::Relations::ERelationRole::Object)] =
		MapRoleTraits(Params.Traits.Object);

	// When bBuiltInTypesRegistered is true (normal runtime), RegisterType calls
	// OnNewTypeRegistered which populates the RelationsDataMap automatically.
	UE::Mass::FTypeHandle MassHandle = EntityManager.GetTypeManager().RegisterType(MoveTemp(MassTraits));

	const RelationTypeHandle TedsHandle = static_cast<RelationTypeHandle>(RegisteredTypes.Num());

	FRegisteredType& NewType = RegisteredTypes.AddDefaulted_GetRef();
	NewType.Name = Params.Name;
	NewType.TedsTraits = Params.Traits;
	NewType.MassHandle = MassHandle;
	NewType.DynamicRelationTag.Reset(NewRelationTag);

	// Register a per-type table derived from TEDSRelations, adding the relation tag
	// so Mass creates relation entities in the correct archetype.
	{
		const FName RelationTableName = FName(*FString::Printf(TEXT("TedsRelation_%s"), *Params.Name.ToString()));
		const UScriptStruct* RelationTag = NewRelationTag;

		FTableRegistrationOptions Options;
		Options.SourceTable = TableManager.Find(TEXT("TEDSRelations"));
		ensureMsgf(Options.SourceTable != InvalidTableHandle, TEXT("RegisterRelationType: 'TEDSRelations' base table not found"));
		TableManager.Register({&RelationTag, 1}, RelationTableName, Options);
	}

	NameToHandle.Add(Params.Name, TedsHandle);
	return TedsHandle;
}

RelationTypeHandle FTedsRelationAdapter::FindRelationType(const FName& Name) const
{
	if (const RelationTypeHandle* Handle = NameToHandle.Find(Name))
	{
		return *Handle;
	}
	return InvalidRelationTypeHandle;
}

bool FTedsRelationAdapter::IsValidRelationType(RelationTypeHandle Type) const
{
	return Type != InvalidRelationTypeHandle &&
		   static_cast<int32>(Type) < RegisteredTypes.Num();
}

const FTedsRelationTraits* FTedsRelationAdapter::GetTraits(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return nullptr;
	}
	return &RegisteredTypes[static_cast<int32>(Type)].TedsTraits;
}

UE::Mass::FTypeHandle FTedsRelationAdapter::GetMassHandle(RelationTypeHandle TedsHandle) const
{
	if (!IsValidRelationType(TedsHandle))
	{
		return UE::Mass::FTypeHandle();
	}
	return RegisteredTypes[static_cast<int32>(TedsHandle)].MassHandle;
}

const UScriptStruct* FTedsRelationAdapter::GetSubjectChangedColumn(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return nullptr;
	}
	return RegisteredTypes[static_cast<int32>(Type)].SubjectChangedColumn;
}

const UScriptStruct* FTedsRelationAdapter::GetObjectChangedColumn(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return nullptr;
	}
	return RegisteredTypes[static_cast<int32>(Type)].ObjectChangedColumn;
}

void FTedsRelationAdapter::SetChangedColumns(RelationTypeHandle Type, const UScriptStruct* SubjectChanged, const UScriptStruct* ObjectChanged)
{
	if (!IsValidRelationType(Type))
	{
		return;
	}
	FRegisteredType& Entry = RegisteredTypes[static_cast<int32>(Type)];
	Entry.SubjectChangedColumn = SubjectChanged;
	Entry.ObjectChangedColumn = ObjectChanged;
}

FName FTedsRelationAdapter::GetTypeName(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return NAME_None;
	}
	return RegisteredTypes[static_cast<int32>(Type)].Name;
}

bool FTedsRelationAdapter::IsHierarchical(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return false;
	}
	return RegisteredTypes[static_cast<int32>(Type)].TedsTraits.HierarchyMode != EHierarchyMode::Disabled;
}


const UScriptStruct* FTedsRelationAdapter::GetRelationTag(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return nullptr;
	}
	return RegisteredTypes[static_cast<int32>(Type)].DynamicRelationTag.Get();
}

bool FTedsRelationAdapter::HasMassRelationData(RelationTypeHandle Type) const
{
	if (!IsValidRelationType(Type))
	{
		return false;
	}
	return RegisteredTypes[static_cast<int32>(Type)].bHasMassRelationData;
}

void FTedsRelationAdapter::SetHasMassRelationData(RelationTypeHandle Type)
{
	if (IsValidRelationType(Type))
	{
		RegisteredTypes[static_cast<int32>(Type)].bHasMassRelationData = true;
	}
}

void FTedsRelationAdapter::ListTypes(TFunctionRef<void(RelationTypeHandle, const FName&)> Callback) const
{
	for (int32 Index = 0; Index < RegisteredTypes.Num(); ++Index)
	{
		Callback(static_cast<RelationTypeHandle>(Index), RegisteredTypes[Index].Name);
	}
}

//
// FTedsHierarchicalRelationManager
//

void FTedsHierarchicalRelationManager::RegisterHierarchyMode(RelationTypeHandle Type, EHierarchyMode Mode, int64 InitialGap)
{
	TypeSettings.Add(Type, { Mode, InitialGap });
	if (Mode == EHierarchyMode::IntervalEncoded)
	{
		TypeVersions.Add(Type, 1u);  // Start at 1; 0 means "never stamped"
	}
}

RowHandle FTedsHierarchicalRelationManager::FindRelationRow(RelationTypeHandle Type, RowHandle Subject) const
{
	if (const TMap<RowHandle, RowHandle>* TypeMap = SubjectToRelationRow.Find(Type))
	{
		if (const RowHandle* FoundRelationRow = TypeMap->Find(Subject))
		{
			return *FoundRelationRow;
		}
	}
	return InvalidRowHandle;
}

int32 FTedsHierarchicalRelationManager::ReadDepth(
	const ICoreProvider& Provider, RelationTypeHandle Type, RowHandle RelationRow) const
{
	const FHierarchySettings* SettingsPtr = TypeSettings.Find(Type);
	if (!SettingsPtr)
	{
		return 0;
	}
	const FHierarchySettings& Settings = *SettingsPtr;
	const EHierarchyMode Mode = Settings.Mode;
	if (Mode == EHierarchyMode::IntervalEncoded)
	{
		if (const auto* Metadata = static_cast<const FIntervalEncodedHierarchyMetadata*>(
				Provider.GetColumnData(RelationRow, FIntervalEncodedHierarchyMetadata::StaticStruct())))
		{
			return Metadata->Depth;
		}
	}
	else
	{
		if (const auto* Metadata = static_cast<const FWalkOnlyHierarchyMetadata*>(
				Provider.GetColumnData(RelationRow, FWalkOnlyHierarchyMetadata::StaticStruct())))
		{
			return Metadata->Depth;
		}
	}
	return 0;
}

RowHandle FTedsHierarchicalRelationManager::ReadRoot(
	const ICoreProvider& Provider, RelationTypeHandle Type, RowHandle RelationRow) const
{
	const FHierarchySettings* SettingsPtr = TypeSettings.Find(Type);
	if (!SettingsPtr)
	{
		return InvalidRowHandle;
	}
	const FHierarchySettings& Settings = *SettingsPtr;
	const EHierarchyMode Mode = Settings.Mode;
	if (Mode == EHierarchyMode::IntervalEncoded)
	{
		if (const auto* Metadata = static_cast<const FIntervalEncodedHierarchyMetadata*>(
				Provider.GetColumnData(RelationRow, FIntervalEncodedHierarchyMetadata::StaticStruct())))
		{
			return Metadata->Root;
		}
	}
	else
	{
		if (const auto* Metadata = static_cast<const FWalkOnlyHierarchyMetadata*>(
				Provider.GetColumnData(RelationRow, FWalkOnlyHierarchyMetadata::StaticStruct())))
		{
			return Metadata->Root;
		}
	}
	return InvalidRowHandle;
}

const FIntervalEncodedHierarchyMetadata* FTedsHierarchicalRelationManager::ReadIntervalMetadata(
	const ICoreProvider& Provider, RowHandle RelationRow) const
{
	return static_cast<const FIntervalEncodedHierarchyMetadata*>(
		Provider.GetColumnData(RelationRow, FIntervalEncodedHierarchyMetadata::StaticStruct()));
}

FIntervalEncodedHierarchyMetadata* FTedsHierarchicalRelationManager::ReadIntervalMetadataMutable(
	ICoreProvider& Provider, RowHandle RelationRow) const
{
	return static_cast<FIntervalEncodedHierarchyMetadata*>(
		Provider.GetColumnData(RelationRow, FIntervalEncodedHierarchyMetadata::StaticStruct()));
}

FWalkOnlyHierarchyMetadata* FTedsHierarchicalRelationManager::ReadWalkOnlyMetadataMutable(
	ICoreProvider& Provider, RowHandle RelationRow) const
{
	return static_cast<FWalkOnlyHierarchyMetadata*>(
		Provider.GetColumnData(RelationRow, FWalkOnlyHierarchyMetadata::StaticStruct()));
}

void FTedsHierarchicalRelationManager::InitializeHierarchyMetadata(
	ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle RelationRow,
	RowHandle Subject,
	RowHandle Object)
{
	// Self-relations would corrupt hierarchy metadata: FindRelationRow(Type, Object)
	// would return the freshly-inserted but uninitialized row for the subject itself.
	if (Subject == Object)
	{
		ensureMsgf(false, TEXT("InitializeHierarchyMetadata: self-relation (Subject == Object) is not supported for hierarchical types."));
		return;
	}

	SubjectToRelationRow.FindOrAdd(Type).Add(Subject, RelationRow);

	const FHierarchySettings* SettingsPtr = TypeSettings.Find(Type);
	if (!SettingsPtr)
	{
		return;
	}
	const EHierarchyMode Mode = SettingsPtr->Mode;

	// Look up the ancestor's (Object's) hierarchy metadata to compute descendant depth and root.
	// The ancestor's relation row is the one where the ancestor is the Subject (ancestor-to-root edge).
	int32 AncestorDepth = 0;
	RowHandle Root = Object;

	const RowHandle AncestorRelationRow = FindRelationRow(Type, Object);
	if (AncestorRelationRow != InvalidRowHandle)
	{
		AncestorDepth = ReadDepth(Provider, Type, AncestorRelationRow);
		RowHandle AncestorRoot = ReadRoot(Provider, Type, AncestorRelationRow);
		if (AncestorRoot != InvalidRowHandle)
		{
			Root = AncestorRoot;
		}
	}

	if (Mode == EHierarchyMode::WalkOnly)
	{
		FWalkOnlyHierarchyMetadata* Metadata = ReadWalkOnlyMetadataMutable(Provider, RelationRow);
		if (!Metadata)
		{
			return;
		}
		Metadata->Depth = AncestorDepth + 1;
		Metadata->Root = Root;
		return;
	}

	// IntervalEncoded path
	FIntervalEncodedHierarchyMetadata* Metadata = ReadIntervalMetadataMutable(Provider, RelationRow);
	if (!Metadata)
	{
		return;
	}

	Metadata->Depth = AncestorDepth + 1;
	Metadata->Root = Root;

	int64 AncestorLeft = 0;
	int64 AncestorRight = MaxInterval;

	if (AncestorRelationRow != InvalidRowHandle)
	{
		if (const FIntervalEncodedHierarchyMetadata* AncestorIntervalMeta = ReadIntervalMetadata(Provider, AncestorRelationRow))
		{
			AncestorLeft = AncestorIntervalMeta->IntervalLeft;
			AncestorRight = AncestorIntervalMeta->IntervalRight;
		}
	}

	// Find the rightmost existing sibling's interval to place the new descendant after it.
	int64 MaxSiblingRight = AncestorLeft;
	Provider.ForEachRelationSubject(Type, Object, [&](RowHandle SiblingRow)
	{
		if (SiblingRow == Subject)
		{
			return;
		}
		const RowHandle SiblingRelationRow = FindRelationRow(Type, SiblingRow);
		if (SiblingRelationRow != InvalidRowHandle)
		{
			if (const FIntervalEncodedHierarchyMetadata* SiblingMetadata = ReadIntervalMetadata(Provider, SiblingRelationRow))
			{
				MaxSiblingRight = FMath::Max(MaxSiblingRight, SiblingMetadata->IntervalRight);
			}
		}
	});

	FIntervalResult Interval = CalculateDescendantInterval(
		AncestorLeft, AncestorRight, MaxSiblingRight, SettingsPtr->InitialGap);

	Metadata->IntervalLeft = Interval.Left;
	Metadata->IntervalRight = Interval.Right;

	// Non-leaf reparent: existing children's intervals are stale relative to the new
	// parent interval. Advance the type version so IsDescendantOf falls back to tree walk
	// for stale rows until ProcessRebalancing restamps them at FrameEnd.
	if (Provider.HasRelationSubjects(Type, Subject))
	{
		++TypeVersions.FindOrAdd(Type);
		MarkForRebalance(Type);
	}
	Metadata->IntervalVersion = TypeVersions.FindRef(Type);

	if (Interval.bNeedsRebalance)
	{
		// The emergency interval may violate strict containment (Right == AncestorRight).
		// Bump the type version so TryIntervalDescendantCheck sees a mismatch and falls
		// back to the parent-chain walk for this node until ProcessRebalancing corrects it.
		++TypeVersions.FindOrAdd(Type);
		MarkForRebalance(Type);
	}
}

void FTedsHierarchicalRelationManager::RemoveHierarchyMetadata(RelationTypeHandle Type, RowHandle Subject)
{
	if (TMap<RowHandle, RowHandle>* TypeMap = SubjectToRelationRow.Find(Type))
	{
		TypeMap->Remove(Subject);
	}
}

/**
 * Shared interval-based fast path for IsDescendantOf.
 *
 * Attempts the O(1) interval containment check when intervals are valid and version
 * stamps match. Returns an optional: set to true/false if the interval check was
 * conclusive, or unset if a parent-chain walk fallback is needed.
 *
 * @param ReadMetadataFunction Callable: (RowHandle RelationRow) -> const FIntervalEncodedHierarchyMetadata*
 */
template<typename ReadMetadataFunctionType>
static TOptional<bool> TryIntervalDescendantCheck(
	RowHandle DescendantRelationRow,
	RowHandle AncestorRelationRow,
	EHierarchyMode Mode,
	uint32 CurrentTypeVersion,
	ReadMetadataFunctionType&& ReadMetadataFunction)
{
	if (Mode != EHierarchyMode::IntervalEncoded)
	{
		// Interval data is only maintained for IntervalEncoded relations.
		// For WalkOnly (and any future non-interval mode), skip to the parent-chain walk.
		return {};
	}

	if (DescendantRelationRow == InvalidRowHandle)
	{
		return false;
	}

	const FIntervalEncodedHierarchyMetadata* DescendantMetadata = ReadMetadataFunction(DescendantRelationRow);
	if (!DescendantMetadata)
	{
		return false;
	}

	const FIntervalEncodedHierarchyMetadata* AncestorMetadata = (AncestorRelationRow != InvalidRowHandle)
		? ReadMetadataFunction(AncestorRelationRow)
		: nullptr;

	// If the ancestor has a relation row but its metadata is missing, the data is corrupt.
	if (AncestorRelationRow != InvalidRowHandle && !AncestorMetadata)
	{
		return false;
	}

	// Attempt O(1) interval containment when both version stamps match the current type version.
	// Root ancestors (no relation row, so no metadata) always require the parent-chain walk.
	if (Mode == EHierarchyMode::IntervalEncoded && AncestorMetadata != nullptr)
	{
		const bool bDescendantValid =
			DescendantMetadata->IntervalLeft != 0 &&
			DescendantMetadata->IntervalVersion == CurrentTypeVersion;
		const bool bAncestorValid =
			AncestorMetadata->IntervalLeft != 0 &&
			AncestorMetadata->IntervalVersion == CurrentTypeVersion;

		if (bDescendantValid && bAncestorValid)
		{
			return IsIntervalDescendant(
				DescendantMetadata->IntervalLeft, DescendantMetadata->IntervalRight,
				AncestorMetadata->IntervalLeft,   AncestorMetadata->IntervalRight);
		}
	}

	// Interval check was not conclusive; caller must walk the parent chain.
	return {};
}

bool FTedsHierarchicalRelationManager::IsDescendantOf(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Descendant,
	RowHandle Ancestor,
	bool bIncludeSelf) const
{
	if (Ancestor == InvalidRowHandle || Descendant == InvalidRowHandle)
	{
		return false;
	}

	if (Descendant == Ancestor)
	{
		return bIncludeSelf;
	}

	const FHierarchySettings* SettingsPtr = TypeSettings.Find(Type);
	if (!SettingsPtr)
	{
		return false; // Disabled or unregistered relation type has no hierarchy data.
	}
	const FHierarchySettings& Settings = *SettingsPtr;
	const EHierarchyMode Mode = Settings.Mode;
	const uint32 CurrentTypeVersion = TypeVersions.FindRef(Type);

	auto ReadMetadataViaProvider = [this, &Provider](RowHandle RelationRow) -> const FIntervalEncodedHierarchyMetadata*
	{
		return ReadIntervalMetadata(Provider, RelationRow);
	};

	TOptional<bool> IntervalResult = TryIntervalDescendantCheck(
		FindRelationRow(Type, Descendant),
		FindRelationRow(Type, Ancestor),
		Mode, CurrentTypeVersion, ReadMetadataViaProvider);

	if (IntervalResult.IsSet())
	{
		return IntervalResult.GetValue();
	}

	// Fallback: walk the parent chain (with cycle guard)
	RowHandle CurrentRow = Descendant;
	int32 WalkDepth = 0;
	while (CurrentRow != InvalidRowHandle && CurrentRow != Ancestor && ++WalkDepth < MaxHierarchyWalkDepth)
	{
		CurrentRow = Provider.GetRelationObject(Type, CurrentRow);
	}
	ensureMsgf(WalkDepth < MaxHierarchyWalkDepth,
		TEXT("IsDescendantOf: parent-chain depth exceeded limit, possible hierarchy cycle"));
	return CurrentRow == Ancestor;
}

bool FTedsHierarchicalRelationManager::IsDescendantOf(
	FMassEntityManager& EntityManager,
	const FTedsRelationAdapter& RelationAdapter,
	RelationTypeHandle Type,
	RowHandle Descendant,
	RowHandle Ancestor,
	bool bIncludeSelf) const
{
	if (Ancestor == InvalidRowHandle || Descendant == InvalidRowHandle)
	{
		return false;
	}

	if (Descendant == Ancestor)
	{
		return bIncludeSelf;
	}

	const FHierarchySettings* SettingsPtr = TypeSettings.Find(Type);
	if (!SettingsPtr)
	{
		return false; // Disabled or unregistered relation type has no hierarchy data.
	}
	const FHierarchySettings& Settings = *SettingsPtr;
	const EHierarchyMode Mode = Settings.Mode;
	const uint32 CurrentTypeVersion = TypeVersions.FindRef(Type);

	// Read FIntervalEncodedHierarchyMetadata via Mass fragment access (no ICoreProvider needed)
	auto ReadMetadataFromMass = [&EntityManager](RowHandle RelationRow) -> const FIntervalEncodedHierarchyMetadata*
	{
		const FMassEntityHandle Entity = FMassEntityHandle::FromNumber(RelationRow);
		if (!EntityManager.IsEntityActive(Entity))
		{
			return nullptr;
		}
		FStructView View = EntityManager.GetFragmentDataStruct(Entity, FIntervalEncodedHierarchyMetadata::StaticStruct());
		if (!View.IsValid())
		{
			return nullptr;
		}
		return reinterpret_cast<const FIntervalEncodedHierarchyMetadata*>(View.GetMemory());
	};

	TOptional<bool> IntervalResult = TryIntervalDescendantCheck(
		FindRelationRow(Type, Descendant),
		FindRelationRow(Type, Ancestor),
		Mode, CurrentTypeVersion, ReadMetadataFromMass);

	if (IntervalResult.IsSet())
	{
		return IntervalResult.GetValue();
	}

	// Fallback: walk ancestor chain via Mass relation manager (with cycle guard)
	const UE::Mass::FTypeHandle MassHandle = RelationAdapter.GetMassHandle(Type);
	RowHandle CurrentRow = Descendant;
	int32 WalkDepth = 0;
	while (CurrentRow != InvalidRowHandle && CurrentRow != Ancestor && ++WalkDepth < MaxHierarchyWalkDepth)
	{
		const TArray<FMassEntityHandle> AncestorHandles = EntityManager.GetRelationManager().GetRelationObjects(
			MassHandle, FMassEntityHandle::FromNumber(CurrentRow));
		CurrentRow = AncestorHandles.Num() > 0 ? AncestorHandles[0].AsNumber() : InvalidRowHandle;
	}
	ensureMsgf(WalkDepth < MaxHierarchyWalkDepth,
		TEXT("IsDescendantOf(EntityManager): ancestor-chain depth exceeded limit, possible hierarchy cycle"));
	return CurrentRow == Ancestor;
}

RowHandle FTedsHierarchicalRelationManager::GetHierarchyRoot(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row) const
{


	// Root nodes have no relation row; they ARE the root.
	const RowHandle RelationRow = FindRelationRow(Type, Row);
	if (RelationRow == InvalidRowHandle)
	{
		return Row;
	}

	const RowHandle Root = ReadRoot(Provider, Type, RelationRow);
	if (Root != InvalidRowHandle)
	{
		return Root;
	}

	// Fallback: walk ancestor chain to the root (with cycle guard).
	RowHandle CurrentRow = Row;
	RowHandle AncestorRow = Provider.GetRelationObject(Type, CurrentRow);
	int32 WalkDepth = 0;
	while (AncestorRow != InvalidRowHandle && ++WalkDepth < MaxHierarchyWalkDepth)
	{
		CurrentRow = AncestorRow;
		AncestorRow = Provider.GetRelationObject(Type, CurrentRow);
	}
	ensureMsgf(WalkDepth < MaxHierarchyWalkDepth,
		TEXT("GetHierarchyRoot: ancestor-chain depth exceeded limit, possible hierarchy cycle"));
	return CurrentRow;
}

int32 FTedsHierarchicalRelationManager::GetHierarchyDepth(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row) const
{


	const RowHandle RelationRow = FindRelationRow(Type, Row);
	if (RelationRow == InvalidRowHandle)
	{
		return 0;
	}

	return ReadDepth(Provider, Type, RelationRow);
}

int32 FTedsHierarchicalRelationManager::ComputeDescendantCount(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row) const
{

	int32 Count = 0;
	TraverseDescendants(Provider, Type, Row,
		[&Count](RowHandle, RowHandle, int32)
		{
			++Count;
		});
	return Count;
}

void FTedsHierarchicalRelationManager::GetDescendants(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row,
	TArray<RowHandle>& OutDescendants) const
{

	OutDescendants.Reset();
	GatherDescendants(Provider, Type, Row, OutDescendants);
}

void FTedsHierarchicalRelationManager::GatherDescendants(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row,
	TArray<RowHandle>& OutDescendants) const
{
	// Iterative DFS with FMemStack-backed allocations (freed in one shot when Mark destructs).
	FMemMark Mark(FMemStack::Get());
	TSet<RowHandle, DefaultKeyFuncs<RowHandle>, FMemStackSetAllocator> VisitedRows;
	TArray<RowHandle, TMemStackAllocator<>> WorkStack;
	Provider.ForEachRelationSubject(Type, Row, [&WorkStack](RowHandle DescendantRow) { WorkStack.Add(DescendantRow); });

	while (!WorkStack.IsEmpty())
	{
		const RowHandle CurrentRow = WorkStack.Pop();
		if (VisitedRows.Contains(CurrentRow))
		{
			ensureMsgf(false, TEXT("GatherDescendants: cycle detected in hierarchy"));
			continue;
		}
		VisitedRows.Add(CurrentRow);
		OutDescendants.Add(CurrentRow);

		Provider.ForEachRelationSubject(Type, CurrentRow, [&WorkStack](RowHandle DescendantRow) { WorkStack.Add(DescendantRow); });
	}
}

void FTedsHierarchicalRelationManager::GetAncestors(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row,
	TArray<RowHandle>& OutAncestors) const
{

	OutAncestors.Reset();

	RowHandle CurrentRow = Provider.GetRelationObject(Type, Row);
	int32 WalkDepth = 0;
	while (CurrentRow != InvalidRowHandle && ++WalkDepth < MaxHierarchyWalkDepth)
	{
		OutAncestors.Add(CurrentRow);
		CurrentRow = Provider.GetRelationObject(Type, CurrentRow);
	}
	ensureMsgf(WalkDepth < MaxHierarchyWalkDepth,
		TEXT("GetAncestors: parent-chain depth exceeded limit, possible hierarchy cycle"));
}

void FTedsHierarchicalRelationManager::TraverseDescendants(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row,
	ICoreProvider::FRelationTraversalCallback Callback,
	ICoreProvider::ETraversalOrder Order,
	int32 MaxDepth) const
{


	if (Order == ICoreProvider::ETraversalOrder::PreOrder)
	{
		TraversePreOrder(Provider, Type, Row, MaxDepth, Callback);
	}
	else
	{
		TraversePostOrder(Provider, Type, Row, MaxDepth, Callback);
	}
}

void FTedsHierarchicalRelationManager::TraversePreOrder(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle StartRow,
	int32 MaxDepth,
	ICoreProvider::FRelationTraversalCallback& Callback) const
{
	struct FTraversalEntry
	{
		RowHandle Row;
		RowHandle AncestorRow;
		int32 Depth;
	};

	TSet<RowHandle> VisitedRows;
	TArray<FTraversalEntry, TInlineAllocator<64>> Stack;
	Stack.Push({StartRow, InvalidRowHandle, 0});

	while (!Stack.IsEmpty())
	{
		const FTraversalEntry Entry = Stack.Pop();

		if (Entry.Depth > 0)
		{
			if (VisitedRows.Contains(Entry.Row))
			{
				ensureMsgf(false, TEXT("TraversePreOrder: cycle detected in hierarchy"));
				continue;
			}
			VisitedRows.Add(Entry.Row);
			Callback(Entry.Row, Entry.AncestorRow, Entry.Depth);
		}

		if (Entry.Depth < MaxDepth)
		{
			Provider.ForEachRelationSubject(Type, Entry.Row, [&Stack, &Entry](RowHandle DescendantRow)
			{
				Stack.Push({DescendantRow, Entry.Row, Entry.Depth + 1});
			});
		}
	}
}

void FTedsHierarchicalRelationManager::TraversePostOrder(
	const ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle StartRow,
	int32 MaxDepth,
	ICoreProvider::FRelationTraversalCallback& Callback) const
{
	// Uses enter/exit markers: descendants are visited before their ancestor.
	struct FTraversalEntry
	{
		RowHandle Row;
		RowHandle AncestorRow;
		int32 Depth;
		bool bExiting;
	};

	TSet<RowHandle> VisitedRows;
	TArray<FTraversalEntry, TInlineAllocator<64>> Stack;
	Stack.Push({StartRow, InvalidRowHandle, 0, false});

	while (!Stack.IsEmpty())
	{
		const FTraversalEntry Entry = Stack.Pop();

		if (Entry.bExiting)
		{
			if (Entry.Depth > 0)
			{
				Callback(Entry.Row, Entry.AncestorRow, Entry.Depth);
			}
			continue;
		}

		if (Entry.Depth > 0)
		{
			if (VisitedRows.Contains(Entry.Row))
			{
				ensureMsgf(false, TEXT("TraversePostOrder: cycle detected in hierarchy"));
				continue;
			}
			VisitedRows.Add(Entry.Row);
		}

		// Push exit marker (processed after all descendants)
		Stack.Push({Entry.Row, Entry.AncestorRow, Entry.Depth, true});

		if (Entry.Depth < MaxDepth)
		{
			Provider.ForEachRelationSubject(Type, Entry.Row, [&Stack, &Entry](RowHandle DescendantRow)
			{
				Stack.Push({DescendantRow, Entry.Row, Entry.Depth + 1, false});
			});
		}
	}
}

void FTedsHierarchicalRelationManager::MarkForRebalance(RelationTypeHandle Type)
{
	TypesNeedingRebalance.Add(Type);
}

void FTedsHierarchicalRelationManager::ProcessRebalancing(ICoreProvider& Provider)
{


	if (TypesNeedingRebalance.IsEmpty())
	{
		return;
	}

	for (RelationTypeHandle Type : TypesNeedingRebalance)
	{
		// Collect all unique hierarchy roots for this type by scanning all known subjects.
		TSet<RowHandle> RootsToRebalance;

		if (const TMap<RowHandle, RowHandle>* SubjectMap = SubjectToRelationRow.Find(Type))
		{
			for (const auto& [SubjectRow, SubjectRelationRow] : *SubjectMap)
			{
				if (!Provider.IsRowAvailable(SubjectRow))
				{
					continue;
				}
				const RowHandle RootRow = GetHierarchyRoot(Provider, Type, SubjectRow);
				if (RootRow != InvalidRowHandle && Provider.IsRowAvailable(RootRow))
				{
					RootsToRebalance.Add(RootRow);
				}
			}
		}

		for (RowHandle RootRow : RootsToRebalance)
		{
			const int32 DescendantCount = ComputeDescendantCount(Provider, Type, RootRow);
			if (DescendantCount == 0)
			{
				continue;
			}

			// Distribute intervals evenly across the full available range.
			const int64 Gap = FMath::Max<int64>(
				MinimumIntervalGap,
				MaxInterval / (DescendantCount * 2 + 2));

			AssignIntervals(Provider, Type, RootRow, 0, Gap);
		}
	}

	TypesNeedingRebalance.Empty();
}

int64 FTedsHierarchicalRelationManager::AssignIntervals(
	ICoreProvider& Provider,
	RelationTypeHandle Type,
	RowHandle Row,
	int64 StartInterval,
	int64 Gap)
{
	// Iterative DFS with two-phase (Enter/Exit) processing:
	//   Enter: assign IntervalLeft, push children
	//   Exit:  assign IntervalRight and version stamp
	struct FIntervalEntry
	{
		RowHandle Row;
		bool bExiting;
	};

	// Cache the version pointer once (nullptr for WalkOnly types, valid for IntervalEncoded).
	const uint32* TypeVersion = TypeVersions.Find(Type);

	int64 CurrentInterval = StartInterval;
	FMemMark Mark(FMemStack::Get());
	TSet<RowHandle, DefaultKeyFuncs<RowHandle>, FMemStackSetAllocator> VisitedRows;
	TArray<FIntervalEntry, TMemStackAllocator<>> Stack;
	Stack.Push({Row, false});

	while (!Stack.IsEmpty())
	{
		const auto [CurrentRow, bExiting] = Stack.Pop();

		// On exit, assign IntervalRight (only for non-root rows that have metadata).
		if (bExiting)
		{
			const RowHandle RelationRow = FindRelationRow(Type, CurrentRow);
			if (RelationRow != InvalidRowHandle)
			{
				if (FIntervalEncodedHierarchyMetadata* Metadata = ReadIntervalMetadataMutable(Provider, RelationRow))
				{
					CurrentInterval += Gap;
					Metadata->IntervalRight = CurrentInterval;

					if (TypeVersion)
					{
						Metadata->IntervalVersion = *TypeVersion;
					}
				}
			}
			continue;
		}

		// Cycle detection (shared for both root and non-root nodes).
		if (VisitedRows.Contains(CurrentRow))
		{
			ensureMsgf(false, TEXT("AssignIntervals: cycle detected in hierarchy"));
			continue;
		}
		VisitedRows.Add(CurrentRow);

		// Assign IntervalLeft for non-root rows that have metadata.
		const RowHandle RelationRow = FindRelationRow(Type, CurrentRow);
		if (RelationRow != InvalidRowHandle)
		{
			if (FIntervalEncodedHierarchyMetadata* Metadata = ReadIntervalMetadataMutable(Provider, RelationRow))
			{
				CurrentInterval += Gap;
				Metadata->IntervalLeft = CurrentInterval;
			}
		}

		// Push exit marker, then descendants in reverse order for left-to-right DFS.
		Stack.Push({CurrentRow, true});

		TArray<RowHandle, TInlineAllocator<16>> DescendantRows;
		Provider.ForEachRelationSubject(Type, CurrentRow, [&DescendantRows](RowHandle DescendantRow) { DescendantRows.Add(DescendantRow); });
		for (int32 Index = DescendantRows.Num() - 1; Index >= 0; --Index)
		{
			Stack.Push({DescendantRows[Index], false});
		}
	}

	return CurrentInterval;
}

} // namespace UE::Editor::DataStorage
