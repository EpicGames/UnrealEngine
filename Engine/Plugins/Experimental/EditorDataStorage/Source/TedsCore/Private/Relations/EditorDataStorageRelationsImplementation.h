// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Containers/Map.h"
#include "MassEntityRelations.h"
#include "MassRelationManager.h"
#include "UObject/StrongObjectPtr.h"

struct FMassEntityManager;
struct FWalkOnlyHierarchyMetadata;
struct FIntervalEncodedHierarchyMetadata;

namespace UE::Editor::DataStorage
{
	class FTableManager;
	/** Maps TEDS relation types to their underlying Mass representation. */
	class FTedsRelationAdapter final
	{
	public:
		FTedsRelationAdapter() = default;
		~FTedsRelationAdapter() = default;

		// Non-copyable
		FTedsRelationAdapter(const FTedsRelationAdapter&) = delete;
		FTedsRelationAdapter& operator=(const FTedsRelationAdapter&) = delete;

		/** Register a new relation type. Returns the handle, or InvalidRelationTypeHandle on failure. */
		RelationTypeHandle RegisterRelationType(
			FMassEntityManager& EntityManager,
			FTableManager& TableManager,
			const FRelationRegistrationParams& Params);

		/** Find a relation type by name. Returns InvalidRelationTypeHandle if not found. */
		RelationTypeHandle FindRelationType(const FName& Name) const;

		bool IsValidRelationType(RelationTypeHandle Type) const;
		const FTedsRelationTraits* GetTraits(RelationTypeHandle Type) const;
		UE::Mass::FTypeHandle GetMassHandle(RelationTypeHandle TedsHandle) const;
		FName GetTypeName(RelationTypeHandle Type) const;
		bool IsHierarchical(RelationTypeHandle Type) const;
		const UScriptStruct* GetRelationTag(RelationTypeHandle Type) const;
		const UScriptStruct* GetSubjectChangedColumn(RelationTypeHandle Type) const;
		const UScriptStruct* GetObjectChangedColumn(RelationTypeHandle Type) const;
		void SetChangedColumns(RelationTypeHandle Type, const UScriptStruct* SubjectChanged, const UScriptStruct* ObjectChanged);
		bool HasMassRelationData(RelationTypeHandle Type) const;
		void SetHasMassRelationData(RelationTypeHandle Type);
		void ListTypes(TFunctionRef<void(RelationTypeHandle, const FName&)> Callback) const;

	private:
		struct FRegisteredType
		{
			FName Name;
			FTedsRelationTraits TedsTraits;
			UE::Mass::FTypeHandle MassHandle;
			TStrongObjectPtr<UScriptStruct> DynamicRelationTag;
			bool bHasMassRelationData = false;
			const UScriptStruct* SubjectChangedColumn = nullptr;
			const UScriptStruct* ObjectChangedColumn = nullptr;
		};

		TArray<FRegisteredType> RegisteredTypes;
		TMap<FName, RelationTypeHandle> NameToHandle;
	};

	/** Manages hierarchy metadata (depth, root, interval encoding) for hierarchical relation types. */
	class FTedsHierarchicalRelationManager final
	{
	public:
		FTedsHierarchicalRelationManager() = default;

		/** Initialize hierarchy metadata for a newly created relation row. */
		void InitializeHierarchyMetadata(
			ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle RelationRow,
			RowHandle Subject,
			RowHandle Object);

		/** Clean up hierarchy metadata when a relation is destroyed. */
		void RemoveHierarchyMetadata(RelationTypeHandle Type, RowHandle Subject);

		/** Check if Descendant is anywhere below Ancestor in the hierarchy. */
		bool IsDescendantOf(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Descendant,
			RowHandle Ancestor,
			bool bIncludeSelf = false) const;

		/** IsDescendantOf overload for use inside query contexts (reads Mass fragments directly). */
		bool IsDescendantOf(
			FMassEntityManager& EntityManager,
			const FTedsRelationAdapter& RelationAdapter,
			RelationTypeHandle Type,
			RowHandle Descendant,
			RowHandle Ancestor,
			bool bIncludeSelf = false) const;

		/** Get the root of the hierarchy containing the given row. */
		RowHandle GetHierarchyRoot(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row) const;

		/** Get the depth of a row in its hierarchy. Roots are depth 0. */
		int32 GetHierarchyDepth(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row) const;

		/** Count descendants by walking the tree. Not cached. */
		int32 ComputeDescendantCount(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row) const;

		/** Get all descendants of a row. */
		void GetDescendants(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row,
			TArray<RowHandle>& OutDescendants) const;

		/** Get all ancestors of a row, ordered parent to root. */
		void GetAncestors(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row,
			TArray<RowHandle>& OutAncestors) const;

		/** Traverse all descendants depth-first. */
		void TraverseDescendants(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row,
			ICoreProvider::FRelationTraversalCallback Callback,
			ICoreProvider::ETraversalOrder Order = ICoreProvider::ETraversalOrder::PreOrder,
			int32 MaxDepth = TNumericLimits<int32>::Max()) const;

		/** Schedule interval rebalancing for this type at the next ProcessRebalancing call. */
		void MarkForRebalance(RelationTypeHandle Type);

		/** Rebalance intervals for all marked types. Called at frame end. */
		void ProcessRebalancing(ICoreProvider& Provider);

		/** Register the hierarchy mode and initial gap for a relation type. */
		void RegisterHierarchyMode(RelationTypeHandle Type, EHierarchyMode Mode, int64 InitialGap);

	private:
		RowHandle FindRelationRow(RelationTypeHandle Type, RowHandle Subject) const;

		// Metadata accessors that dispatch based on the hierarchy mode (WalkOnly vs IntervalEncoded).
		int32 ReadDepth(const ICoreProvider& Provider, RelationTypeHandle Type, RowHandle RelationRow) const;
		RowHandle ReadRoot(const ICoreProvider& Provider, RelationTypeHandle Type, RowHandle RelationRow) const;
		const FIntervalEncodedHierarchyMetadata* ReadIntervalMetadata(const ICoreProvider& Provider, RowHandle RelationRow) const;
		FIntervalEncodedHierarchyMetadata* ReadIntervalMetadataMutable(ICoreProvider& Provider, RowHandle RelationRow) const;
		FWalkOnlyHierarchyMetadata* ReadWalkOnlyMetadataMutable(ICoreProvider& Provider, RowHandle RelationRow) const;

		void TraversePreOrder(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle StartRow,
			int32 MaxDepth,
			ICoreProvider::FRelationTraversalCallback& Callback) const;

		void TraversePostOrder(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle StartRow,
			int32 MaxDepth,
			ICoreProvider::FRelationTraversalCallback& Callback) const;

		void GatherDescendants(
			const ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row,
			TArray<RowHandle>& OutDescendants) const;

		/** Assign interval [Left, Right] values to all nodes in a subtree via DFS. Returns the next available interval. */
		int64 AssignIntervals(
			ICoreProvider& Provider,
			RelationTypeHandle Type,
			RowHandle Row,
			int64 StartInterval,
			int64 Gap);

		// (RelationType, SubjectRow) -> RelationRow lookup for hierarchy metadata.
		TMap<RelationTypeHandle, TMap<RowHandle, RowHandle>> SubjectToRelationRow;

		struct FHierarchySettings
		{
			EHierarchyMode Mode = EHierarchyMode::Disabled;
			int64 InitialGap = 1000;
		};
		TMap<RelationTypeHandle, FHierarchySettings> TypeSettings;

		// Per-type interval version counters. Starts at 1; 0 means "never stamped" (always falls back to walk).
		TMap<RelationTypeHandle, uint32> TypeVersions;

		// Types queued for interval rebalancing at next ProcessRebalancing call.
		TSet<RelationTypeHandle> TypesNeedingRebalance;
	};

} // namespace UE::Editor::DataStorage
