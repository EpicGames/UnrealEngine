// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scope/TypedElementScopeSystem.h"

#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Scope/EditorDataScopeColumns.h"
#include "DataStorage/Scope/EditorDataScopeVersion.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/PlatformStackWalk.h"

namespace UE::Editor::DataStorage
{

static const FName GScopeHierarchyName = TEXT("EditorScope");
static const FName GScopeTableName = TEXT("EditorScopeTable");

FScopeSystem::FScopeSystem(ICoreProvider& InDataStorage)
	: DataStorage(InDataStorage)
{
}

TableHandle FScopeSystem::GetOrCreateScopeTable()
{
	TableHandle Table = DataStorage.FindTable(GScopeTableName);
	if (Table == InvalidTableHandle)
	{
		Table = DataStorage.RegisterTable<FDataStorageScopeDataTag, FScopeRowSourceInfo, FScopeDataVersionColumn, FTypedElementLabelColumn>(
			GScopeTableName);
	}
	return Table;
}

// ============================================================================
// Scope Row Lifecycle & Hierarchy
// ============================================================================

RowHandle FScopeSystem::AddScopeRow()
{
	return AddScopeRow(FStringView());
}

RowHandle FScopeSystem::AddScopeRow(FStringView Label)
{
	RowHandle Row = DataStorage.AddRow(GetOrCreateScopeTable());

	// Capture callstack at creation site for debugger visibility
	FScopeRowSourceInfo SourceInfo;
	SourceInfo.CallstackDepth = FPlatformStackWalk::CaptureStackBackTrace(
		SourceInfo.Callstack, FScopeRowSourceInfo::MaxCallstackDepth);
	DataStorage.SetScopeData<FScopeRowSourceInfo>(Row, MoveTemp(SourceInfo));

	const FString LabelString = Label.IsEmpty() ? LexToString(Row) : FString(Label);
	SetScopeData(Row, FTypedElementLabelColumn::StaticStruct(),
		[&LabelString](void* ColumnData, const UScriptStruct&)
		{
			new(ColumnData) FTypedElementLabelColumn{ .Label = LabelString };
		},
		[](const UScriptStruct&, void* Destination, void* Source)
		{
			*static_cast<FTypedElementLabelColumn*>(Destination) = MoveTemp(*static_cast<FTypedElementLabelColumn*>(Source));
		});

	return Row;
}

RowHandle FScopeSystem::AddScopeRow(FStringView Label, RowHandle Parent)
{
	RowHandle Row = AddScopeRow(Label);
	SetParentScope(Row, Parent);
	return Row;
}

void FScopeSystem::RemoveScopeRow(RowHandle Row)
{
	DataStorage.RemoveRow(Row);
}

void FScopeSystem::SetParentScope(RowHandle Child, RowHandle Parent)
{
	DataStorage.SetParentRow(GetScopeHierarchy(), Child, Parent);
}

RowHandle FScopeSystem::GetParentScope(RowHandle Row)
{
	return DataStorage.GetParentRow(GetScopeHierarchy(), Row);
}

// ============================================================================
// Scope Data Lookup (hierarchy-walking)
// ============================================================================

const void* FScopeSystem::GetScopeDataRaw(RowHandle Row, const UScriptStruct* ColumnType)
{
	RowHandle Current = Row;
	while (DataStorage.IsRowAvailable(Current))
	{
		const void* Data = DataStorage.GetColumnData(Current, ColumnType);
		if (Data != nullptr)
		{
			return Data;
		}
		Current = GetParentScope(Current);
	}

	return nullptr;
}

// ============================================================================
// Visible Scope Columns (hierarchy-walking)
// ============================================================================

TArray<const UScriptStruct*> FScopeSystem::GetAllVisibleScopeColumns(RowHandle Row)
{
	TArray<const UScriptStruct*> Result;
	TSet<const UScriptStruct*> SeenTypes;

	RowHandle Current = Row;
	while (DataStorage.IsRowAvailable(Current))
	{
		DataStorage.ListColumns(Current, [&](const UScriptStruct& ColumnType)
		{
			if (!SeenTypes.Contains(&ColumnType))
			{
				SeenTypes.Add(&ColumnType);
				Result.Add(&ColumnType);
			}
		});
		Current = GetParentScope(Current);
	}
	return Result;
}

// ============================================================================
// Versioned Scope Data Operations (single-row)
// ============================================================================

Scope::FScopeDataVersion FScopeSystem::SetScopeData(
	RowHandle Row, const UScriptStruct* ColumnType,
	ColumnCreationCallbackRef Initializer, ColumnCopyOrMoveCallback Relocator)
{
	DataStorage.AddColumnData(Row, ColumnType, Initializer, Relocator);

	FScopeDataVersionColumn* VersionCol = DataStorage.GetColumn<FScopeDataVersionColumn>(Row);
	if (!VersionCol)
	{
		return Scope::FScopeDataVersion();
	}

	return Scope::FScopeDataVersion::Make(Row, ++VersionCol->Version);
}

bool FScopeSystem::RemoveScopeData(RowHandle Row, const UScriptStruct* ColumnType)
{
	if (!DataStorage.HasColumns(Row, TConstArrayView<const UScriptStruct*>(&ColumnType, 1)))
	{
		return false;
	}

	DataStorage.RemoveColumn(Row, ColumnType);

	if (FScopeDataVersionColumn* VersionCol = DataStorage.GetColumn<FScopeDataVersionColumn>(Row))
	{
		++VersionCol->Version;
	}

	return true;
}

// ============================================================================
// Versioned Hierarchy Walk
// ============================================================================

Scope::FScopeDataVersion FScopeSystem::GetScopeDataVersion(
	RowHandle Row, const UScriptStruct* ColumnType)
{
	RowHandle Current = Row;
	while (DataStorage.IsRowAvailable(Current))
	{
		if (DataStorage.GetColumnData(Current, ColumnType) != nullptr)
		{
			if (const FScopeDataVersionColumn* VersionCol =
					DataStorage.GetColumn<FScopeDataVersionColumn>(Current))
			{
				return Scope::FScopeDataVersion::Make(Current, VersionCol->Version);
			}
		}
		Current = GetParentScope(Current);
	}

	return Scope::FScopeDataVersion();
}

// ============================================================================
// Scope Hierarchy
// ============================================================================

FHierarchyHandle FScopeSystem::GetScopeHierarchy()
{
	FHierarchyHandle Handle = DataStorage.FindHierarchyByName(GScopeHierarchyName);
	if (!DataStorage.IsValidHierarchyHandle(Handle))
	{
		FHierarchyRegistrationParams Params;
		Params.Name = GScopeHierarchyName;
		Params.bEnableParentChangedColumn = true;
		Handle = DataStorage.RegisterHierarchy(Params);
	}
	return Handle;
}

// ============================================================================
// Root Scope
// ============================================================================

RowHandle FScopeSystem::GetRootScope() const
{
	return RootScopeRow;
}

void FScopeSystem::InitRootScope()
{
	if (RootScopeRow != InvalidRowHandle)
	{
		return;
	}
	RootScopeRow = AddScopeRow(TEXT("Root"));
	Scope::SetCurrentScope(RootScopeRow);
}

} // namespace UE::Editor::DataStorage
