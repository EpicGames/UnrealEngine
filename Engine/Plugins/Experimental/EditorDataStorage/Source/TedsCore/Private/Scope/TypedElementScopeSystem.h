// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Scope/EditorDataScopeTypes.h"
#include "DataStorage/Handles.h"

class UScriptStruct;

namespace UE::Editor::DataStorage
{

class ICoreProvider;

class FScopeSystem
{
public:
	explicit FScopeSystem(ICoreProvider& InDataStorage);

	RowHandle AddScopeRow();
	RowHandle AddScopeRow(FStringView Label);
	RowHandle AddScopeRow(FStringView Label, RowHandle Parent);
	void RemoveScopeRow(RowHandle Row);
	void SetParentScope(RowHandle Child, RowHandle Parent);
	RowHandle GetParentScope(RowHandle Row);
	const void* GetScopeDataRaw(RowHandle Row, const UScriptStruct* ColumnType);
	Scope::FScopeDataVersion GetScopeDataVersion(RowHandle Row, const UScriptStruct* ColumnType);
	Scope::FScopeDataVersion SetScopeData(RowHandle Row, const UScriptStruct* ColumnType,
		ColumnCreationCallbackRef Initializer, ColumnCopyOrMoveCallback Relocator);
	bool RemoveScopeData(RowHandle Row, const UScriptStruct* ColumnType);
	TArray<const UScriptStruct*> GetAllVisibleScopeColumns(RowHandle Row);
	FHierarchyHandle GetScopeHierarchy();
	RowHandle GetRootScope() const;
	void InitRootScope();

private:
	TableHandle GetOrCreateScopeTable();

	ICoreProvider& DataStorage;
	RowHandle RootScopeRow = InvalidRowHandle;
};

} // namespace UE::Editor::DataStorage
