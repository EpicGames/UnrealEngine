// Copyright Epic Games, Inc. All Rights Reserved.

#include "TableManager.h"

#include <cinttypes>
#include "DataStorage/Debug/Log.h"
#include "DynamicColumnGenerator.h"
#include "EditorDataStorageSettings.h"
#include "Elements/Framework/TypedElementQueryContract.h"
#include "MassEntityManager.h"
#include "MassEntityRelations.h"
#include "MassEntityTypes.h"
#include "TypedElementUtils.h"

namespace UE::Editor::DataStorage
{
	FTableManager::FTableManager(FMassEntityManager& InMassEntityManager, FDynamicColumnGenerator& InDynamicColumns)
		: MassEntityManager(InMassEntityManager)
		, DynamicColumns(InDynamicColumns)
	{
		OnNewArchetypeEventHandle = InMassEntityManager.GetOnNewArchetypeEvent().AddRaw(this, &FTableManager::OnNewArchetype);
	}

	FTableManager::~FTableManager()
	{
		MassEntityManager.GetOnNewArchetypeEvent().Remove(OnNewArchetypeEventHandle);
		Clear();
	}

	void FTableManager::Clear()
	{
		Tables.Reset();
		NameLookup.Reset();
		MassLookup.Reset();
	}

	bool FTableManager::IsTableValid(TableHandle Table) const
	{
		return Table != InvalidTableHandle && Table < Tables.Num();
	}

	TableHandle FTableManager::Register(TConstArrayView<const UScriptStruct*> ColumnList, const FName& TableName)
	{
		return Register(ColumnList, TableName, {});
	}

	TableHandle FTableManager::Register(
		TConstArrayView<const UScriptStruct*> ColumnList, const FName& TableName, FTableRegistrationOptions Options)
	{
		checkf(!NameLookup.Contains(TableName), TEXT("Table '%s' has already been registered with the editor data storage."),
			*TableName.ToString());
		TableHandle Result = Tables.Num();

		ETableType Type;
		FMassArchetypeCompositionDescriptor Composition;
		if (IsTableValid(Options.SourceTable))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Options.SourceTable);
			FTableInfo& TableInfo = Tables[TableHandleAsIndex];
			TableInfo.Derivatives.Add(Result);
			Composition = MassEntityManager.GetArchetypeComposition(TableInfo.Archetype);
			RemoveTableTag(Composition);
			Type = ETableType::Derivative;

			UE_LOGF(LogEditorDataStorage, VeryVerbose, "New table derivative '%ls' from template '%ls", 
				*TableName.ToString(), 
				*TableInfo.Name.ToString());
		}
		else
		{
			Type = ETableType::Declared;

			UE_LOGF(LogEditorDataStorage, VeryVerbose, "New table declared '%ls'", *TableName.ToString());
		}

		FMassArchetypeCreationParams ArchetypeCreationParams;
		ArchetypeCreationParams.DebugName = TableName;
		ArchetypeCreationParams.ChunkMemorySize = Options.MemorySize == 0 ? GetTableChunkSize(TableName) : Options.MemorySize;

		for (const UScriptStruct* Column : ColumnList)
		{
			Composition.Add(Column);
		}
		// Add table tag.
		const UScriptStruct* TableTag = DynamicColumns.GenerateColumn(*FTableDynamicTag::StaticStruct(), TableName).Type;
		checkf(TableTag, TEXT("Failed to generate table tag for table '%s'"), *TableName.ToString());
		Composition.Add(TableTag);

		bRegisteringNewTable = true;
		FMassArchetypeHandle Archetype = MassEntityManager.CreateArchetype(Composition, ArchetypeCreationParams);
		bRegisteringNewTable = false;

		Tables.Emplace(FTableInfo
			{
				.Archetype = Archetype,
				.Name = TableName,
				.Type = Type,
				.Tag = TNonNullPtr<const UScriptStruct>(TableTag),
				.Parent = Options.SourceTable,
			});
		NameLookup.Add(TableName, Result);
		MassLookup.Add(Archetype, Result);
		return Result;
	}

	TableHandle FTableManager::Find(const FName& Name) const
	{
		const TableHandle* TableHandle = NameLookup.Find(Name);
		return TableHandle ? *TableHandle : InvalidTableHandle;
	}

	TableHandle FTableManager::Find(const FMassArchetypeHandle& Archetype) const
	{
		const TableHandle* TableHandle = MassLookup.Find(Archetype);
		return TableHandle ? *TableHandle : InvalidTableHandle;
	}

	TableHandle FTableManager::Find(RowHandle Row) const
	{
		return Find(MassEntityManager.GetArchetypeForEntity(FMassEntityHandle::FromNumber(Row)));
	}

	const UScriptStruct* FTableManager::GetTagForTable(TableHandle Table) const
	{
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			return Tables[TableHandleAsIndex].Tag;
		}
		else
		{
			return nullptr;
		}
	}

	void FTableManager::List(TFunctionRef<void(TableHandle)> Callback) const
	{
		int32 TableCount = Tables.Num();
		for (int32 Table = 0; Table < TableCount; Table++)
		{
			Callback(Table);
		}
	}

	void FTableManager::List(ETableType TableType, TFunctionRef<void(TableHandle)> Callback) const
	{
		int32 Table = 0;
		for (const FTableInfo& TableInfo : Tables)
		{
			if (TableInfo.Type == TableType)
			{
				Callback(Table);
			}
			Table++;
		}
	}

	void FTableManager::ListColumns(TableHandle Table, TFunctionRef<bool(const UScriptStruct*)> Callback) const
	{
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			MassEntityManager.ForEachArchetypeElementType(Tables[TableHandleAsIndex].Archetype, Callback);
		}
	}

	bool FTableManager::HasColumns(TableHandle Table, TConstArrayView<const UScriptStruct*> Columns) const
	{
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			FMassArchetypeCompositionDescriptor Composition = MassEntityManager.GetArchetypeComposition(Tables[TableHandleAsIndex].Archetype);
			for (const UScriptStruct* Column : Columns)
			{
				if (!Composition.Contains(Column))
				{
					return false;
				}
			}
			return true;
		}
		else
		{
			return false;
		}
	}

	FTableInfoView FTableManager::GetInfo(TableHandle Table) const
	{
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			const FTableInfo& Info = Tables[TableHandleAsIndex];
			return FTableInfoView
			{
				.Variants = Info.Variants,
				.Derivatives = Info.Derivatives,
				.Name = Info.Name,
				.Type = Info.Type,
				.Parent = Info.Parent
			};
		}
		else
		{
			return FTableInfoView();
		}
	}

	void FTableManager::ListRows(TableHandle Table, TFunctionRef<void (FRowHandleArrayView)> Callback) const
	{
		if (FMassArchetypeHandle Archetype = GetArchetype(Table); Archetype.IsValid())
		{
			MassEntityManager.ForEachArchetypeChunkEntityList(Archetype,
				[&Callback](TConstArrayView<FMassEntityHandle> EntityHandles)
				{
					Callback(FRowHandleArrayView(MassEntitiesToRowsConversion(EntityHandles), FRowHandleArrayView::EFlags::IsUnique));
				});
		}
	}

	void FTableManager::RegisterForeignKey(TableHandle Table, const FName& Domain, Queries::TConstQueryFunction<FForeignKey> KeyConstructor)
	{
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			Tables[TableHandleAsIndex].ForeignKeyConstructors.FindOrAdd(Domain) = MoveTemp(KeyConstructor);
		}
	}

	Queries::EFlowControl FTableManager::BuildForeignKeys(TArrayView<FForeignKey> Results, const FName& Domain,
		Queries::TForwardingConstQueryContext<Queries::CurrentTableInfo> Contract, Queries::IQueryFunctionResponse& Response) const
	{
		using namespace UE::Editor::DataStorage::Queries;

		struct FQueryResult : TResult<FForeignKey>
		{
			TArrayView<FForeignKey> Results;
			int32 Index = 0;
			virtual void Add(RowHandle, FForeignKey Key) override
			{
				Results[Index++] = MoveTemp(Key);
			}
		};

		FQueryResult QueryResult;
		QueryResult.Results = Results;

		TableHandle Table = Contract.GetCurrentTable();
		if (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			const FTableInfo* TableInfo = &Tables[TableHandleAsIndex];
			const TConstQueryFunction<FForeignKey>* KeyConstructor = TableInfo->ForeignKeyConstructors.Find(Domain);
			while(true)
			{
				if (KeyConstructor)
				{
					return KeyConstructor->Call<
						EFunctionCallConfig::VerifyCapabilityCompatibility |
						EFunctionCallConfig::ReportCapabilityCompatibility |
						EFunctionCallConfig::VerifyColumns
					>(QueryResult, Contract, Response);
				}
				else if (IsTableValid(TableInfo->Parent))
				{
					TableHandleAsIndex = ConvertTableHandleToIndex(TableInfo->Parent);
					checkf(TableHandleAsIndex < Tables.Num(),
						TEXT("Unable to retrieve internal archetype in editor storage because table handle %" PRIu64 " is invalid."), Table);
					TableInfo = &Tables[TableHandleAsIndex];
					KeyConstructor = TableInfo->ForeignKeyConstructors.Find(Domain);
				}
				else
				{
					break;
				}
			}
		}
		return Queries::EFlowControl::Break;
	}

	void FTableManager::ListForeignKeyDomains(TableHandle Table, bool bIncludeParents, TFunctionRef<void(const FName&)> Callback) const
	{
		using ForeignKeyMapConstIterator = FTableInfo::ForeignKeyMap::TConstIterator;

		while (IsTableValid(Table))
		{
			int32 TableHandleAsIndex = ConvertTableHandleToIndex(Table);
			const FTableInfo& TableInfo = Tables[TableHandleAsIndex];
			for (ForeignKeyMapConstIterator It = TableInfo.ForeignKeyConstructors.CreateConstIterator(); It; ++It)
			{
				Callback(It.Key());
			}

			Table = bIncludeParents ? TableInfo.Parent : InvalidTableHandle;
		}
	}

	FMassArchetypeHandle FTableManager::GetArchetype(TableHandle Table) const
	{
		return IsTableValid(Table)
			? Tables[ConvertTableHandleToIndex(Table)].Archetype
			: FMassArchetypeHandle();
	}

	void FTableManager::OnNewArchetype(const FMassArchetypeHandle& Archetype)
	{
		if (!bRegisteringNewTable)
		{
			const UScriptStruct* TableTag = nullptr;
			const UScriptStruct* RelationTag = nullptr;

			FMassArchetypeCompositionDescriptor Composition = MassEntityManager.GetArchetypeComposition(Archetype);
			const FMassElementBitSet& Columns = Composition.GetElementsBitSet();
			Columns.ExportTypes([&TableTag, &RelationTag](const UScriptStruct* Type)
				{
					if (Type->IsChildOf(FTableDynamicTag::StaticStruct()))
					{
						TableTag = Type;
						return false;
					}
					if (Type->IsChildOf(FMassRelation::StaticStruct()))
					{
						RelationTag = Type;
					}
					return true;
				});

			// Relation entity archetypes are created by Mass's FRelationManager and
			// don't carry TEDS table tags. Look up the parent table using the per-
			// relation-type tag name -- both the tag and the registered table share the
			// same "TedsRelation_{name}" name, so no transformation is needed.
			if (!TableTag && RelationTag)
			{
				FName ParentFName = FName(RelationTag->GetName());
				TableHandle Parent = Find(ParentFName);
				if (IsTableValid(Parent))
				{
					int32 NewTable = Tables.Emplace(FTableInfo
						{
							.Archetype = Archetype,
							.Name = ParentFName,
							.Type = ETableType::Variant,
							.Tag = RelationTag,
							.Parent = Parent
						});
					MassLookup.Add(Archetype, NewTable);
					Tables[ConvertTableHandleToIndex(Parent)].Variants.Add(NewTable);
					UE_LOGF(LogEditorDataStorage, VeryVerbose, "New relation table variant for '%ls'", *ParentFName.ToString());
				}
				return;
			}

			checkf(TableTag, TEXT("Expected a table tag to be on a variant archetype. This function shouldn't be called on new archetypes."));
			const FString& ParentName = TableTag->GetMetaData(TEXT("EditorDataStorage_DerivedFromDynamicTemplate"));
			FName  ParentFName = FName(ParentName);
			TableHandle Parent = Find(ParentFName);
			checkf(IsTableValid(Parent), TEXT("Unable to register a table variant because parent table '%s' wasn't found."), *ParentName);
			int32 NewTable = Tables.Emplace(FTableInfo
				{
					.Archetype = Archetype,
					.Name = ParentFName,
					.Type = ETableType::Variant,
					.Tag = TNonNullPtr<const UScriptStruct>(TableTag),
					.Parent = Parent,
				});
			TableHandle Variant = MassLookup.Add(Archetype, NewTable);
			Tables[ConvertTableHandleToIndex(Parent)].Variants.Add(NewTable);

			UE_LOGF(LogEditorDataStorage, VeryVerbose, "New table variant for '%ls'", *ParentName);
		}
	}

	int32 FTableManager::ConvertTableHandleToIndex(const TableHandle InTableHandle) const
	{
		checkf(
			InTableHandle != InvalidTableHandle &&
			(InTableHandle <= static_cast<TableHandle>(std::numeric_limits<int32>::max()) && 
			InTableHandle >= 0),
			TEXT("Invalid editor data storage table handle: %" PRIu64 "."), InTableHandle);
		return static_cast<int32>(InTableHandle);
	}

	int32 FTableManager::GetTableChunkSize(const FName& TableName) const
	{
		const UEditorDataStorageSettings* Settings = GetDefault<UEditorDataStorageSettings>();
		const EChunkMemorySize* TableSpecificSize = Settings->TableSpecificChunkMemorySize.Find(TableName);
		return TableSpecificSize
			? static_cast<int32>(*TableSpecificSize)
			: static_cast<int32>(Settings->ChunkMemorySize);
	}

	void FTableManager::RemoveTableTag(FMassArchetypeCompositionDescriptor& Composition)
	{
		FMassElementBitSet& Columns = Composition.GetElementsBitSet();
		Columns.ExportTypes([&Columns](const UScriptStruct* TagType)
			{
				if (TagType->IsChildOf(FTableDynamicTag::StaticStruct()))
				{
					Columns.Remove(TagType);
					return false;
				}
				return true;
			});
	}
} // namespace UE::Editor::DataStorage
