// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "Delegates/IDelegateInstance.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "MassArchetypeTypes.h"
#include "UObject/NameTypes.h"

#include "TableManager.generated.h"

struct FMassArchetypeCompositionDescriptor;
struct FMassEntityManager;
class UScriptStruct;

/**
 * Base type for the tag that automatically gets added to any table to identify it.
 */
USTRUCT(meta = (DisplayName = "Table", EditorDataStorage_DynamicColumnTemplate))
struct FEditorDataStorageTableDynamicTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

namespace UE::Editor::DataStorage
{
	using FTableDynamicTag = FEditorDataStorageTableDynamicTag;

	class FDynamicColumnGenerator;

	class FTableManager final
	{
	public:
		FTableManager(FMassEntityManager& InMassEntityManager, FDynamicColumnGenerator& InDynamicColumns);
		~FTableManager();

		void Clear();

		TableHandle Register(TConstArrayView<const UScriptStruct*> ColumnList, const FName& TableName);
		TableHandle Register(
			TConstArrayView<const UScriptStruct*> ColumnList, const FName& TableName, FTableRegistrationOptions Options);
		
		bool IsTableValid(TableHandle Table) const;

		TableHandle Find(const FName& TableName) const;
		TableHandle Find(const FMassArchetypeHandle& Archetype) const;
		TableHandle Find(RowHandle Row) const;
		
		const UScriptStruct* GetTagForTable(TableHandle Table) const;
		
		void List(TFunctionRef<void(TableHandle)> Callback) const;
		void List(ETableType TableType, TFunctionRef<void(TableHandle)> Callback) const;
		
		void ListColumns(TableHandle Table, TFunctionRef<bool(const UScriptStruct*)> Callback) const;
		bool HasColumns(TableHandle Table, TConstArrayView<const UScriptStruct*> Columns) const;
		
		FTableInfoView GetInfo(TableHandle Table) const;
		void ListRows(TableHandle Table, TFunctionRef<void(FRowHandleArrayView)> Callback) const;

		void RegisterForeignKey(TableHandle Table, const FName& Domain, Queries::TConstQueryFunction<FForeignKey> KeyConstructor);
		Queries::EFlowControl BuildForeignKeys(TArrayView<FForeignKey> Results, const FName& Domain,
			Queries::TForwardingConstQueryContext<Queries::CurrentTableInfo> Contract, Queries::IQueryFunctionResponse& Response) const;
		void ListForeignKeyDomains(TableHandle Table, bool bIncludeParents, TFunctionRef<void(const FName&)> Callback) const;

		FMassArchetypeHandle GetArchetype(TableHandle Table) const;
	private:
		struct FTableInfo
		{
			using ForeignKeyMap = TMap<FName, Queries::TConstQueryFunction<FForeignKey>>;
			ForeignKeyMap ForeignKeyConstructors;
			TArray<TableHandle> Variants;
			TArray<TableHandle> Derivatives;
			FMassArchetypeHandle Archetype;
			FName Name;
			ETableType Type;
			TNonNullPtr<const UScriptStruct> Tag;
			// If this is a dynamically generated variant then the Parent represents the table used as the basis for the variant. 
			// If this is a derived copy then the Parent represents the table that was passed in as the template for this table.
			TableHandle Parent;
		};

		void OnNewArchetype(const FMassArchetypeHandle& Archetype);

		int32 ConvertTableHandleToIndex(const TableHandle InTableHandle) const;
		int32 GetTableChunkSize(const FName& TableName) const;
		void RemoveTableTag(FMassArchetypeCompositionDescriptor& Composition);

		TArray<FTableInfo> Tables; // Contains tables and variants.
		TMap<FName, TableHandle> NameLookup; // Only includes tables.
		TMap<FMassArchetypeHandle, TableHandle> MassLookup; // Includes tables and variants.

		FDelegateHandle OnNewArchetypeEventHandle;
		FMassEntityManager& MassEntityManager;
		FDynamicColumnGenerator& DynamicColumns;

		bool bRegisteringNewTable = false;
	};
} // namespace UE::Editor::DataStorage
