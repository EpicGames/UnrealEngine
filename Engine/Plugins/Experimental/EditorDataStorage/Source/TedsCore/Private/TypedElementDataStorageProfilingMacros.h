// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfilingDebugging/CpuProfilerTrace.h"

#ifndef ENABLE_TEDS_PROFILING_MACROS
#define ENABLE_TEDS_PROFILING_MACROS 1
#endif

#if ENABLE_TEDS_PROFILING_MACROS

#define TEDS_EVENT_PREFIX TEXT("[TEDS] ")
// Scoped event which prefixes the string literal with "[TEDS] " for consistent filtering
#define TEDS_EVENT_SCOPE(StrLiteral) TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEDS_EVENT_PREFIX StrLiteral)

#else

#define TEDS_EVENT_SCOPE(StrLiteral) do {} while(false)

#endif