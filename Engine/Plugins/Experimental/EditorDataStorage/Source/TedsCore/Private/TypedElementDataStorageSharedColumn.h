// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TypedElementDataStorageSharedColumn.generated.h"

/*
 * Shared Columns are virtual columns shared by many rows
 */
USTRUCT()
struct FTedsSharedColumn
{
	GENERATED_BODY()
};