// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabase.h"

#include <cinttypes>
#include "DataStorage/Features.h"
#include "DataStorage/Debug/Log.h"
#include "Relations/EditorDataStorageRelationColumns.h"
#include "Relations/EditorDataStorageRelationsImplementation.h"
#include "Editor.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/World.h"
#include "GlobalLock.h"
#include "Math/UnrealMathUtility.h"
#include "MassEntityEditorSubsystem.h"
#include "MassEntityManager.h"
#include "MassEntityRelations.h"
#include "MassEntityTypes.h"
#include "MassRelationManager.h"
#include "MassProcessor.h"
#include "MassSubsystemAccess.h"
#include "Processors/TypedElementProcessorAdaptors.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Stats/Stats.h"
#include "TickTaskManagerInterface.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementDataStorageProfilingMacros.h"
#include "TypedElementUtils.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementDatabase)

#if COUNTERSTRACE_ENABLED

DECLARE_STATS_GROUP(TEXT("Editor Data Storage (Core)"), STATGROUP_TedsCore, STATCAT_Advanced);

DECLARE_FLOAT_COUNTER_STAT(TEXT("Coop task time (ms)"), STAT_TedsCore_CoopTaskTimeMs, STATGROUP_TedsCore);
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsCore_CoopTaskTimeMs, TEXT("TedsCore/Tick/CoopTaskTimeMs"));

#endif

const FName UEditorDataStorage::TickGroupName_Default(TEXT("Default"));
const FName UEditorDataStorage::TickGroupName_PreUpdate(TEXT("PreUpdate"));
const FName UEditorDataStorage::TickGroupName_Update(TEXT("Update"));
const FName UEditorDataStorage::TickGroupName_PostUpdate(TEXT("PostUpdate"));
const FName UEditorDataStorage::TickGroupName_SyncWidget(TEXT("SyncWidgets"));
const FName UEditorDataStorage::TickGroupName_SyncExternalToDataStorage(TEXT("SyncExternalToDataStorage"));
const FName UEditorDataStorage::TickGroupName_SyncDataStorageToExternal(TEXT("SyncDataStorageToExternal"));

FAutoConsoleCommandWithOutputDevice PrintQueryCallbacksConsoleCommand(
	TEXT("TEDS.PrintQueryCallbacks"),
	TEXT("Prints out a list of all processors."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			using namespace UE::Editor::DataStorage;
			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				DataStorage->DebugPrintQueryCallbacks(Output);
			}
		}));

FAutoConsoleCommandWithOutputDevice PrintSupportedColumnsConsoleCommand(
	TEXT("TEDS.PrintSupportedColumns"),
	TEXT("Prints out a list of available Data Storage columns."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			Output.Log(TEXT("The Editor Data Storage supports the following columns:"));

			for (TObjectIterator<const UScriptStruct> It; It; ++It)
			{
				if (UE::Mass::IsA<FMassFragment>(*It) || UE::Mass::IsA<FMassTag>(*It))
				{
					int32 StructureSize = It->GetStructureSize();
					if (StructureSize >= 1024 * 1024)
					{
						Output.Logf(TEXT("    [%6.2f mib] %s"), static_cast<float>(StructureSize) / (1024.0f * 1024.0f), *It->GetFullName());
					}
					else if (StructureSize >= 1024)
					{
						Output.Logf(TEXT("    [%6.2f kib] %s"), static_cast<float>(StructureSize) / 1024.0f, *It->GetFullName());
					}
					else
					{
						Output.Logf(TEXT("    [%6i b  ] %s"), StructureSize, *It->GetFullName());
					}
				}
			}
			Output.Log(TEXT("End of Typed Elements Data Storage supported column list."));
		}));

static TAutoConsoleVariable<int32> CooperativeTasks_TargetFrameRate(TEXT("TEDS.Feature.CooperativeTasks.TargetFrameRate"), 60,
	TEXT("TEDS runs additional cooperative background tasks with the time that is left at the end of a frame. Adjust this value to set "
		"how much TEDS considers as available time. Clamped between 16 and 120 fps."));

static TAutoConsoleVariable<float> CooperativeTasks_MinTimePerFrameMs(TEXT("TEDS.Feature.CooperativeTasks.MinTimePerFrameMs"), 2.0f,
	TEXT("The minimal amount of time in milliseconds that TEDS will spend guaranteed on cooperative tasks."));

static TAutoConsoleVariable<float> CooperativeTasks_MinTimePerTaskMs(TEXT("TEDS.Feature.CooperativeTasks.MinTimePerTaskMs"), 1.0f,
	TEXT("The minimal amount of time in milliseconds that needs to remain for TEDS to consider running another cooperative task."));

static TAutoConsoleVariable<float> CooperativeTasks_ReportThresholdMs(TEXT("TEDS.Feature.CooperativeTasks.ReportThresholdMs"), 1.0f,
	TEXT("The maximum amount of time in milliseconds that cooperative tasks are allowed to go over their allotted time before they're reported."));

static TAutoConsoleVariable<int32> CooperativeTasks_MaxIdleFrames(TEXT("TEDS.Feature.CooperativeTasks.MaxIdleFrames"), 5,
	TEXT("The maximum number of frames the cooperative task scheduler will remain idle during low frame rates before starting to call tasks ")
	TEXT("again with the minimum time per task. This allows the editor to recover from spikes and avoids moving to emergency execution for ")
	TEXT("smaller interruptions due to for instance OS threading changes, but prevents cooperative tasks from starving."));

namespace UE::Editor::DataStorage::Private
{
	struct ColumnsToBitSetsResult
	{
		bool bMustUpdateFragments = false;
		bool bMustUpdateTags = false;
		
		bool MustUpdate() const { return bMustUpdateFragments || bMustUpdateTags; }
	};

	void ColumnsToBitSets(TConstArrayView<const UScriptStruct*> Columns, FMassElementBitSet& Elements)
	{
		for (const UScriptStruct* ColumnType : Columns)
		{
			Elements.Add(ColumnType);
		}
	}

	FMassElementBitSet ColumnsToBitSets(TConstArrayView<const UScriptStruct*> Columns)
	{
		FMassElementBitSet Result;
		ColumnsToBitSets(Columns, Result);
		return Result;
	}

	ColumnsToBitSetsResult ColumnsToBitSets(TConstArrayView<const UScriptStruct*> Columns, FMassFragmentBitSet& Fragments, FMassTagBitSet& Tags)
	{
		ColumnsToBitSetsResult Result;

		for (const UScriptStruct* ColumnType : Columns)
		{
			if (UE::Mass::IsA<FMassFragment>(ColumnType))
			{
				Fragments.Add(ColumnType);
				Result.bMustUpdateFragments = true;
			}
			else if (UE::Mass::IsA<FMassTag>(ColumnType))
			{
				Tags.Add(ColumnType);
				Result.bMustUpdateTags = true;
			}
		}
		return Result;
	}
} // namespace UE::Editor::DataStorage::Private

UEditorDataStorage::FActiveCallScope::FActiveCallScope(const UEditorDataStorage& Owner)
	: Owner(Owner)
{
	Owner.ActiveCallsCount++;
}

UEditorDataStorage::FActiveCallScope::~FActiveCallScope()
{
	Owner.ActiveCallsCount--;
	if (Owner.ActiveCallsCount == 0)
	{
		Owner.Environment->FlushCommands();
	}
}

UEditorDataStorage::~UEditorDataStorage()
{
	UE_LOGF(LogEditorDataStorage, Log, "Destroyed TEDS Core.");
}

void UEditorDataStorage::Initialize()
{
	using namespace UE::Editor::DataStorage;

	UE_LOGF(LogEditorDataStorage, Log, "Initializing TEDS Core.");

	UMassEntityEditorSubsystem* Mass = UMassEntityEditorSubsystem::Get();
	check(Mass);
	OnPreMassTickHandle = Mass->GetOnPreTickDelegate().AddUObject(this, &UEditorDataStorage::OnPreMassTick);
	OnPostMassTickHandle = Mass->GetOnPostTickDelegate().AddUObject(this, &UEditorDataStorage::OnPostMassTick);

	ActiveEditorEntityManager = Mass->GetMutableEntityManager();
	ActiveEditorPhaseManager = Mass->GetMutablePhaseManager();
	if (ActiveEditorEntityManager && ActiveEditorPhaseManager)
	{
		Environment = MakeShared<FEnvironment>(*this, *ActiveEditorEntityManager, *ActiveEditorPhaseManager);
		ContextEnvironment.Environment = Environment.Get();
		ContextEnvironment.DirectContext = this;
		
		using PhaseType = std::underlying_type_t<EQueryTickPhase>;
		for (PhaseType PhaseId = 0; PhaseId < static_cast<PhaseType>(EQueryTickPhase::Max); ++PhaseId)
		{
			EQueryTickPhase Phase = static_cast<EQueryTickPhase>(PhaseId);
			EMassProcessingPhase MassPhase = FTypedElementQueryProcessorData::MapToMassProcessingPhase(Phase);

			ActiveEditorPhaseManager->GetOnPhaseStart(MassPhase).AddLambda(
				[this, Phase](float DeltaTime)
				{
					PreparePhase(Phase, DeltaTime);
				});

			ActiveEditorPhaseManager->GetOnPhaseEnd(MassPhase).AddLambda(
				[this, Phase](float DeltaTime)
				{
					FinalizePhase(Phase, DeltaTime);
				});

			// Update external source to TEDS at the start of the phase.
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage),
				Phase, {}, {}, EExecutionMode::Threaded);
			
			// Default group.
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::Default),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage), EExecutionMode::Threaded);

			// Order the update groups.
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::PreUpdate),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::Default), EExecutionMode::Threaded);
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::Update),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::PreUpdate), EExecutionMode::Threaded);
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::PostUpdate),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::Update), EExecutionMode::Threaded);

			// After everything has processed sync the data in TEDS to external sources.
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::PostUpdate), EExecutionMode::Threaded);

			// Update any widgets with data from TEDS.
			RegisterTickGroup(GetQueryTickGroupName(EQueryTickGroups::SyncWidgets),
				Phase, {}, GetQueryTickGroupName(EQueryTickGroups::PostUpdate), EExecutionMode::GameThread /* Needs main thread*/);
		}
	}

	// Register the shared relations table. All relation entities use this base table
	// (or variants of it when they acquire additional columns like FIntervalEncodedHierarchyMetadata).
	{
		const UScriptStruct* RelationFragment = FMassRelationFragment::StaticStruct();
		Environment->GetTableManager().Register({&RelationFragment, 1}, TEXT("TEDSRelations"));
	}

	UE_LOGF(LogEditorDataStorage, Log, "Initialized TEDS Core.");
}

void UEditorDataStorage::PostInitialize()
{
}

void UEditorDataStorage::SetFactories(TConstArrayView<UClass*> FactoryClasses)
{
	Factories.Reserve(FactoryClasses.Num());

	UClass* BaseFactoryType = UEditorDataStorageFactory::StaticClass();

	for (UClass* FactoryClass : FactoryClasses)
	{
		if (FactoryClass->HasAnyClassFlags(CLASS_Abstract))
		{
			continue;
		}
		if (!FactoryClass->IsChildOf(BaseFactoryType))
		{
			continue;
		}
		UEditorDataStorageFactory* Factory = NewObject<UEditorDataStorageFactory>(this, FactoryClass, NAME_None, EObjectFlags::RF_Transient);
		Factories.Add(FFactoryTypePair
			{
				.Type = FactoryClass,
				.Instance = Factory
			});
	}

	Factories.StableSort(
	[](const FFactoryTypePair& Lhs, const FFactoryTypePair& Rhs)
	{
		return Lhs.Instance->GetOrder() < Rhs.Instance->GetOrder();
	});
	
	for (FFactoryTypePair& Factory : Factories)
	{
		Factory.Instance->PreRegister(*this);
	}
}

void UEditorDataStorage::ResetFactories()
{
	for (int32 Index = Factories.Num() - 1; Index >= 0; --Index)
	{
		const FFactoryTypePair& Factory = Factories[Index];
		Factory.Instance->PreShutdown(*this);
	}
	Factories.Empty();
}

UEditorDataStorage::FactoryIterator UEditorDataStorage::CreateFactoryIterator()
{
	return UEditorDataStorage::FactoryIterator(this);
}

UEditorDataStorage::FactoryConstIterator UEditorDataStorage::CreateFactoryIterator() const
{
	return UEditorDataStorage::FactoryConstIterator(this);
}

const UEditorDataStorageFactory* UEditorDataStorage::FindFactory(const UClass* FactoryType) const
{
	for (const FFactoryTypePair& Factory : Factories)
	{
		if (Factory.Type == FactoryType)
		{
			return Factory.Instance;
		}
	}
	return nullptr;
}

UEditorDataStorageFactory* UEditorDataStorage::FindFactory(const UClass* FactoryType)
{
	return const_cast<UEditorDataStorageFactory*>(static_cast<const UEditorDataStorage*>(this)->FindFactory(FactoryType));
}

void UEditorDataStorage::Deinitialize()
{
	UE_LOGF(LogEditorDataStorage, Log, "De-initializing TEDS Core.");

	checkf(Factories.IsEmpty(), TEXT("ResetFactories should have been called before deinitialized"));

	Reset();

	UE_LOGF(LogEditorDataStorage, Log, "De-initialized TEDS Core.");
}

void UEditorDataStorage::OnPreMassTick(float DeltaTime)
{
	checkf(IsAvailable(), TEXT("Typed Element Database was ticked while it's not ready."));
	
	ActiveCallsCount++;
	OnUpdateDelegate.Broadcast();
	// Process pending commands after other systems have had a chance to update. Other systems may have executed work needed
	// to complete pending work.
	Environment->GetDirectDeferredCommands().ProcessCommands();
}

void UEditorDataStorage::OnPostMassTick(float DeltaTime)
{
	checkf(IsAvailable(), TEXT("Typed Element Database was ticked while it's not ready."));
	
	Environment->NextUpdateCycle();
	ActiveCallsCount--;

	OnUpdateCompletedDelegate.Broadcast();

	TickCooperativeTasks();
}

TSharedPtr<FMassEntityManager> UEditorDataStorage::GetActiveMutableEditorEntityManager()
{
	return ActiveEditorEntityManager;
}

TSharedPtr<const FMassEntityManager> UEditorDataStorage::GetActiveEditorEntityManager() const
{
	return ActiveEditorEntityManager;
}



UE::Editor::DataStorage::RowHandle UEditorDataStorage::ReserveRow()
{
	return ActiveEditorEntityManager 
		? ActiveEditorEntityManager->ReserveEntity().AsNumber()
		: UE::Editor::DataStorage::InvalidRowHandle;
}

void UEditorDataStorage::BatchReserveRows(int32 Count, TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> ReservationCallback)
{
	if (ActiveEditorEntityManager)
	{
		TArrayView<FMassEntityHandle> ReservedEntities = Environment->GetScratchBuffer().AllocateZeroInitializedArray<FMassEntityHandle>(Count);
		ActiveEditorEntityManager->BatchReserveEntities(ReservedEntities);

		for (FMassEntityHandle ReservedEntity : ReservedEntities)
		{
			ReservationCallback(ReservedEntity.AsNumber());
		}
	}
}

void UEditorDataStorage::BatchReserveRows(TArrayView<UE::Editor::DataStorage::RowHandle> ReservedRows)
{
	using namespace UE::Editor::DataStorage;

	if (ActiveEditorEntityManager)
	{
		ActiveEditorEntityManager->BatchReserveEntities(RowsToMassEntitiesConversion(ReservedRows));
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::AddRow(UE::Editor::DataStorage::TableHandle Table)
{
	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(), 
		TEXT("Attempting to add a row to the editor data storage before it has been fully initialized."));
	checkf(IsTableValid(Table), TEXT("Attempting to add a row to a non-existing table."));
	FActiveCallScope Scope(*this);
	return ActiveEditorEntityManager->CreateEntity(Environment->GetTableManager().GetArchetype(Table)).AsNumber();
	

}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::AddRow(UE::Editor::DataStorage::TableHandle Table,
	UE::Editor::DataStorage::RowCreationCallbackRef OnCreated)
{
	using namespace UE::Editor::DataStorage;

	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(),
		TEXT("Attempting to add a row to the editor data storage before it has been fully initialized."));	
	checkf(IsTableValid(Table), TEXT("Attempting to a row to a non-existing table."));
	
	FActiveCallScope Scope(*this);

	TArray<FMassEntityHandle> Entity;
	Entity.Reserve(1);
	TSharedRef<FMassEntityManager::FEntityCreationContext> Context =
		ActiveEditorEntityManager->BatchCreateEntities(Environment->GetTableManager().GetArchetype(Table), 1, Entity);

	checkf(!Entity.IsEmpty(), TEXT("Add row tried to create a new row but none were provided by the backend."));
	RowHandle Result = Entity[0].AsNumber();
	OnCreated(Entity[0].AsNumber());
	return Result;
}

bool UEditorDataStorage::AddRow(UE::Editor::DataStorage::RowHandle ReservedRow, UE::Editor::DataStorage::TableHandle Table)
{
	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(),
		TEXT("Attempting to add a row to the editor data storage before it has been fully initialized."));
	checkf(IsTableValid(Table), TEXT("Attempting to add a row to a non-existing table."));
	checkf(!IsRowAssigned(ReservedRow), TEXT("Attempting to assign a table to row that already has a table assigned."));

	FActiveCallScope Scope(*this);
	ActiveEditorEntityManager->BuildEntity(
		FMassEntityHandle::FromNumber(ReservedRow), 
		Environment->GetTableManager().GetArchetype(Table));
		
	return true;
}

bool UEditorDataStorage::AddRow(UE::Editor::DataStorage::RowHandle ReservedRow, UE::Editor::DataStorage::TableHandle Table,
	UE::Editor::DataStorage::RowCreationCallbackRef OnCreated)
{
	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(),
		TEXT("Attempting to add a row to the editor data storage before it has been fully initialized."));	
	checkf(IsTableValid(Table), TEXT("Attempting to add a row to a non-existing table."));

	FActiveCallScope Scope(*this);
	TSharedRef<FMassEntityManager::FEntityCreationContext> Context =
		ActiveEditorEntityManager->BatchCreateReservedEntities(
			Environment->GetTableManager().GetArchetype(Table), 
			{ FMassEntityHandle::FromNumber(ReservedRow) });

	OnCreated(ReservedRow);
	return true;
}

bool UEditorDataStorage::BatchAddRow(
	UE::Editor::DataStorage::TableHandle Table, int32 Count, UE::Editor::DataStorage::RowCreationCallbackRef OnCreated)
{
	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(),
		TEXT("Attempting to batch add rows to the editor data storage before it has been fully initialized."));
	checkf(IsTableValid(Table), TEXT("Attempting to add rows to a non-existing table."));

	FActiveCallScope Scope(*this);

	TArray<FMassEntityHandle> Entities;
	Entities.Reserve(Count);
	TSharedRef<FMassEntityManager::FEntityCreationContext> Context = 
		ActiveEditorEntityManager->BatchCreateEntities(Environment->GetTableManager().GetArchetype(Table), Count, Entities);
		
	for (FMassEntityHandle Entity : Entities)
	{
		OnCreated(Entity.AsNumber());
	}

	return true;
}

bool UEditorDataStorage::BatchAddRow(UE::Editor::DataStorage::TableHandle Table,
	TConstArrayView<UE::Editor::DataStorage::RowHandle> ReservedHandles, UE::Editor::DataStorage::RowCreationCallbackRef OnCreated)
{
	using namespace UE::Editor::DataStorage;

	checkf(ActiveEditorEntityManager.IsValid() && Environment.IsValid(),
		TEXT("Attempting to batch add rows to the editor data storage before it has been fully initialized."));
	checkf(IsTableValid(Table), TEXT("Attempting to add rows to a non-existing table."));

	FActiveCallScope Scope(*this);

	TSharedRef<FMassEntityManager::FEntityCreationContext> Context =
		ActiveEditorEntityManager->BatchCreateReservedEntities(
			Environment->GetTableManager().GetArchetype(Table), 
			RowsToMassEntitiesConversion(ReservedHandles));

	for (RowHandle Entity : ReservedHandles)
	{
		OnCreated(Entity);
	}

	return true;
}


void UEditorDataStorage::RemoveRow(UE::Editor::DataStorage::RowHandle Row)
{
	FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
	if (ActiveEditorEntityManager && ActiveEditorEntityManager->IsEntityValid(Entity))
	{
		if (ActiveEditorEntityManager->IsEntityBuilt(FMassEntityHandle::FromNumber(Row)))
		{
			FActiveCallScope Scope(*this);
			ActiveEditorEntityManager->DestroyEntity(FMassEntityHandle::FromNumber(Row));
		}
		else
		{
			ActiveEditorEntityManager->ReleaseReservedEntity(FMassEntityHandle::FromNumber(Row));
		}
		Environment->GetMappingTable().MarkDirty();
	}
}

void UEditorDataStorage::BatchRemoveRows(TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows)
{
	using namespace UE::Editor::DataStorage;

	if (ActiveEditorEntityManager)
	{
		FActiveCallScope Scope(*this);
		ActiveEditorEntityManager->BatchDestroyEntities(RowsToMassEntitiesConversion(Rows));
		Environment->GetMappingTable().MarkDirty();
	}
}

void UEditorDataStorage::RemoveAllRowsWithColumns(TConstArrayView<const UScriptStruct*> Columns)
{
	if (ActiveEditorEntityManager)
	{
		FMassFragmentRequirements Requirements;
		Requirements.Initialize(ActiveEditorEntityManager.ToSharedRef());
		for (const UScriptStruct* Column : Columns)
		{
			if (Column)
			{
				if (Column->IsChildOf(FEditorDataStorageTag::StaticStruct()))
				{
					Requirements.AddTagRequirement(*Column, EMassFragmentPresence::All);
				}
				else
				{
					Requirements.AddRequirement(Column, EMassFragmentAccess::None, EMassFragmentPresence::All);
				}
			}
		}

		TArray<FMassArchetypeHandle> MatchingArchetypes;
		ActiveEditorEntityManager->GetMatchingArchetypes(Requirements, MatchingArchetypes);
		
		if (!MatchingArchetypes.IsEmpty())
		{
			FActiveCallScope Scope(*this);
			
			TArray<FMassArchetypeEntityCollection> Collections;
			Collections.Reserve(MatchingArchetypes.Num());
			for (FMassArchetypeHandle& Archetype : MatchingArchetypes)
			{
				FMassArchetypeEntityCollection Collection(Archetype);
				Collections.Add(MoveTemp(Collection));
			}

			ActiveEditorEntityManager->BatchDestroyEntityChunks(Collections);

			Environment->GetMappingTable().MarkDirty();
		}
	}
}

bool UEditorDataStorage::IsRowAvailable(UE::Editor::DataStorage::RowHandle Row) const
{
	return ActiveEditorEntityManager ? UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_IsRowAvailable(*ActiveEditorEntityManager, Row) : false;
}

bool UEditorDataStorage::IsRowAssigned(UE::Editor::DataStorage::RowHandle Row) const
{
	return ActiveEditorEntityManager ? UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_IsRowAssigned(*ActiveEditorEntityManager, Row) : false;
}

bool UEditorDataStorage::IsRowAvailableUnsafe(UE::Editor::DataStorage::RowHandle Row) const
{
	return UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_IsRowAvailable(*ActiveEditorEntityManager, Row);
}

bool UEditorDataStorage::IsRowAssignedUnsafe(UE::Editor::DataStorage::RowHandle Row) const
{
	return UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_IsRowAssigned(*ActiveEditorEntityManager, Row);
}

void UEditorDataStorage::FilterRowsBy(UE::Editor::DataStorage::FRowHandleArray& Result,  UE::Editor::DataStorage::FRowHandleArrayView Input, 
	EFilterOptions Options, UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>& Filter) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	if (ActiveEditorEntityManager && Environment && !Input.IsEmpty())
	{
		struct FQueryResult : TResult<bool>
		{
			FRowHandleArray& Result;
			bool bAcceptedResult;

			explicit FQueryResult(FRowHandleArray& Result, bool bAcceptedResult)
				: Result(Result)
				, bAcceptedResult(bAcceptedResult)
			{}

			virtual void Add(RowHandle Row, bool bResultValue) override
			{
				if (bResultValue == bAcceptedResult)
				{
					Result.Add(Row);
				}
			}
		};

		const bool bAcceptedResult = !EnumHasAnyFlags(Options, EFilterOptions::InvertFilter);
		constexpr EFunctionCallConfig CallingFlags = 
			EFunctionCallConfig::VerifyColumns | 
			EFunctionCallConfig::ContinueOnIncompleteRow |
			EFunctionCallConfig::VerifyCapabilityCompatibility |
			EFunctionCallConfig::ReportCapabilityCompatibility;

		FConstDirectContextEnvironment LocalContextEnvironment(*Environment, *this);
		QueryContext_ConstDirectApi Context(LocalContextEnvironment);
		for (RowHandle Row : Input)
		{
			LocalContextEnvironment.SetBatch(FRowHandleArrayView(&Row, 1, 
				FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique));
			FQueryResult QueryResult(Result, bAcceptedResult);
			if (Filter.Call<CallingFlags>(QueryResult, Context, LocalContextEnvironment) == EFlowControl::Break)
			{
				break;
			}
		}
	}
}

void UEditorDataStorage::AddColumn(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType)
{
	if (ColumnType && ActiveEditorEntityManager)
	{
		if (IsRowAssigned(Row))
		{
			FActiveCallScope Scope(*this);
			UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_AddColumnCommand(*ActiveEditorEntityManager, Row, ColumnType);
		}
		else
		{
			Environment->GetDirectDeferredCommands().Queue_AddColumnCommand(Row, ColumnType);
		}
	}
}

void UEditorDataStorage::AddColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType,
	const UE::Editor::DataStorage::ColumnCreationCallbackRef Initializer,
	UE::Editor::DataStorage::ColumnCopyOrMoveCallback Relocator)
{
	if (ActiveEditorEntityManager && UE::Mass::IsA<FMassFragment>(ColumnType))
	{
		if (IsRowAssigned(Row))
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			FStructView Column = ActiveEditorEntityManager->GetFragmentDataStruct(Entity, ColumnType);
			if (!Column.IsValid())
			{
				FActiveCallScope Scope(*this);
				ActiveEditorEntityManager->AddFragmentToEntity(Entity, ColumnType, Initializer);
			}
			else
			{
				Initializer(Column.GetMemory(), *ColumnType);
			}
		}
		else
		{
			void* Column = Environment->GetDirectDeferredCommands().Queue_AddDataColumnCommandUnitialized(Row, ColumnType, Relocator);
			Initializer(Column, *ColumnType);
		}
	}
}

void UEditorDataStorage::RemoveColumn(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType)
{
	if (ColumnType && ActiveEditorEntityManager)
	{
		if (IsRowAssigned(Row))
		{
			FActiveCallScope Scope(*this);
			UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_RemoveColumnCommand(*ActiveEditorEntityManager, Row, ColumnType);
		}
		else
		{
			Environment->GetDirectDeferredCommands().Queue_RemoveColumnCommand(Row, ColumnType);
		}
	}
}

const void* UEditorDataStorage::GetColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) const
{
	FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
	if (ActiveEditorEntityManager && UE::Mass::IsA<FMassFragment>(ColumnType))
	{
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FStructView Column = ActiveEditorEntityManager->GetFragmentDataStruct(Entity, ColumnType);
			if (Column.IsValid())
			{
				return Column.GetMemory();
			}
		}
		else if (ActiveEditorEntityManager->IsEntityValid(Entity) && ActiveEditorEntityManager->IsEntityReserved(Entity))
		{
			return Environment->GetDirectDeferredCommands().GetQueuedDataColumn(Row, ColumnType);
		}
	}
	return nullptr;
}

void* UEditorDataStorage::GetColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType)
{
	return const_cast<void*>(static_cast<const UEditorDataStorage*>(this)->GetColumnData(Row, ColumnType));
}

void UEditorDataStorage::AddColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> Columns)
{
	using namespace UE::Editor::DataStorage;
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);

		FMassFragmentBitSet FragmentsToAdd;
		FMassTagBitSet TagsToAdd;
		if (Private::ColumnsToBitSets(Columns, FragmentsToAdd, TagsToAdd).MustUpdate())
		{
			if (ActiveEditorEntityManager->IsEntityActive(Entity))
			{
				Legacy::FCommandBuffer::Execute_AddColumnsCommand(*ActiveEditorEntityManager, Row, FragmentsToAdd, TagsToAdd);
			}
			else
			{
				Environment->GetDirectDeferredCommands().Queue_AddColumnsCommand(Row, FragmentsToAdd, TagsToAdd);
			}
		}
	}
}

void UEditorDataStorage::AddColumn(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::FValueTag& Tag, const FName& InValue)
{
	if (ActiveEditorEntityManager)
	{
		const FConstSharedStruct SharedStruct = Environment->GenerateValueTag(Tag, InValue);

		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FActiveCallScope Scope(*this);
			UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_AddSharedColumnCommand(*ActiveEditorEntityManager, Row, SharedStruct);
		}
	}
}

void UEditorDataStorage::RemoveColumn(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::FValueTag& Tag)
{
	if (ActiveEditorEntityManager)
	{
		const UScriptStruct* ValueTagType = Environment->GenerateColumnType(Tag);
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FActiveCallScope Scope(*this);
			UE::Editor::DataStorage::Legacy::FCommandBuffer::Execute_RemoveSharedColumnCommand(*ActiveEditorEntityManager, Row, *ValueTagType);
		}
	}
}

void UEditorDataStorage::RemoveColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> Columns)
{
	using namespace UE::Editor::DataStorage;

	FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
	if (ActiveEditorEntityManager)
	{
		FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);

		FMassFragmentBitSet FragmentsToRemove;
		FMassTagBitSet TagsToRemove;
		if (Private::ColumnsToBitSets(Columns, FragmentsToRemove, TagsToRemove).MustUpdate())
		{
			if (ActiveEditorEntityManager->IsEntityActive(Entity))
			{
				FActiveCallScope Scope(*this);
				Legacy::FCommandBuffer::Execute_RemoveColumnsCommand(*ActiveEditorEntityManager, Row, FragmentsToRemove, TagsToRemove);
			}
			else
			{
				Environment->GetDirectDeferredCommands().Queue_RemoveColumnsCommand(Row, FragmentsToRemove, TagsToRemove);
			}
		}
	}
}

void UEditorDataStorage::AddRemoveColumns(UE::Editor::DataStorage::RowHandle Row,
	TConstArrayView<const UScriptStruct*> ColumnsToAdd, TConstArrayView<const UScriptStruct*> ColumnsToRemove)
{
	using namespace UE::Editor::DataStorage;
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);

		FMassFragmentBitSet FragmentsToAdd;
		FMassTagBitSet TagsToAdd;
		FMassTagBitSet TagsToRemove;
		FMassFragmentBitSet FragmentsToRemove;

		bool bMustAddColumns = Private::ColumnsToBitSets(ColumnsToAdd, FragmentsToAdd, TagsToAdd).MustUpdate();
		bool bMustRemoveColumns = Private::ColumnsToBitSets(ColumnsToRemove, FragmentsToRemove, TagsToRemove).MustUpdate();
		
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FActiveCallScope Scope(*this);
			if (bMustAddColumns)
			{
				Legacy::FCommandBuffer::Execute_AddColumnsCommand(*ActiveEditorEntityManager, Row, FragmentsToAdd, TagsToAdd);
			}
			if (bMustRemoveColumns)
			{
				Legacy::FCommandBuffer::Execute_RemoveColumnsCommand(*ActiveEditorEntityManager, Row, FragmentsToRemove, TagsToRemove);
			}
		}
		else
		{
			if (bMustAddColumns)
			{
				Environment->GetDirectDeferredCommands().Queue_AddColumnsCommand(Row, FragmentsToAdd, TagsToAdd);
			}
			if (bMustRemoveColumns)
			{
				Environment->GetDirectDeferredCommands().Queue_RemoveColumnsCommand(Row, FragmentsToRemove, TagsToRemove);
			}
		}
	}
}

void UEditorDataStorage::BatchAddRemoveColumns(TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows,
	TConstArrayView<const UScriptStruct*> ColumnsToAdd, TConstArrayView<const UScriptStruct*> ColumnsToRemove)
{	
	using namespace UE::Editor::DataStorage;
	
	using RowHandleArray = TArray<RowHandle, TInlineAllocator<32>>;
	using EntityHandleArray = TArray<FMassEntityHandle, TInlineAllocator<32>>;
	using EntityArchetypeLookup = TMap<FMassArchetypeHandle, EntityHandleArray, TInlineSetAllocator<32>>;
	using ArchetypeEntityArray = TArray<FMassArchetypeEntityCollection, TInlineAllocator<32>>;

	if (ActiveEditorEntityManager && !Rows.IsEmpty() && (!ColumnsToAdd.IsEmpty() || !ColumnsToRemove.IsEmpty()))
	{	
		FMassElementBitSet ElementsToAdd = Private::ColumnsToBitSets(ColumnsToAdd);
		FMassElementBitSet ElementsToRemove = Private::ColumnsToBitSets(ColumnsToRemove);
				
		// Sort rows (entities) into to matching table (archetype) bucket. If a row hasn't been constructed yet, but
		// only reserved, then queue the command for later execution.
		EntityArchetypeLookup LookupTable;
		RowHandleArray DelayedRows;
		for (RowHandle EntityId : Rows)
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(EntityId);
			if (ActiveEditorEntityManager->IsEntityActive(Entity))
			{
				FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);
				EntityHandleArray& EntityCollection = LookupTable.FindOrAdd(Archetype);
				EntityCollection.Add(Entity);
			}
			else
			{
				DelayedRows.Add(EntityId);
			}
		}
		
		if (!LookupTable.IsEmpty())
		{
			FActiveCallScope Scope(*this);

			// Construct table (archetype) specific row (entity) collections.
			ArchetypeEntityArray EntityCollections;
			EntityCollections.Reserve(LookupTable.Num());
			for (auto It = LookupTable.CreateConstIterator(); It; ++It)
			{
				EntityCollections.Emplace(It.Key(), It.Value(), FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates);
			}

			ActiveEditorEntityManager->BatchChangeCompositionForEntities(EntityCollections, ElementsToAdd, ElementsToRemove);
		}

		if (!DelayedRows.IsEmpty())
		{
			Legacy::FCommandBuffer& CommandBuffer = Environment->GetDirectDeferredCommands();
			CommandBuffer.Queue_AddRemoveColumnsCommand(DelayedRows, ElementsToAdd, ElementsToRemove);
		}
	}
}

bool UEditorDataStorage::HasColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) const
{
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			bool bHasAllColumns = true;
			FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);
			const FMassArchetypeCompositionDescriptor& Composition = ActiveEditorEntityManager->GetArchetypeComposition(Archetype);

			const UScriptStruct* const* ColumnTypesEnd = ColumnTypes.end();
			for (const UScriptStruct* const* ColumnType = ColumnTypes.begin(); ColumnType != ColumnTypesEnd && bHasAllColumns; ++ColumnType)
			{
				if (UE::Mass::IsA<FMassFragment>(*ColumnType) || UE::Mass::IsA<FMassTag>(*ColumnType))
				{
					bHasAllColumns = Composition.GetElementsBitSet().Contains(*ColumnType);
				}
				else
				{
					return false;
				}
			}
			return bHasAllColumns;
		}
		else if (ActiveEditorEntityManager->IsEntityValid(Entity) && ActiveEditorEntityManager->IsEntityReserved(Entity))
		{
			bool bHasAllColumns = true;
			const UE::Editor::DataStorage::Legacy::FCommandBuffer& CommandBuffer = Environment->GetDirectDeferredCommands();
			const UScriptStruct* const* ColumnTypesEnd = ColumnTypes.end();
			for (const UScriptStruct* const* ColumnType = ColumnTypes.begin(); ColumnType != ColumnTypesEnd && bHasAllColumns; ++ColumnType)
			{
				bHasAllColumns = CommandBuffer.HasColumn(Row, *ColumnType);
			}
			return bHasAllColumns;
		}
	}
	return false;
}

bool UEditorDataStorage::HasColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes) const
{
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			bool bHasAllColumns = true;
			FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);
			const FMassArchetypeCompositionDescriptor& Composition = ActiveEditorEntityManager->GetArchetypeComposition(Archetype);

			const TWeakObjectPtr<const UScriptStruct>* ColumnTypesEnd = ColumnTypes.end();
			for (const TWeakObjectPtr<const UScriptStruct>* WeakColumnType = ColumnTypes.begin(); WeakColumnType != ColumnTypesEnd && bHasAllColumns; ++WeakColumnType)
			{
				if (const UScriptStruct* ColumnType = WeakColumnType->Get())
				{
					if (UE::Mass::IsA<FMassFragment>(ColumnType) || UE::Mass::IsA<FMassTag>(ColumnType))
					{
						bHasAllColumns = Composition.GetElementsBitSet().Contains(ColumnType);
						continue;
					}
				}
				return false;
			}
			return bHasAllColumns;
		}
		else if (ActiveEditorEntityManager->IsEntityValid(Entity) && ActiveEditorEntityManager->IsEntityReserved(Entity))
		{
			bool bHasAllColumns = true;
			const UE::Editor::DataStorage::Legacy::FCommandBuffer& CommandBuffer = Environment->GetDirectDeferredCommands();
			const TWeakObjectPtr<const UScriptStruct>* ColumnTypesEnd = ColumnTypes.end();
			for (const TWeakObjectPtr<const UScriptStruct>* ColumnType = ColumnTypes.begin(); ColumnType != ColumnTypesEnd && bHasAllColumns; ++ColumnType)
			{
				// It's ok if ColumnType is null since it's not de-referenced, only used as a hash.
				bHasAllColumns = CommandBuffer.HasColumn(Row, ColumnType->Get());
			}
			return bHasAllColumns;
		}
		else
		{
			return false;
		}
	}
	return false;
}

void UEditorDataStorage::ListColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ColumnListCallbackRef Callback) const
{
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);
			const FMassArchetypeCompositionDescriptor& Composition = ActiveEditorEntityManager->GetArchetypeComposition(Archetype);
			
			auto CallbackWrapper = [&Callback](const UScriptStruct* ColumnType)
				{
					if (ColumnType)
					{
						Callback(*ColumnType);
					}
					return true;
				};
			Composition.GetElementsBitSet().ExportTypes(CallbackWrapper);
		}
		else if (ActiveEditorEntityManager->IsEntityValid(Entity) && ActiveEditorEntityManager->IsEntityReserved(Entity))
		{
			Environment->GetDirectDeferredCommands().ListColumns(Row, Callback);
		}
	}
}

void UEditorDataStorage::ListColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ColumnListWithDataCallbackRef Callback)
{
	if (ActiveEditorEntityManager)
	{
		FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
		if (ActiveEditorEntityManager->IsEntityActive(Entity))
		{
			FMassArchetypeHandle Archetype = ActiveEditorEntityManager->GetArchetypeForEntity(Entity);
			const FMassArchetypeCompositionDescriptor& Composition = ActiveEditorEntityManager->GetArchetypeComposition(Archetype);

			// @todo consider a unified path for FMassElementBitSet
			Composition.Get<FMassFragmentBitSet>().ExportTypes(
				[this, &Callback, Entity](const UScriptStruct* ColumnType)
				{
					if (ColumnType)
					{
						Callback(ActiveEditorEntityManager->GetFragmentDataStruct(Entity, ColumnType).GetMemory(), *ColumnType);
					}
					return true;
				});
			Composition.Get<FMassTagBitSet>().ExportTypes(
				[&Callback](const UScriptStruct* ColumnType)
				{
					if (ColumnType)
					{
						Callback(nullptr, *ColumnType);
					}
					return true;
				});

		}
		else if (ActiveEditorEntityManager->IsEntityValid(Entity) && ActiveEditorEntityManager->IsEntityReserved(Entity))
		{
			Environment->GetDirectDeferredCommands().ListColumns(Row, Callback);
		}
	}
}

bool UEditorDataStorage::MatchesColumns(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::Queries::FConditions& Conditions) const
{
	if (ActiveEditorEntityManager)
	{
		checkf(Conditions.IsCompiled(), TEXT("Query Conditions must be compiled before they can be used"));

		TArray<TWeakObjectPtr<const UScriptStruct>> Columns;
		
		ListColumns(Row, [&Columns](const UScriptStruct& InColumn)
		{
			Columns.Add(&InColumn);
		});
		
		return Conditions.Verify(Columns);
	}
	return false;
}

const UScriptStruct* UEditorDataStorage::GenerateDynamicColumn(const UE::Editor::DataStorage::FDynamicColumnDescription& Description)
{
	return Environment->GenerateDynamicColumn(*Description.TemplateType, Description.Identifier);
}

void UEditorDataStorage::ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const UScriptStruct& Type)> Callback) const
{
	return Environment->ForEachDynamicColumn(Template, Callback);
}

UE::Editor::DataStorage::FHierarchyHandle UEditorDataStorage::RegisterHierarchy(
	const UE::Editor::DataStorage::FHierarchyRegistrationParams& Params)
{
	return Environment->GetHierarchyRegistrar().RegisterHierarchy(this, Params);
}

UE::Editor::DataStorage::FHierarchyHandle UEditorDataStorage::FindHierarchyByName(const FName& InName) const
{
	return Environment->GetHierarchyRegistrar().FindHierarchyByName(InName);
}

bool UEditorDataStorage::IsValidHierarchyHandle(UE::Editor::DataStorage::FHierarchyHandle Handle) const
{
	return Environment->GetHierarchyRegistrar().GetAccessInterface(Handle) != nullptr;
}

const UScriptStruct* UEditorDataStorage::GetChildTagType(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->GetChildTagType();
	}
	return nullptr;
}

const UScriptStruct* UEditorDataStorage::GetParentTagType(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->GetParentTagType();
	}
	return nullptr;
}

const UScriptStruct* UEditorDataStorage::GetHierarchyDataColumnType(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->GetHierarchyDataColumnType();
	}
	return nullptr;
}

const UScriptStruct* UEditorDataStorage::GetParentChangedColumnType(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->GetParentChangedColumnType();
	}
	return nullptr;
}

void UEditorDataStorage::ResolveHierarchy(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle)
{
	Environment->GetHierarchyRegistrar().ResolveHierarchy(*this, InHierarchyHandle);
}

void UEditorDataStorage::ListHierarchyNames(TFunctionRef<void(const FName& HierarchyName)> Callback) const
{
	Environment->GetHierarchyRegistrar().ListHierarchyNames(Callback);
}

void UEditorDataStorage::SetParentRow(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, UE::Editor::DataStorage::RowHandle Target,
	UE::Editor::DataStorage::RowHandle Parent)
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		AccessInterface->SetParentRow(this, Target, Parent);
	}
}

void UEditorDataStorage::SetUnresolvedParent(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, UE::Editor::DataStorage::RowHandle Target,
	UE::Editor::DataStorage::FMapKey ParentId, FName MappingDomain)
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		AccessInterface->SetUnresolvedParent(this, Target, ParentId, MappingDomain);
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::GetParentRow(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle,
	UE::Editor::DataStorage::RowHandle Target) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->GetParentRow(this, Target);
	}
	return UE::Editor::DataStorage::InvalidRowHandle;
}

TFunction<UE::Editor::DataStorage::RowHandle(const void*, const UScriptStruct*)> UEditorDataStorage::CreateParentExtractionFunction(
	UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->CreateParentExtractionFunction();
	}
	return [](const void*, const UScriptStruct*) { return UE::Editor::DataStorage::InvalidRowHandle; };
}

bool UEditorDataStorage::HasChildren(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, UE::Editor::DataStorage::RowHandle Row) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->HasChildren(*this, Row);
	}
	return false;
}

void UEditorDataStorage::WalkDepthFirst(
	UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle,
	UE::Editor::DataStorage::RowHandle Row,
	FHierarchyIterationCallback VisitFn,
	ETraversalOrder TraversalOrder) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->WalkDepthFirst(*this, Row, MoveTemp(VisitFn), TraversalOrder);
	}
}

bool UEditorDataStorage::IterateChildren(
	UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle,
	UE::Editor::DataStorage::RowHandle Row,
	TFunctionRef<bool(const ICoreProvider& Context, UE::Editor::DataStorage::RowHandle Child)> VisitFn) const
{
	if (const UE::Editor::DataStorage::FTedsHierarchyAccessInterface* AccessInterface = Environment->GetHierarchyRegistrar().GetAccessInterface(InHierarchyHandle))
	{
		return AccessInterface->IterateChildren(*this, Row, MoveTemp(VisitFn));
	}
	return false;
}

//
// Relations Interface Implementation
//

UE::Editor::DataStorage::RelationTypeHandle UEditorDataStorage::RegisterRelationType(
	const UE::Editor::DataStorage::FRelationRegistrationParams& Params)
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager)
	{
		return InvalidRelationTypeHandle;
	}

	const RelationTypeHandle Handle = Environment->GetRelationAdapter().RegisterRelationType(
		*ActiveEditorEntityManager, Environment->GetTableManager(), Params);

	if (Handle == InvalidRelationTypeHandle)
	{
		return InvalidRelationTypeHandle;
	}

	if (Params.Traits.HierarchyMode != EHierarchyMode::Disabled)
	{
		Environment->GetHierarchicalRelationManager().RegisterHierarchyMode(Handle, Params.Traits.HierarchyMode, Params.Traits.IntervalGap);
	}

	// Generate optional change-notification columns and register FrameEnd cleanup processors.
	auto GenerateAndRegisterCleanup = [this, Handle](bool bEnable, const UScriptStruct* TemplateType, const FName& Identifier)
		-> const UScriptStruct*
	{
		if (!bEnable)
		{
			return nullptr;
		}
		using namespace UE::Editor::DataStorage::Queries;
		FDynamicColumnDescription Desc{ .TemplateType = TemplateType, .Identifier = Identifier };
		const UScriptStruct* Column = GenerateDynamicColumn(Desc);
		if (Column)
		{
			// Remove the change column from all matching rows at FrameEnd.
			RegisterQuery(
				Select(
					*FString::Printf(TEXT("Cleanup RelationChanged %s"), *Identifier.ToString()),
					FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
					[Column](IQueryContext& Context, const RowHandle* Rows)
					{
						Context.RemoveColumns(
							TConstArrayView<RowHandle>(Rows, Context.GetRowCount()),
							{Column});
					})
				.Where()
					.All(Column)
				.Compile());
		}
		return Column;
	};

	const UScriptStruct* SubjectChanged = GenerateAndRegisterCleanup(
		Params.bEnableSubjectChangedColumn,
		FTedsRelationSubjectChanged_Template::StaticStruct(),
		Params.Name);
	const UScriptStruct* ObjectChanged = GenerateAndRegisterCleanup(
		Params.bEnableObjectChangedColumn,
		FTedsRelationObjectChanged_Template::StaticStruct(),
		Params.Name);

	if (SubjectChanged || ObjectChanged)
	{
		Environment->GetRelationAdapter().SetChangedColumns(Handle, SubjectChanged, ObjectChanged);
	}

	return Handle;
}

UE::Editor::DataStorage::RelationTypeHandle UEditorDataStorage::FindRelationType(const FName& InName) const
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager)
	{
		return InvalidRelationTypeHandle;
	}

	return Environment->GetRelationAdapter().FindRelationType(InName);
}

bool UEditorDataStorage::IsValidRelationType(UE::Editor::DataStorage::RelationTypeHandle Type) const
{
	if (!ActiveEditorEntityManager)
	{
		return false;
	}

	return Environment->GetRelationAdapter().IsValidRelationType(Type);
}

const UE::Editor::DataStorage::FTedsRelationTraits* UEditorDataStorage::GetRelationTypeTraits(
	UE::Editor::DataStorage::RelationTypeHandle Type) const
{
	if (!ActiveEditorEntityManager)
	{
		return nullptr;
	}

	return Environment->GetRelationAdapter().GetTraits(Type);
}

void UEditorDataStorage::ListRelationTypes(
	TFunctionRef<void(UE::Editor::DataStorage::RelationTypeHandle, const FName&)> Callback) const
{
	if (ActiveEditorEntityManager)
	{
		Environment->GetRelationAdapter().ListTypes(Callback);
	}
}

const UScriptStruct* UEditorDataStorage::GetRelationSubjectChangedColumn(
	UE::Editor::DataStorage::RelationTypeHandle Type) const
{
	return Environment ? Environment->GetRelationAdapter().GetSubjectChangedColumn(Type) : nullptr;
}

const UScriptStruct* UEditorDataStorage::GetRelationObjectChangedColumn(
	UE::Editor::DataStorage::RelationTypeHandle Type) const
{
	return Environment ? Environment->GetRelationAdapter().GetObjectChangedColumn(Type) : nullptr;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::CreateRelation(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject,
	UE::Editor::DataStorage::RowHandle Object)
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager
		|| !Environment->GetRelationAdapter().IsValidRelationType(Type)
		|| !IsRowAssigned(Subject)
		|| !IsRowAssigned(Object))
	{
		return InvalidRowHandle;
	}

	if (Subject == Object)
	{
		ensureMsgf(false, TEXT("CreateRelation: self-relations (Subject == Object) are not supported."));
		return InvalidRowHandle;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);

	// Pre-check exclusivity (S3): reject if exclusive and relation already exists
	const FTedsRelationTraits* Traits = Environment->GetRelationAdapter().GetTraits(Type);
	if (Traits)
	{
		if (Traits->Subject.bExclusive)
		{
			// bExclusive on Subject means this Subject may have at most one Object across all relations
			// of this type. Reject if the Subject already has ANY Object, not just the specific pair.
			if (HasRelationObject(Type, Subject))
			{
				return InvalidRowHandle;
			}
		}
	}

	// Ensure Mass's RelationsDataMap is populated for this type before creating instances.
	// FTypeManager::RegisterType only calls OnNewTypeRegistered (which populates RelationsDataMap)
	// after bBuiltInTypesRegistered is set. If TEDS registered the type before Mass finished
	// built-in processing, the data won't exist yet. We trigger it on first use.
	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		ActiveEditorEntityManager->OnNewTypeRegistered(MassHandle);
		Environment->GetRelationAdapter().SetHasMassRelationData(Type);
	}

	// Delegate to Mass
	const FMassEntityHandle MassRelationEntity = ActiveEditorEntityManager->GetRelationManager()
		.CreateRelationInstance(MassHandle, FMassEntityHandle::FromNumber(Subject), FMassEntityHandle::FromNumber(Object));

	const RowHandle RelationRow = MassRelationEntity.AsNumber();

	// If this is a hierarchical relation, add the appropriate metadata column and initialize it.
	// Note: AddColumn causes a per-entity archetype migration. This is acceptable because:
	// - Relation entities are internal (not user-visible participant rows)
	// - Hierarchy changes are not a hot-path operation
	// Future optimization (S2): Mass's FRelationManager::CreateRelationInstances has a hardcoded
	// archetype (FMassRelationFragment + tag). To avoid per-creation migration, Mass would need
	// an API extension to accept additional fragments at creation time. That change belongs in a
	// separate CL to the MassEntity module.
	if (RelationRow != InvalidRowHandle && Traits && Traits->HierarchyMode != EHierarchyMode::Disabled )
	{
		const UScriptStruct* MetadataStruct = (Traits->HierarchyMode == EHierarchyMode::IntervalEncoded)
			? FIntervalEncodedHierarchyMetadata::StaticStruct()
			: FWalkOnlyHierarchyMetadata::StaticStruct();
		AddColumn(RelationRow, MetadataStruct);
		Environment->GetHierarchicalRelationManager().InitializeHierarchyMetadata(*this, Type, RelationRow, Subject, Object);
	}

	// Stamp change-notification columns on participant rows if enabled.
	if (RelationRow != InvalidRowHandle)
	{
		if (const UScriptStruct* SubjectChanged = Environment->GetRelationAdapter().GetSubjectChangedColumn(Type))
		{
			AddColumn(Subject, SubjectChanged);
		}
		if (const UScriptStruct* ObjectChanged = Environment->GetRelationAdapter().GetObjectChangedColumn(Type))
		{
			AddColumn(Object, ObjectChanged);
		}
	}

	return RelationRow;
}

void UEditorDataStorage::BatchCreateRelations(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	TConstArrayView<UE::Editor::DataStorage::RowHandle> Subjects,
	TConstArrayView<UE::Editor::DataStorage::RowHandle> Objects,
	TArray<UE::Editor::DataStorage::RowHandle>* OutRelationRows)
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager || Subjects.IsEmpty() || Objects.IsEmpty()
		|| !Environment->GetRelationAdapter().IsValidRelationType(Type))
	{
		if (OutRelationRows)
		{
			OutRelationRows->Reset();
		}
		return;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);

	// Convert RowHandle arrays to FMassEntityHandle arrays
	TArray<FMassEntityHandle> MassSubjects;
	TArray<FMassEntityHandle> MassObjects;
	MassSubjects.Reserve(Subjects.Num());
	MassObjects.Reserve(Objects.Num());

	for (RowHandle Row : Subjects)
	{
		MassSubjects.Add(FMassEntityHandle::FromNumber(Row));
	}
	for (RowHandle Row : Objects)
	{
		MassObjects.Add(FMassEntityHandle::FromNumber(Row));
	}

	// Ensure Mass's RelationsDataMap is populated (see CreateRelation for explanation)
	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		ActiveEditorEntityManager->OnNewTypeRegistered(MassHandle);
		Environment->GetRelationAdapter().SetHasMassRelationData(Type);
	}

	// Call FRelationManager::CreateRelationInstances directly (C3) to get handles back
	TArray<FMassEntityHandle> MassRelationEntities =
		ActiveEditorEntityManager->GetRelationManager().CreateRelationInstances(
			MassHandle, MassSubjects, MassObjects);

	// Initialize hierarchy metadata if needed
	const FTedsRelationTraits* Traits = Environment->GetRelationAdapter().GetTraits(Type);
	const bool bIsHierarchical = Traits && Traits->HierarchyMode != EHierarchyMode::Disabled;
	const UScriptStruct* MetadataStruct = nullptr;
	if (bIsHierarchical)
	{
		MetadataStruct = (Traits->HierarchyMode == EHierarchyMode::IntervalEncoded)
			? FIntervalEncodedHierarchyMetadata::StaticStruct()
			: FWalkOnlyHierarchyMetadata::StaticStruct();
	}

	if (OutRelationRows)
	{
		OutRelationRows->Reset(MassRelationEntities.Num());
	}

	for (int32 Index = 0; Index < MassRelationEntities.Num(); ++Index)
	{
		const RowHandle RelationRow = MassRelationEntities[Index].AsNumber();
		if (OutRelationRows)
		{
			OutRelationRows->Add(RelationRow);
		}

		if (bIsHierarchical && RelationRow != InvalidRowHandle)
		{
			AddColumn(RelationRow, MetadataStruct);
			// Match Mass's clamping behavior: FMath::Min(Array.Num()-1, PairIndex)
			const RowHandle SubjectRow = Subjects[FMath::Min(Subjects.Num() - 1, Index)];
			const RowHandle ObjectRow = Objects[FMath::Min(Objects.Num() - 1, Index)];
			Environment->GetHierarchicalRelationManager().InitializeHierarchyMetadata(*this, Type, RelationRow, SubjectRow, ObjectRow);
		}
	}
}

bool UEditorDataStorage::DestroyRelation(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject,
	UE::Editor::DataStorage::RowHandle Object)
{
	if (!ActiveEditorEntityManager || !Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return false;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);

	// Clean up hierarchy metadata mapping before destroying the relation
	if (Environment->GetRelationAdapter().IsHierarchical(Type))
	{
		Environment->GetHierarchicalRelationManager().RemoveHierarchyMetadata(Type, Subject);
	}

	const bool bDestroyed = ActiveEditorEntityManager->GetRelationManager().DestroyRelationInstance(
		MassHandle, FMassEntityHandle::FromNumber(Subject), FMassEntityHandle::FromNumber(Object));

	// Stamp change-notification columns on participant rows if the relation was actually destroyed.
	if (bDestroyed)
	{
		if (const UScriptStruct* SubjectChanged = Environment->GetRelationAdapter().GetSubjectChangedColumn(Type))
		{
			if (IsRowAssigned(Subject))
			{
				AddColumn(Subject, SubjectChanged);
			}
		}
		if (const UScriptStruct* ObjectChanged = Environment->GetRelationAdapter().GetObjectChangedColumn(Type))
		{
			if (IsRowAssigned(Object))
			{
				AddColumn(Object, ObjectChanged);
			}
		}
	}

	return bDestroyed;
}

bool UEditorDataStorage::HasRelation(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject,
	UE::Editor::DataStorage::RowHandle Object) const
{
	if (!ActiveEditorEntityManager)
	{
		return false;
	}

	// Guard: Mass populates RelationData lazily on first CreateRelationInstance.
	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return false;
	}
	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	return ActiveEditorEntityManager->GetRelationManager().IsSubjectOfRelation(
		MassHandle, FMassEntityHandle::FromNumber(Subject), FMassEntityHandle::FromNumber(Object));
}

bool UEditorDataStorage::HasRelationObject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject) const
{
	using namespace UE::Editor::DataStorage;
	if (!ActiveEditorEntityManager || !Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return false;
	}
	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	return ActiveEditorEntityManager->GetRelationManager().HasRelationObjects(
		MassHandle, FMassEntityHandle::FromNumber(Subject));
}

bool UEditorDataStorage::HasRelationSubject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Object) const
{
	using namespace UE::Editor::DataStorage;
	if (!ActiveEditorEntityManager || !Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return false;
	}
	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	return ActiveEditorEntityManager->GetRelationManager().HasRelationSubjects(
		MassHandle, FMassEntityHandle::FromNumber(Object));
}

void UEditorDataStorage::GetRelationObjects(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject,
	TArray<UE::Editor::DataStorage::RowHandle>& OutObjects) const
{
	using namespace UE::Editor::DataStorage;
	OutObjects.Reset();

	if (!ActiveEditorEntityManager)
	{
		return;
	}

	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return;
	}
	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	TArray<FMassEntityHandle> MassObjects = ActiveEditorEntityManager->GetRelationManager().GetRelationObjects(
		MassHandle, FMassEntityHandle::FromNumber(Subject));

	OutObjects.Reserve(MassObjects.Num());
	for (const FMassEntityHandle& Handle : MassObjects)
	{
		OutObjects.Add(Handle.AsNumber());
	}
}

void UEditorDataStorage::GetRelationSubjects(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Object,
	TArray<UE::Editor::DataStorage::RowHandle>& OutSubjects) const
{
	using namespace UE::Editor::DataStorage;
	OutSubjects.Reset();

	if (!ActiveEditorEntityManager)
	{
		return;
	}

	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return;
	}
	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	TArray<FMassEntityHandle> MassSubjects = ActiveEditorEntityManager->GetRelationManager().GetRelationSubjects(
		MassHandle, FMassEntityHandle::FromNumber(Object));

	OutSubjects.Reserve(MassSubjects.Num());
	for (const FMassEntityHandle& Handle : MassSubjects)
	{
		OutSubjects.Add(Handle.AsNumber());
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::GetRelationObject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject) const
{
	using namespace UE::Editor::DataStorage;

	TArray<RowHandle> Objects;
	GetRelationObjects(Type, Subject, Objects);

	// For relations with exclusive Object, there's at most one object
	return Objects.Num() > 0 ? Objects[0] : InvalidRowHandle;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::GetRelationSubject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Object) const
{
	using namespace UE::Editor::DataStorage;

	TArray<RowHandle> Subjects;
	GetRelationSubjects(Type, Object, Subjects);

	// For relations with exclusive Subject, there's at most one subject
	return Subjects.Num() > 0 ? Subjects[0] : InvalidRowHandle;
}

bool UEditorDataStorage::IsDescendantOf(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Descendant,
	UE::Editor::DataStorage::RowHandle Ancestor,
	bool bIncludeSelf) const
{
	if (!ActiveEditorEntityManager)
	{
		return false;
	}

	return Environment->GetHierarchicalRelationManager().IsDescendantOf(*this, Type, Descendant, Ancestor, bIncludeSelf);
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::GetHierarchyRoot(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Row) const
{
	if (!ActiveEditorEntityManager)
	{
		return UE::Editor::DataStorage::InvalidRowHandle;
	}

	return Environment->GetHierarchicalRelationManager().GetHierarchyRoot(*this, Type, Row);
}

int32 UEditorDataStorage::GetHierarchyDepth(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Row) const
{
	if (!ActiveEditorEntityManager)
	{
		return 0;
	}

	return Environment->GetHierarchicalRelationManager().GetHierarchyDepth(*this, Type, Row);
}

void UEditorDataStorage::GetDescendants(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Ancestor,
	TArray<UE::Editor::DataStorage::RowHandle>& OutDescendants) const
{
	if (!ActiveEditorEntityManager)
	{
		OutDescendants.Reset();
		return;
	}

	Environment->GetHierarchicalRelationManager().GetDescendants(*this, Type, Ancestor, OutDescendants);
}

void UEditorDataStorage::GetAncestors(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Descendant,
	TArray<UE::Editor::DataStorage::RowHandle>& OutAncestors) const
{
	if (!ActiveEditorEntityManager)
	{
		OutAncestors.Reset();
		return;
	}

	Environment->GetHierarchicalRelationManager().GetAncestors(*this, Type, Descendant, OutAncestors);
}

void UEditorDataStorage::TraverseDescendants(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle StartRow,
	FRelationTraversalCallback Callback,
	ETraversalOrder Order,
	int32 MaxDepth) const
{
	if (!ActiveEditorEntityManager)
	{
		return;
	}

	Environment->GetHierarchicalRelationManager().TraverseDescendants(*this, Type, StartRow, Callback, Order, MaxDepth);
}

void UEditorDataStorage::ForEachRelationObject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Subject,
	TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> Callback) const
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager)
	{
		return;
	}

	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	ActiveEditorEntityManager->GetRelationManager().ForEachRelationObject(
		MassHandle, FMassEntityHandle::FromNumber(Subject),
		[&Callback](FMassEntityHandle MassHandle)
		{
			Callback(MassHandle.AsNumber());
		});
}

void UEditorDataStorage::ForEachRelationSubject(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Object,
	TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> Callback) const
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager)
	{
		return;
	}

	if (!Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	ActiveEditorEntityManager->GetRelationManager().ForEachRelationSubject(
		MassHandle, FMassEntityHandle::FromNumber(Object),
		[&Callback](FMassEntityHandle MassHandle)
		{
			Callback(MassHandle.AsNumber());
		});
}

void UEditorDataStorage::TraverseAncestors(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Descendant,
	TFunctionRef<bool(UE::Editor::DataStorage::RowHandle, int32)> Callback) const
{
	using namespace UE::Editor::DataStorage;

	constexpr int32 MaxAncestorWalkDepth = 65536;
	RowHandle Current = GetRelationObject(Type, Descendant);
	int32 Depth = 1;
	while (Current != InvalidRowHandle && Depth < MaxAncestorWalkDepth)
	{
		if (!Callback(Current, Depth))
		{
			break;
		}
		Current = GetRelationObject(Type, Current);
		++Depth;
	}
}

int32 UEditorDataStorage::ComputeDescendantCount(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Ancestor) const
{
	int32 Count = 0;
	TraverseDescendants(Type, Ancestor,
		[&Count](UE::Editor::DataStorage::RowHandle, UE::Editor::DataStorage::RowHandle, int32)
		{
			++Count;
		});
	return Count;
}

bool UEditorDataStorage::HasRelationSubjects(
	UE::Editor::DataStorage::RelationTypeHandle Type,
	UE::Editor::DataStorage::RowHandle Object) const
{
	using namespace UE::Editor::DataStorage;

	if (!ActiveEditorEntityManager || !Environment->GetRelationAdapter().HasMassRelationData(Type))
	{
		return false;
	}

	const UE::Mass::FTypeHandle MassHandle = Environment->GetRelationAdapter().GetMassHandle(Type);
	return ActiveEditorEntityManager->GetRelationManager().HasRelationSubjects(
		MassHandle, FMassEntityHandle::FromNumber(Object));
}

void UEditorDataStorage::RegisterTickGroup(
	FName GroupName, UE::Editor::DataStorage::EQueryTickPhase Phase, FName BeforeGroup, FName AfterGroup, UE::Editor::DataStorage::EExecutionMode ExecutionMode)
{
	Environment->GetQueryStore().RegisterTickGroup(GroupName, Phase, BeforeGroup, AfterGroup, ExecutionMode);
}

void UEditorDataStorage::UnregisterTickGroup(FName GroupName, UE::Editor::DataStorage::EQueryTickPhase Phase)
{
	Environment->GetQueryStore().UnregisterTickGroup(GroupName, Phase);
}

UE::Editor::DataStorage::QueryHandle UEditorDataStorage::RegisterQuery(UE::Editor::DataStorage::FQueryDescription&& Query)
{
	return (ActiveEditorEntityManager && ActiveEditorPhaseManager)
		? Environment->GetQueryStore().RegisterQuery(MoveTemp(Query), *Environment, *ActiveEditorEntityManager, *ActiveEditorPhaseManager).Packed()
		: UE::Editor::DataStorage::InvalidQueryHandle;
}

void UEditorDataStorage::UnregisterQuery(UE::Editor::DataStorage::QueryHandle Query)
{
	if (ActiveEditorEntityManager && ActiveEditorPhaseManager)
	{
		const UE::Editor::DataStorage::FExtendedQueryStore::Handle StorageHandle(Query);
		Environment->GetQueryStore().UnregisterQuery(StorageHandle, *ActiveEditorEntityManager, *ActiveEditorPhaseManager);
	}
}

const UE::Editor::DataStorage::FQueryDescription& UEditorDataStorage::GetQueryDescription(UE::Editor::DataStorage::QueryHandle Query) const
{
	const UE::Editor::DataStorage::FExtendedQueryStore::Handle StorageHandle(Query);
	return Environment->GetQueryStore().GetQueryDescription(StorageHandle);
}

uint64 UEditorDataStorage::CalculateQueryTablesTopologyHash(UE::Editor::DataStorage::QueryHandle Query) const
{
	const UE::Editor::DataStorage::FExtendedQueryStore::Handle StorageHandle(Query);
	return Environment->GetQueryStore().CalculateQueryTablesTopologyHash(StorageHandle);
}

FName UEditorDataStorage::GetQueryTickGroupName(UE::Editor::DataStorage::EQueryTickGroups Group) const
{
	using namespace UE::Editor::DataStorage;

	switch (Group)
	{
		case EQueryTickGroups::Default:
			return TickGroupName_Default;
		case EQueryTickGroups::PreUpdate:
			return TickGroupName_PreUpdate;
		case EQueryTickGroups::Update:
			return TickGroupName_Update;
		case EQueryTickGroups::PostUpdate:
			return TickGroupName_PostUpdate;
		case EQueryTickGroups::SyncExternalToDataStorage:
			return TickGroupName_SyncExternalToDataStorage;
		case EQueryTickGroups::SyncDataStorageToExternal:
			return TickGroupName_SyncDataStorageToExternal;
		case EQueryTickGroups::SyncWidgets:
			return TickGroupName_SyncWidget;
		default:
			checkf(false, TEXT("EQueryTickGroups value %i can't be translated to a group name by this Data Storage backend."), static_cast<int>(Group));
			return NAME_None;
	}
}

UE::Editor::DataStorage::FQueryResult UEditorDataStorage::RunQuery(UE::Editor::DataStorage::QueryHandle Query)
{
	using namespace UE::Editor::DataStorage;
	TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.RunQuery);

	if (ActiveEditorEntityManager)
	{
		// There's no need to scope this because this version of the query isn't running anything that could change tables.
		const FExtendedQueryStore::Handle StorageHandle(Query);
		return Environment->GetQueryStore().RunQuery(*ActiveEditorEntityManager, StorageHandle);
	}
	else
	{
		return FQueryResult();
	}
}

UE::Editor::DataStorage::FQueryResult UEditorDataStorage::RunQuery(
	UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::DirectQueryCallbackRef Callback)
{
	using namespace UE::Editor::DataStorage;

	TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.RunQuery);

	if (ActiveEditorEntityManager)
	{
		FActiveCallScope Scope(*this);
		const FExtendedQueryStore::Handle StorageHandle(Query);
		return Environment->GetQueryStore().RunQuery(*ActiveEditorEntityManager, *Environment, StorageHandle, 
			EDirectQueryExecutionFlags::Default, Callback);
	}
	else
	{
		return FQueryResult();
	}
}

UE::Editor::DataStorage::FQueryResult UEditorDataStorage::RunQuery(
	UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::EDirectQueryExecutionFlags InFlags,
	UE::Editor::DataStorage::DirectQueryCallbackRef Callback)
{
	using namespace UE::Editor::DataStorage;

	TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.RunQuery);

	if (ActiveEditorEntityManager)
	{
		FActiveCallScope Scope(*this);
		const FExtendedQueryStore::Handle StorageHandle(Query);
		return Environment->GetQueryStore().RunQuery(*ActiveEditorEntityManager, *Environment, StorageHandle, InFlags, Callback);
	}
	else
	{
		return FQueryResult();
	}
}

UE::Editor::DataStorage::FQueryResult UEditorDataStorage::RunQuery(
	UE::Editor::DataStorage::QueryHandle Query,
	UE::Editor::DataStorage::ERunQueryFlags InFlags,
	const UE::Editor::DataStorage::Queries::TQueryFunction<void>& Callback)
{
	using namespace UE::Editor::DataStorage;

	TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.RunQuery);

	if (ActiveEditorEntityManager)
	{
		FActiveCallScope Scope(*this); 
		return Environment->GetQueryStore().RunQuery(*ActiveEditorEntityManager, *Environment,
			FExtendedQueryStore::Handle(Query), InFlags, Callback);
	}
	else
	{
		return FQueryResult();
	}
}

UE::Editor::DataStorage::FQueryResult UEditorDataStorage::RunQuery(
	UE::Editor::DataStorage::QueryHandle Query,
	UE::Editor::DataStorage::ERunQueryFlags InFlags,
	const UE::Editor::DataStorage::Queries::TConstQueryFunction<void>& Callback) const
{
	using namespace UE::Editor::DataStorage;

	TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.RunQuery);

	if (ActiveEditorEntityManager)
	{
		FActiveCallScope Scope(*this);
		return Environment->GetQueryStore().RunQuery(*ActiveEditorEntityManager, *Environment,
			FExtendedQueryStore::Handle(Query), InFlags, Callback);
	}
	else
	{
		return FQueryResult();
	}
}

void UEditorDataStorage::ActivateQueries(FName ActivationName)
{
	if (ActiveEditorEntityManager)
	{
		Environment->GetQueryStore().ActivateQueries(ActivationName);
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorage::LookupMappedRow(
	const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& Key) const
{
	return Environment->GetMappingTable().Lookup(UE::Editor::DataStorage::EGlobalLockScope::Public, Domain, Key);
}

void UEditorDataStorage::MapRow(const FName& Domain, UE::Editor::DataStorage::FMapKey Key, UE::Editor::DataStorage::RowHandle Row)
{
	Environment->GetMappingTable().Map(UE::Editor::DataStorage::EGlobalLockScope::Public, Domain, MoveTemp(Key), Row);
}

void UEditorDataStorage::BatchMapRows(
	const FName& Domain, TArrayView<TPair<UE::Editor::DataStorage::FMapKey, UE::Editor::DataStorage::RowHandle>> MapRowPairs)
{
	Environment->GetMappingTable().BatchMap(UE::Editor::DataStorage::EGlobalLockScope::Public, Domain, MapRowPairs);
}

void UEditorDataStorage::RemapRow(
	const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& OriginalKey, UE::Editor::DataStorage::FMapKey NewKey)
{
	Environment->GetMappingTable().Remap(UE::Editor::DataStorage::EGlobalLockScope::Public, Domain, OriginalKey, MoveTemp(NewKey));
}

void UEditorDataStorage::RemoveRowMapping(
	const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& Key)
{
	Environment->GetMappingTable().Remove(UE::Editor::DataStorage::EGlobalLockScope::Public, Domain, Key);
}

UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate& UEditorDataStorage::OnUpdate()
{
	return OnUpdateDelegate;
}

UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate& UEditorDataStorage::OnUpdateCompleted()
{
	return OnUpdateCompletedDelegate;
}

void UEditorDataStorage::RegisterCooperativeUpdate(const FName& TaskName, ECooperativeTaskPriority Priority, FOnCooperativeUpdate Callback)
{
	checkf(!bRunningCooperativeUpdate, TEXT("Cooperative tasks can't be added to TEDS during the cooperative task update."));

	uint8 OrderValue;
	// This distribution means that if there's one high, one medium and one low task the high task would be called twice as often as the
	// medium task and four times as often the low priority task. These numbers are skewed if there's a different number of task per
	// priority, but no matter how many tasks there are, on average high will be more frequent than medium and medium more frequently
	// called than low.
	// As an example, if there's 2 high, 5 medium and 3 low tasks than a 1000 iterations will be distributed as:
	//		2 High:   total 247 calls, avg. 123.5 calls
	//		5 Medium: total 529 calls, avg. 105.8 calls
	//		3 Low :	  total 224 calls, avg.  74.7 calls
	// So while there are 529 medium tasks executed compared to only 247 high tasks, because there are 5 medium and only 2 high task, the
	// high task occur with higher frequency than medium tasks.
	switch (Priority)
	{
	case ECooperativeTaskPriority::High:
		// Don't go below 1 because that practically means it's always set to 0. If Medium is set to 2 it means they'll effectively have 
		// the same priority because they'll both constantly be ping-pong-ing around 0 and 1.
		OrderValue = 2;
		break;
	case ECooperativeTaskPriority::Medium:
		OrderValue = 4;
		break;
	case ECooperativeTaskPriority::Low:
		[[fallthrough]];
	default:
		OrderValue = 8;
		break;
	}

	// Just add to the back of the queue as the task will eventually bubble up and 
	// land in its correct position.
	CooperativeTasks.Add(FCooperativeTask
		{
			.Callback = MoveTemp(Callback),
			.Name = TaskName,
			.OrderResetValue = OrderValue,
			.Order = OrderValue
		});
}

void UEditorDataStorage::UnregisterCooperativeUpdate(const FName& TaskName)
{
	checkf(!bRunningCooperativeUpdate, TEXT("Cooperative tasks can't be removed from TEDS during the cooperative task update."));

	if (uint32 Index = CooperativeTasks.IndexOfByPredicate([&TaskName](const FCooperativeTask& Task) { return Task.Name == TaskName; });
		Index != INDEX_NONE)
	{
		// Maintain order as the order needed to make sure tasks are ordered in the correct order.
		CooperativeTasks.RemoveAt(Index);
	}
}

void UEditorDataStorage::TickCooperativeTasks()
{
	// The minimum amount of time of time that will be spend each tick on cooperative tasks. A cap is set on the minimum
	// time spend processing tasks each frame to avoid situations where the editor goes over frame budget for prolonged
	// periods, causing the tasks to not be run. This can be detrimental if any of the tasks are needed to speed up
	// the editor like garbage collection tasks.
	const FTimespan MinTimePerTick = FTimespan::FromMilliseconds(CooperativeTasks_MinTimePerFrameMs.GetValueOnGameThread());
	// Minimum amount of time that needs to be left in order for a task to run. This is to prevent there being a few
	// nanoseconds left that are used by a task that takes several milliseconds. Without continuous analysis this
	// value is a guess
	const FTimespan MinTimePerTask = FTimespan::FromMilliseconds(CooperativeTasks_MinTimePerTaskMs.GetValueOnGameThread());
	// The expected duration per frame.
	int64 TargetFrameRate = FMath::Clamp(CooperativeTasks_TargetFrameRate.GetValueOnGameThread(), 16, 120);
	const FTimespan TargetFrameDuration = FTimespan::FromMilliseconds(1000.0 / static_cast<double>(TargetFrameRate));
	// The amount of time a task has to go over the allotted time before it's reported.
	const FTimespan OverBudgetReportThreshold = FTimespan::FromMilliseconds(CooperativeTasks_ReportThresholdMs.GetValueOnGameThread());

	if (!CooperativeTasks.IsEmpty())
	{
		TEDS_EVENT_SCOPE("Tick time sliced tasks");

		bRunningCooperativeUpdate = true;

		// Don't count the time from last frame spend in cooperative tasks because that's part of the time that needs
		// to be distributed again.
		FTimespan LastFrameDuration(
			FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - CooperativeTickStartTime)) + 
			LastFrameCooperativeTickDuration);

		FTimespan RemainingTime;
		if (LastFrameDuration < (TargetFrameDuration - MinTimePerTick))
		{
			// There's time left to do cooperative tasks
			RemainingTime = TargetFrameDuration - LastFrameDuration;
			CooperativeTaskIdleFrameCount = 0;
		}
		else
		{
			// There's no time left to do work;
			if (CooperativeTaskIdleFrameCount < CooperativeTasks_MaxIdleFrames.GetValueOnGameThread())
			{
#if COUNTERSTRACE_ENABLED
				SET_FLOAT_STAT(STAT_TedsCore_CoopTaskTimeMs, 0.0);
				TRACE_COUNTER_SET(COUNTER_TedsCore_CoopTaskTimeMs, 0.0);
#endif
				LastFrameCooperativeTickDuration = 0;
				bRunningCooperativeUpdate = false;
				CooperativeTickStartTime = FPlatformTime::Cycles64();
				CooperativeTaskIdleFrameCount++;
				return;
			}
			else
			{
				RemainingTime = MinTimePerTick;
			}
		}

#if COUNTERSTRACE_ENABLED
		double RemainingTimeMs = RemainingTime.GetTotalMilliseconds();
		SET_FLOAT_STAT(STAT_TedsCore_CoopTaskTimeMs, RemainingTimeMs);
		TRACE_COUNTER_SET(COUNTER_TedsCore_CoopTaskTimeMs, RemainingTimeMs);
#endif
		
		// Update the priorities. This causes all tasks to eventually float to the top so even low priority tasks
		// get to run on occasion.
		for (FCooperativeTask& Task : CooperativeTasks)
		{
			Task.Order = Task.Order > 0 ? Task.Order - 1 : Task.Order;
			Task.bHasRun = false;
		}

		int32 CurrentTaskIndex = 0;
		
		uint64 ProcessingStartTime = FPlatformTime::Cycles64();
		while (CurrentTaskIndex < CooperativeTasks.Num())
		{
			FCooperativeTask& CurrentTask = CooperativeTasks[CurrentTaskIndex];
			// Make sure that tasks aren't run multiple times per frame and eat up cycles from lower priority tasks
			if (!CurrentTask.bHasRun)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(
					*FString::Printf(TEXT("[TEDS] Tick time sliced tasks: '%s'"), *CurrentTask.Name.ToString()));

				// Run the task.
				{
					// Measure only the task specifically going over time, not the cost of any bookkeeping.
					uint64 TaskStartTime = FPlatformTime::Cycles64();

					CurrentTask.Callback(RemainingTime);
					CurrentTask.Order = CurrentTask.OrderResetValue;
					CurrentTask.bHasRun = true;

					FTimespan TaskDuration(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - TaskStartTime)));
					if (TaskDuration > RemainingTime + OverBudgetReportThreshold)
					{
						// Report this as verbose and not a warning because it's expected that when the frame behaves erratic during loading or
						// under heavy load this can trigger for even well behaving tasks. It's also possible that the OS takes control away from
						// the editor for a while to give other programs a chance to run which can also cause spikes. This becomes useful if there's
						// a pattern of frequent runs going over budget.
						UE_LOGF(LogEditorDataStorage, Verbose, "Time sliced task: '%ls' took %.2fms, but was allotted %.2fms",
							*CurrentTask.Name.ToString(), TaskDuration.GetTotalMilliseconds(), RemainingTime.GetTotalMilliseconds());
					}
				}

				// Find the new slot to put the processed task in by bubbling it up the queue.
				for (int32 Index = CurrentTaskIndex + 1; Index < CooperativeTasks.Num(); Index++)
				{
					if (CooperativeTasks[Index - 1].Order >= CooperativeTasks[Index].Order)
					{
						Swap(CooperativeTasks[Index - 1], CooperativeTasks[Index]);
					}
					else
					{
						break;
					}
				}
			}
			else
			{
				CurrentTaskIndex++;
			}

			LastFrameCooperativeTickDuration = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ProcessingStartTime));
			if (LastFrameCooperativeTickDuration >= RemainingTime)
			{
				break;
			}
		}

		bRunningCooperativeUpdate = false;

		CooperativeTickStartTime = FPlatformTime::Cycles64();
	}
}

bool UEditorDataStorage::IsAvailable() const
{
	return bool(ActiveEditorEntityManager);
}

void* UEditorDataStorage::GetExternalSystemAddress(UClass* Target)
{
	if (Target && Target->IsChildOf<USubsystem>())
	{
		return FMassSubsystemAccess::FetchSubsystemInstance(/*World=*/nullptr, Target);
	}
	return nullptr;
}

bool UEditorDataStorage::SupportsExtension(FName Extension) const
{
	return false;
}

void UEditorDataStorage::ListExtensions(TFunctionRef<void(FName)> Callback) const
{
}

void UEditorDataStorage::PreparePhase(UE::Editor::DataStorage::EQueryTickPhase Phase, float DeltaTime)
{
	using namespace UE::Editor::DataStorage;

	if (ActiveEditorEntityManager)
	{
		{
			// The preamble queries are all run on the game thread. While this is true it's safe to take a global write lock.
			// If there's a performance loss because this lock is held too long, the work in RunPhasePreambleQueries can be split
			// into a step that runs the queries and uses a shared lock and one that executes the command buffer with an exclusive lock.
			FScopedExclusiveLock Lock(EGlobalLockScope::Public);
			Environment->GetQueryStore().RunPhasePreambleQueries(*ActiveEditorEntityManager, *Environment, Phase, DeltaTime);
		}
		// During the processing of queries no mutation can happen to the structure of the database, just fields being updated. As such
		// it's safe to only take a shared lock
		// TODO: This requires Mass to tell TEDS it's about to flush its deferred commands.
		// FGlobalLock::InternalSharedLock();
	}
}

void UEditorDataStorage::FinalizePhase(UE::Editor::DataStorage::EQueryTickPhase Phase, float DeltaTime)
{
	using namespace UE::Editor::DataStorage;

	if (ActiveEditorEntityManager)
	{
		// During the processing of queries no mutation can happen to the structure of the database, just fields being updated. As such
		// it's safe to only take a shared lock
		// TODO: This requires Mass to tell TEDS it's about to flush its deferred commands. Right now this gets called after the
		// deferred commands are run, which require exclusive access.
		//FGlobalLock::InternalSharedUnlock();

		// The preamble queries are all run on the game thread. While this is true it's safe to take a global write lock.
		// If there's a performance loss because this lock is held too long, the work in RunPhasePostambleQueries can be split
		// into a step that runs the queries and uses a shared lock and one that executes the command buffer with an exclusive lock.
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);
		Environment->GetQueryStore().RunPhasePostambleQueries(*ActiveEditorEntityManager, *Environment, Phase, DeltaTime);

		// Process interval rebalancing for hierarchical relations at the end of each frame.
		// This is lightweight when there are no tagged rows (just a query that returns empty).
		if (Phase == EQueryTickPhase::FrameEnd )
		{
			Environment->GetHierarchicalRelationManager().ProcessRebalancing(*this);
		}

		// Flush any custom commands that have queued up in between phases.
		ActiveCallsCount--;
		checkf(ActiveCallsCount == 0, TEXT("During phase finalization there should be no TEDS running."));
		Environment->FlushCommands();
		ActiveCallsCount++;
	}
}

void UEditorDataStorage::Reset()
{
	UE_LOGF(LogEditorDataStorage, Log, "Resetting TEDS Core.");

	if (UMassEntityEditorSubsystem* Mass = UMassEntityEditorSubsystem::Get())
	{
		Mass->GetOnPostTickDelegate().Remove(OnPostMassTickHandle);
		Mass->GetOnPreTickDelegate().Remove(OnPreMassTickHandle);
	}
	OnPostMassTickHandle.Reset();
	OnPreMassTickHandle.Reset();

	if (ActiveEditorEntityManager && ActiveEditorPhaseManager)
	{
		Environment->GetQueryStore().Clear(*ActiveEditorEntityManager.Get(), *ActiveEditorPhaseManager.Get());
		Environment->GetTableManager().Clear();
	}
	
	checkf(Environment.IsUnique(), TEXT("UEditorDataStorage should hold the last reference to the Environment and be the one to delete it."));
	Environment.Reset();

	ActiveEditorPhaseManager.Reset();
	ActiveEditorEntityManager.Reset();

	UE_LOGF(LogEditorDataStorage, Log, "Reset TEDS Core.");
}

TSharedPtr<UE::Editor::DataStorage::FEnvironment> UEditorDataStorage::GetEnvironmentPtr()
{
	return Environment;
}

TSharedPtr<const UE::Editor::DataStorage::FEnvironment> UEditorDataStorage::GetEnvironmentPtr() const
{
	return Environment;
}

UE::Editor::DataStorage::FEnvironment& UEditorDataStorage::GetEnvironment()
{
	checkf(Environment, TEXT("Unable to retrieve the environment before TEDS Core has been initialized."));
	return *Environment;
}

const UE::Editor::DataStorage::FEnvironment& UEditorDataStorage::GetEnvironment() const
{
	checkf(Environment, TEXT("Unable to retrieve the environment before TEDS Core has been initialized."));
	return *Environment;
}

void UEditorDataStorage::DebugPrintQueryCallbacks(FOutputDevice& Output)
{
	Environment->GetQueryStore().DebugPrintQueryCallbacks(Output);
}

void UEditorDataStorage::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UEditorDataStorage* Database = static_cast<UEditorDataStorage*>(InThis);

	for (auto& FactoryPair : Database->Factories)
	{
		Collector.AddReferencedObject(FactoryPair.Instance);
		Collector.AddReferencedObject(FactoryPair.Type);
	}
}
