// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "MassArchetypeTypes.h"
#include "Misc/TVariant.h"
#include "Processors/QueryCapabilityImplementationForwarder.h"
#include "Templates/SharedPointer.h"
#include "TypedElementDatabaseCommandBuffer.h"
#include "UObject/ObjectMacros.h"
#include "Queries/QueryContextCapabilities/EditorDataStorageDynamicColumnInfo.h"
#include "Queries/QueryContextCapabilities/TableCapabilities.h"
#include "Queries/DirectContextEnvironment.h"

#include "TypedElementDatabase.generated.h"

#define UE_API TEDSCORE_API

struct FMassEntityManager;
struct FMassProcessingPhaseManager;
class UEditorDataStorageFactory;
class FOutputDevice;
class UWorld;

namespace UE::Editor::DataStorage
{
	class FEnvironment;
} // namespace UE::Editor::DataStorage

using ImplementationForwarder = UE::Editor::DataStorage::Queries::Private::TImplementationForwarder<
	UE::Editor::DataStorage::Queries::FDirectContextEnvironment,
	UE::Editor::DataStorage::ICoreProvider,
	UE::Editor::DataStorage::Queries::FTableInfoContextCapability,
	UE::Editor::DataStorage::Queries::FTableManagementContextCapability,
	UE::Editor::DataStorage::Queries::FDynamicColumnInfoContextCapability>;

UCLASS(MinimalAPI)
class UEditorDataStorage final
	: public UObject
	, public ImplementationForwarder
{
	GENERATED_BODY()

public:
	template<typename FactoryType, typename DatabaseType>
	class TFactoryIterator
	{
	public:
		using ThisType = TFactoryIterator<FactoryType, DatabaseType>;
		using FactoryPtr = FactoryType*;
		using DatabasePtr = DatabaseType*;

		TFactoryIterator() = default;
		explicit TFactoryIterator(DatabasePtr InDatabase);

		FactoryPtr operator*() const;
		ThisType& operator++();
		operator bool() const;

	private:
		DatabasePtr Database = nullptr;
		int32 Index = 0;
	};

	using FactoryIterator = TFactoryIterator<UEditorDataStorageFactory, UEditorDataStorage>;
	using FactoryConstIterator = TFactoryIterator<const UEditorDataStorageFactory, const UEditorDataStorage>;

public:
	~UEditorDataStorage() override;
	
	UE_API void Initialize();
	UE_API void PostInitialize();
	
	UE_API void SetFactories(TConstArrayView<UClass*> InFactories);
	UE_API void ResetFactories();

	/** An iterator which allows traversal of factory instances. Ordered lowest->highest of GetOrder() */
	UE_API FactoryIterator CreateFactoryIterator();
	/** An iterator which allows traversal of factory instances. Ordered lowest->highest of GetOrder() */
	UE_API FactoryConstIterator CreateFactoryIterator() const;

	/** Returns factory instance given the type of factory */
	UE_API virtual const UEditorDataStorageFactory* FindFactory(const UClass* FactoryType) const override;

	UE_API virtual UEditorDataStorageFactory* FindFactory(const UClass* FactoryType) override;
	
	UE_API void Deinitialize();

	/** Triggered at the start of the underlying Mass' tick cycle. */
	UE_API void OnPreMassTick(float DeltaTime);
	/** Triggered just before underlying Mass processing completes it's tick cycle. */
	UE_API void OnPostMassTick(float DeltaTime);

	UE_API TSharedPtr<FMassEntityManager> GetActiveMutableEditorEntityManager();
	UE_API TSharedPtr<const FMassEntityManager> GetActiveEditorEntityManager() const;


	UE_API virtual UE::Editor::DataStorage::RowHandle ReserveRow() override;
	UE_API virtual void BatchReserveRows(int32 Count, TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> ReservationCallback) override;
	UE_API virtual void BatchReserveRows(TArrayView<UE::Editor::DataStorage::RowHandle> ReservedRows) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle AddRow(UE::Editor::DataStorage::TableHandle Table, 
		UE::Editor::DataStorage::RowCreationCallbackRef OnCreated) override;
	UE_API UE::Editor::DataStorage::RowHandle AddRow(UE::Editor::DataStorage::TableHandle Table) override;
	UE_API virtual bool AddRow(UE::Editor::DataStorage::RowHandle ReservedRow, UE::Editor::DataStorage::TableHandle Table) override;
	UE_API virtual bool AddRow(UE::Editor::DataStorage::RowHandle ReservedRow, UE::Editor::DataStorage::TableHandle Table,
		UE::Editor::DataStorage::RowCreationCallbackRef OnCreated) override;
	UE_API virtual bool BatchAddRow(UE::Editor::DataStorage::TableHandle Table, int32 Count,
		UE::Editor::DataStorage::RowCreationCallbackRef OnCreated) override;
	UE_API virtual bool BatchAddRow(UE::Editor::DataStorage::TableHandle Table, TConstArrayView<UE::Editor::DataStorage::RowHandle> ReservedHandles,
		UE::Editor::DataStorage::RowCreationCallbackRef OnCreated) override;
	UE_API virtual void RemoveRow(UE::Editor::DataStorage::RowHandle Row) override;
	UE_API virtual void BatchRemoveRows(TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows) override;
	UE_API virtual void RemoveAllRowsWithColumns(TConstArrayView<const UScriptStruct*> Columns) override;
	UE_API virtual bool IsRowAvailable(UE::Editor::DataStorage::RowHandle Row) const override;
	UE_API virtual bool IsRowAssigned(UE::Editor::DataStorage::RowHandle Row) const override;
	/** Same as IsRowAvailable, but doesn't check if the data storage has been initialized. */
	UE_API bool IsRowAvailableUnsafe(UE::Editor::DataStorage::RowHandle Row) const;
	/** Same as IsRowAssigned, but doesn't check if the data storage has been initialized. */
	UE_API bool IsRowAssignedUnsafe(UE::Editor::DataStorage::RowHandle Row) const;
	virtual void FilterRowsBy(UE::Editor::DataStorage::FRowHandleArray& Result, UE::Editor::DataStorage::FRowHandleArrayView Input, 
		EFilterOptions Options, UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>& Filter) const override;

	UE_API virtual void AddColumn(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual void AddColumn(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::FValueTag& Tag, const FName& InValue) override;
	UE_API virtual void AddColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType,
		const UE::Editor::DataStorage::ColumnCreationCallbackRef Initializer,
		UE::Editor::DataStorage::ColumnCopyOrMoveCallback Relocator) override;
	UE_API virtual void RemoveColumn(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual void RemoveColumn(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::FValueTag& Tag) override;
	UE_API virtual void* GetColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual const void* GetColumnData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) const override;
	UE_API virtual void AddColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> Columns) override;
	UE_API virtual void RemoveColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> Columns) override;
	UE_API virtual void AddRemoveColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnsToAdd,
		TConstArrayView<const UScriptStruct*> ColumnsToRemove) override;
	UE_API virtual void BatchAddRemoveColumns(TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows,TConstArrayView<const UScriptStruct*> ColumnsToAdd,
		TConstArrayView<const UScriptStruct*> ColumnsToRemove) override;
	UE_API virtual bool HasColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<const UScriptStruct*> ColumnTypes) const override;
	UE_API virtual bool HasColumns(UE::Editor::DataStorage::RowHandle Row, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes) const override;
	UE_API virtual void ListColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ColumnListCallbackRef Callback) const override;
	UE_API virtual void ListColumns(UE::Editor::DataStorage::RowHandle Row, UE::Editor::DataStorage::ColumnListWithDataCallbackRef Callback) override;
	UE_API virtual bool MatchesColumns(UE::Editor::DataStorage::RowHandle Row, const UE::Editor::DataStorage::Queries::FConditions& Conditions) const override;

	UE_API virtual const UScriptStruct* GenerateDynamicColumn(const UE::Editor::DataStorage::FDynamicColumnDescription& Description) override;
	UE_API virtual void ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const UScriptStruct& Type)> Callback) const override;

	UE_API virtual UE::Editor::DataStorage::FHierarchyHandle RegisterHierarchy(const UE::Editor::DataStorage::FHierarchyRegistrationParams& Params) override;
	UE_API virtual UE::Editor::DataStorage::FHierarchyHandle FindHierarchyByName(const FName& Name) const override;
	UE_API virtual bool IsValidHierarchyHandle(UE::Editor::DataStorage::FHierarchyHandle) const override;
	UE_API virtual const UScriptStruct* GetChildTagType(
		UE::Editor::DataStorage::FHierarchyHandle) const override;
	UE_API virtual const UScriptStruct* GetParentTagType(
		UE::Editor::DataStorage::FHierarchyHandle) const override;
	UE_API virtual const UScriptStruct* GetHierarchyDataColumnType(
		UE::Editor::DataStorage::FHierarchyHandle) const override;
	UE_API virtual const UScriptStruct* GetParentChangedColumnType(
		UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) const override;
	UE_API virtual void ResolveHierarchy(UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle) override;
	UE_API virtual void ListHierarchyNames(TFunctionRef<void(const FName& HierarchyName)> Callback) const override;
	UE_API virtual void SetParentRow(
		UE::Editor::DataStorage::FHierarchyHandle, UE::Editor::DataStorage::RowHandle Target,	UE::Editor::DataStorage::RowHandle Parent) override;
	virtual void SetUnresolvedParent(
		UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, UE::Editor::DataStorage::RowHandle Target, UE::Editor::DataStorage::FMapKey ParentId,
		FName MappingDomain) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetParentRow(
		UE::Editor::DataStorage::FHierarchyHandle, UE::Editor::DataStorage::RowHandle Target) const override;
	UE_API virtual TFunction<UE::Editor::DataStorage::RowHandle(const void*, const UScriptStruct*)> CreateParentExtractionFunction(
		UE::Editor::DataStorage::FHierarchyHandle) const override;
	UE_API virtual bool HasChildren(
		UE::Editor::DataStorage::FHierarchyHandle, UE::Editor::DataStorage::RowHandle Row) const override;
	UE_API virtual void WalkDepthFirst(
		UE::Editor::DataStorage::FHierarchyHandle,
		UE::Editor::DataStorage::RowHandle Row, 
		TFunction<void(const ICoreProvider& Context, UE::Editor::DataStorage::RowHandle Owner, UE::Editor::DataStorage::RowHandle Target)> VisitFn,
		ETraversalOrder TraversalOrder = ETraversalOrder::PreOrder) const override;
	UE_API virtual bool IterateChildren(
		UE::Editor::DataStorage::FHierarchyHandle,
		UE::Editor::DataStorage::RowHandle Row,
		TFunctionRef<bool(const ICoreProvider& Context, UE::Editor::DataStorage::RowHandle Child)> VisitFn) const override;

	// Scope
	UE_API virtual UE::Editor::DataStorage::RowHandle AddScopeRow() override;
	UE_API virtual UE::Editor::DataStorage::RowHandle AddScopeRow(FStringView Label) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle AddScopeRow(FStringView Label, UE::Editor::DataStorage::RowHandle Parent) override;
	UE_API virtual void RemoveScopeRow(UE::Editor::DataStorage::RowHandle Row) override;
	UE_API virtual void SetParentScope(UE::Editor::DataStorage::RowHandle Child, UE::Editor::DataStorage::RowHandle Parent) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetParentScope(UE::Editor::DataStorage::RowHandle Row) override;
	UE_API virtual const void* GetScopeDataRaw(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual UE::Editor::DataStorage::Scope::FScopeDataVersion GetScopeDataVersion(
		UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual UE::Editor::DataStorage::Scope::FScopeDataVersion SetScopeData(
		UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType,
		UE::Editor::DataStorage::ColumnCreationCallbackRef Initializer, UE::Editor::DataStorage::ColumnCopyOrMoveCallback Relocator) override;
	UE_API virtual bool RemoveScopeData(UE::Editor::DataStorage::RowHandle Row, const UScriptStruct* ColumnType) override;
	UE_API virtual TArray<const UScriptStruct*> GetAllVisibleScopeColumns(
		UE::Editor::DataStorage::RowHandle Row) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetRootScope() const override;
	UE_API virtual UE::Editor::DataStorage::FHierarchyHandle GetScopeHierarchy() override;

	// Relations Interface - Relation Type Management
	UE_API virtual UE::Editor::DataStorage::RelationTypeHandle RegisterRelationType(
		const UE::Editor::DataStorage::FRelationRegistrationParams& Params) override;
	UE_API virtual UE::Editor::DataStorage::RelationTypeHandle FindRelationType(
		const FName& InName) const override;
	UE_API virtual bool IsValidRelationType(
		UE::Editor::DataStorage::RelationTypeHandle Type) const override;
	UE_API virtual const UE::Editor::DataStorage::FTedsRelationTraits* GetRelationTypeTraits(
		UE::Editor::DataStorage::RelationTypeHandle Type) const override;
	UE_API virtual void ListRelationTypes(
		TFunctionRef<void(UE::Editor::DataStorage::RelationTypeHandle, const FName&)> Callback) const override;
	UE_API virtual const UScriptStruct* GetRelationSubjectChangedColumn(
		UE::Editor::DataStorage::RelationTypeHandle Type) const override;
	UE_API virtual const UScriptStruct* GetRelationObjectChangedColumn(
		UE::Editor::DataStorage::RelationTypeHandle Type) const override;

	// Relations Interface - Relation Instance Management
	UE_API virtual UE::Editor::DataStorage::RowHandle CreateRelation(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject,
		UE::Editor::DataStorage::RowHandle Object) override;
	UE_API virtual void BatchCreateRelations(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		TConstArrayView<UE::Editor::DataStorage::RowHandle> Subjects,
		TConstArrayView<UE::Editor::DataStorage::RowHandle> Objects,
		TArray<UE::Editor::DataStorage::RowHandle>* OutRelationRows = nullptr) override;
	UE_API virtual bool DestroyRelation(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject,
		UE::Editor::DataStorage::RowHandle Object) override;
	UE_API virtual bool HasRelation(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject,
		UE::Editor::DataStorage::RowHandle Object) const override;
	UE_API virtual bool HasRelationObject(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject) const override;
	UE_API virtual bool HasRelationSubject(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Object) const override;

	// Relations Interface - Relation Queries
	UE_API virtual void GetRelationObjects(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject,
		TArray<UE::Editor::DataStorage::RowHandle>& OutObjects) const override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetRelationObject(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject) const override;
	UE_API virtual void GetRelationSubjects(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Object,
		TArray<UE::Editor::DataStorage::RowHandle>& OutSubjects) const override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetRelationSubject(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Object) const override;

	// Relations Interface - Hierarchical Queries
	UE_API virtual bool IsDescendantOf(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Descendant,
		UE::Editor::DataStorage::RowHandle Ancestor,
		bool bIncludeSelf = false) const override;
	UE_API virtual UE::Editor::DataStorage::RowHandle GetHierarchyRoot(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Row) const override;
	UE_API virtual int32 GetHierarchyDepth(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Row) const override;
	UE_API virtual void GetDescendants(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Ancestor,
		TArray<UE::Editor::DataStorage::RowHandle>& OutDescendants) const override;
	UE_API virtual void GetAncestors(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Descendant,
		TArray<UE::Editor::DataStorage::RowHandle>& OutAncestors) const override;
	UE_API virtual void TraverseDescendants(
		UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle StartRow,
		FRelationTraversalCallback Callback,
		ETraversalOrder Order = ETraversalOrder::PreOrder,
		int32 MaxDepth = TNumericLimits<int32>::Max()) const override;

	// Relations Interface - Callback-Based Enumeration
	UE_API virtual void ForEachRelationObject(UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Subject,
		TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> Callback) const override;
	UE_API virtual void ForEachRelationSubject(UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Object,
		TFunctionRef<void(UE::Editor::DataStorage::RowHandle)> Callback) const override;
	UE_API virtual void TraverseAncestors(UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Descendant,
		TFunctionRef<bool(UE::Editor::DataStorage::RowHandle, int32)> Callback) const override;
	UE_API virtual int32 ComputeDescendantCount(UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Ancestor) const override;
	UE_API virtual bool HasRelationSubjects(UE::Editor::DataStorage::RelationTypeHandle Type,
		UE::Editor::DataStorage::RowHandle Object) const override;

	UE_API void RegisterTickGroup(FName GroupName, UE::Editor::DataStorage::EQueryTickPhase Phase, FName BeforeGroup, FName AfterGroup, UE::Editor::DataStorage::EExecutionMode ExecutionMode);
	UE_API void UnregisterTickGroup(FName GroupName, UE::Editor::DataStorage::EQueryTickPhase Phase);

	UE_API UE::Editor::DataStorage::QueryHandle RegisterQuery(UE::Editor::DataStorage::FQueryDescription&& Query) override;
	UE_API virtual void UnregisterQuery(UE::Editor::DataStorage::QueryHandle Query) override;
	UE_API virtual const UE::Editor::DataStorage::FQueryDescription& GetQueryDescription(UE::Editor::DataStorage::QueryHandle Query) const override;
	UE_API virtual uint64 CalculateQueryTablesTopologyHash(UE::Editor::DataStorage::QueryHandle Query) const override;
	UE_API virtual FName GetQueryTickGroupName(UE::Editor::DataStorage::EQueryTickGroups Group) const override;
	UE_API virtual UE::Editor::DataStorage::FQueryResult RunQuery(UE::Editor::DataStorage::QueryHandle Query) override;
	UE_API virtual UE::Editor::DataStorage::FQueryResult RunQuery(UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::DirectQueryCallbackRef Callback) override;
	UE_API virtual UE::Editor::DataStorage::FQueryResult RunQuery(UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::EDirectQueryExecutionFlags Flags,
		UE::Editor::DataStorage::DirectQueryCallbackRef Callback) override;
	UE_API virtual UE::Editor::DataStorage::FQueryResult RunQuery(UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::ERunQueryFlags Flags,
		const UE::Editor::DataStorage::Queries::TQueryFunction<void>& Callback) override;
	UE_API virtual UE::Editor::DataStorage::FQueryResult RunQuery(UE::Editor::DataStorage::QueryHandle Query, UE::Editor::DataStorage::ERunQueryFlags Flags,
		const UE::Editor::DataStorage::Queries::TConstQueryFunction<void>& Callback) const override;
	UE_API virtual void ActivateQueries(FName ActivationName) override;
	
	UE_API virtual UE::Editor::DataStorage::RowHandle LookupMappedRow(
		const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& Key) const override;
	UE_API virtual void MapRow(
		const FName& Domain, UE::Editor::DataStorage::FMapKey Key, UE::Editor::DataStorage::RowHandle Row) override;
	UE_API virtual void BatchMapRows(
		const FName& Domain, TArrayView<TPair<UE::Editor::DataStorage::FMapKey, UE::Editor::DataStorage::RowHandle>> MapRowPairs) override;
	UE_API virtual void RemapRow(
		const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& OriginalKey, UE::Editor::DataStorage::FMapKey NewKey) override;
	UE_API virtual void RemoveRowMapping(
		const FName& Domain, const UE::Editor::DataStorage::FMapKeyView& Key) override;

	UE_API virtual UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate& OnUpdate() override;
	UE_API virtual UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate& OnUpdateCompleted() override;
	UE_API virtual void RegisterCooperativeUpdate(const FName& TaskName, ECooperativeTaskPriority Priority, FOnCooperativeUpdate Callback) override;
	UE_API virtual void UnregisterCooperativeUpdate(const FName& TaskName) override;
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual void* GetExternalSystemAddress(UClass* Target) override;

	UE_API virtual bool SupportsExtension(FName Extension) const override;
	UE_API virtual void ListExtensions(TFunctionRef<void(FName)> Callback) const override;
	
	UE_API TSharedPtr<UE::Editor::DataStorage::FEnvironment> GetEnvironmentPtr();
	UE_API TSharedPtr<const UE::Editor::DataStorage::FEnvironment> GetEnvironmentPtr() const;

	UE::Editor::DataStorage::FEnvironment& GetEnvironment();
	const UE::Editor::DataStorage::FEnvironment& GetEnvironment() const;

	UE_API void DebugPrintQueryCallbacks(FOutputDevice& Output) override;

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:
	void PreparePhase(UE::Editor::DataStorage::EQueryTickPhase Phase, float DeltaTime);
	void FinalizePhase(UE::Editor::DataStorage::EQueryTickPhase Phase, float DeltaTime);
	void Reset();

	void TickCooperativeTasks();
	
	struct FFactoryTypePair
	{
		// Used to find the factory by type without needing to dereference each one
		TObjectPtr<UClass> Type;
		
		TObjectPtr<UEditorDataStorageFactory> Instance;
	};

	// Wrapper around ActiveCallsCount to avoid issues with timing such as early outs.
	struct FActiveCallScope
	{
		explicit FActiveCallScope(const UEditorDataStorage& Owner);
		~FActiveCallScope();

		FActiveCallScope(const FActiveCallScope&) = delete;
		FActiveCallScope(FActiveCallScope&&) = delete;
		FActiveCallScope& operator=(const FActiveCallScope&) = delete;
		FActiveCallScope& operator=(FActiveCallScope&&) = delete;

		const UEditorDataStorage& Owner;
	};
	
	static UE_API const FName TickGroupName_Default;
	static UE_API const FName TickGroupName_PreUpdate;
	static UE_API const FName TickGroupName_Update;
	static UE_API const FName TickGroupName_PostUpdate;
	static UE_API const FName TickGroupName_SyncWidget;
	static UE_API const FName TickGroupName_SyncExternalToDataStorage;
	static UE_API const FName TickGroupName_SyncDataStorageToExternal;
	
	// Ordered array of factories by the return value of GetOrder()
	TArray<FFactoryTypePair> Factories;

	TSharedPtr<UE::Editor::DataStorage::FEnvironment> Environment;

	UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate OnUpdateDelegate;
	UE::Editor::DataStorage::FTypedElementOnDataStorageUpdate OnUpdateCompletedDelegate;
	FDelegateHandle OnPreMassTickHandle;
	FDelegateHandle OnPostMassTickHandle;

	struct FCooperativeTask
	{
		FOnCooperativeUpdate Callback;
		FName Name;
		uint8 OrderResetValue;
		uint8 Order;
		bool bHasRun;
	};
	TArray<FCooperativeTask> CooperativeTasks;
	FTimespan LastFrameCooperativeTickDuration;
	uint64 CooperativeTickStartTime = 0;
	// The number of frames the cooperative tasks have been idle due to a lack of available time. Once this
	// reaches a user specified number the cooperative tasks will start processing with a minimal amount of time
	// to avoid tasks starving.
	int32 CooperativeTaskIdleFrameCount = 0;
	bool bRunningCooperativeUpdate = false;

	TSharedPtr<FMassEntityManager> ActiveEditorEntityManager;
	TSharedPtr<FMassProcessingPhaseManager> ActiveEditorPhaseManager;

	// Track the number of active calls, incrementing whenever a function calls another direct API call. This prevents
	// the command buffer from being flushed early and guarantees it only being flushed when the call chain has completed.
	mutable uint32 ActiveCallsCount = 0;
};

template <typename FactoryType, typename DatabaseType>
UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::TFactoryIterator(DatabasePtr InDatabase): Database(InDatabase)
{}

template <typename FactoryType, typename DatabaseType>
typename UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::FactoryPtr UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::operator*() const
{
	return Database->Factories[Index].Instance;
}

template <typename FactoryType, typename DatabaseType>
typename UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::ThisType& UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::operator++()
{
	if (Database != nullptr && Index < Database->Factories.Num())
	{
		++Index;
	}
	return *this;
}

template <typename FactoryType, typename DatabaseType>
UEditorDataStorage::TFactoryIterator<FactoryType, DatabaseType>::operator bool() const
{
	return Database != nullptr && Index < Database->Factories.Num();
}

#undef UE_API
