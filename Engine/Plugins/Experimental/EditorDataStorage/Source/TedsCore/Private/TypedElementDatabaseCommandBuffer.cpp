// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseCommandBuffer.h"

#include "HAL/UnrealMemory.h"
#include "MassEntityManager.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementDataStorageSharedColumn.h"

namespace UE::Editor::DataStorage::Legacy
{
	// 
	// Commands section
	//

	FCommandBuffer::FCommandBuffer(FEnvironment& InEnvironment)
		: Environment(InEnvironment)
	{
	}

	void* FCommandBuffer::GetQueuedDataColumn(
		RowHandle Row, const UScriptStruct* ColumnType)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType),
			TEXT("Trying to get the column '%s' which isn't a data column."), *ColumnType->GetName());

		void* const* StoredData = PendingColumns.Find(PendingColumnMappingKey(Row, ColumnType));
		if (StoredData)
		{
			if (*StoredData != nullptr)
			{
				return *StoredData;
			}
			else
			{
				// If the column was created but had no data assigned to it, create data for it now. If this code path triggers
				// a lot there may be a large number of AddColumn followed by GetColumn calls. These can be more efficiently done
				// with an AddorGetColumn call.
				void* Result = Queue_AddDataColumnCommandUnitialized(Row, ColumnType,
					[](const UScriptStruct& ColumnType, void* Destination, void* Source)
					{
						ColumnType.CopyScriptStruct(Destination, Source);
					});
				return Result;
			}
		}
		else
		{
			return nullptr;
		}
	}

	bool FCommandBuffer::HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const
	{
		return PendingColumns.Contains(PendingColumnMappingKey(Row, ColumnType));
	}

	void FCommandBuffer::ListColumns(RowHandle Row, ColumnListCallbackRef Callback) const
	{
		for (const TPair<PendingColumnMappingKey, void*>& Column : PendingColumns)
		{
			if (Column.Key.Key == Row)
			{
				if (const UScriptStruct* ColumnType = Column.Key.Value.Get())
				{
					Callback(*ColumnType);
				}
			}
		}
	}

	void FCommandBuffer::ListColumns(RowHandle Row, ColumnListWithDataCallbackRef Callback)
	{
		for (const TPair<PendingColumnMappingKey, void*>& Column : PendingColumns)
		{
			if (Column.Key.Key == Row)
			{
				if (const UScriptStruct* ColumnType = Column.Key.Value.Get())
				{
					Callback(Column.Value, *ColumnType);
				}
			}
		}
	}

	FCommandBuffer::FAddDataColumnCommand::~FAddDataColumnCommand()
	{
		if (ColumnType.IsValid() && (ColumnType->StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor)) == 0)
		{
			ColumnType->DestroyStruct(Data);
		}
	}

	//
	// Queue section
	//

	void FCommandBuffer::Queue_AddColumnCommand(RowHandle Row, const UScriptStruct* ColumnType)
	{
		AddCommand(FAddColumnCommand
			{
				.Row = Row,
				.ColumnType = ColumnType
			});
		PendingColumns.FindOrAdd(PendingColumnMappingKey(Row, ColumnType), nullptr);
	}

	void* FCommandBuffer::Queue_AddDataColumnCommandUnitialized(
		RowHandle Row, const UScriptStruct* ColumnType, ColumnCopyOrMoveCallback Relocator)
	{
		checkf(UE::Mass::IsA<FMassFragment>(ColumnType),
			TEXT("Trying to queue a data column creation for '%s' which isn't a data column."), *ColumnType->GetName());

		PendingColumnMappingKey Key(Row, ColumnType);
		if (void** StoredData = PendingColumns.Find(Key); StoredData == nullptr || *StoredData == nullptr)
		{
			// Initialize to zero to replicate the default from Mass.
			void* Data = Environment.GetScratchBuffer().AllocateZeroInitialized(ColumnType->GetStructureSize(), ColumnType->GetMinAlignment());
			PendingColumns.Add(PendingColumnMappingKey(Row, ColumnType), Data);
			AddCommand(FAddDataColumnCommand
				{
					.Row = Row,
					.ColumnType = ColumnType,
					.Relocator = Relocator,
					.Data = Data
				});
			return Data;
		}
		else
		{
			return *StoredData;
		}
	}

	void FCommandBuffer::Queue_AddColumnsCommand(RowHandle Row,
		FMassFragmentBitSet FragmentsToAdd, FMassTagBitSet TagsToAdd)
	{
		auto AddColumn = [this, Row](const UScriptStruct* ColumnType)
		{
			PendingColumns.FindOrAdd(PendingColumnMappingKey(Row, ColumnType), nullptr);
			return true;
		};
		FragmentsToAdd.ExportTypes(AddColumn);
		TagsToAdd.ExportTypes(AddColumn);

		AddCommand(FAddColumnsCommand
			{
				.Row = Row,
				.FragmentsToAdd = MoveTemp(FragmentsToAdd),
				.TagsToAdd = MoveTemp(TagsToAdd)
			});
	}

	void FCommandBuffer::Queue_RemoveColumnCommand(RowHandle Row, const UScriptStruct* ColumnType)
	{
		AddCommand(FRemoveColumnCommand
			{
				.Row = Row,
				.ColumnType = ColumnType
			});
		PendingColumns.Remove(PendingColumnMappingKey(Row, ColumnType));
	}

	void FCommandBuffer::Queue_RemoveColumnsCommand(RowHandle Row, FMassFragmentBitSet FragmentsToRemove, FMassTagBitSet TagsToRemove)
	{
		auto RemoveColumn = [this, Row](const UScriptStruct* ColumnType)
		{
			PendingColumns.Remove(PendingColumnMappingKey(Row, ColumnType));
			return true;
		};
		FragmentsToRemove.ExportTypes(RemoveColumn);
		TagsToRemove.ExportTypes(RemoveColumn);

		AddCommand(FRemoveColumnsCommand
			{
				.Row = Row,
				.FragmentsToRemove = MoveTemp(FragmentsToRemove),
				.TagsToRemove = MoveTemp(TagsToRemove)
			});
	}

	void FCommandBuffer::Queue_AddRemoveColumnsCommand(TConstArrayView<RowHandle> Rows, FMassElementBitSet ColumnsToAdd, FMassElementBitSet ColumnsToRemove)
	{
		for (RowHandle Row : Rows)
		{
			auto AddColumn = [this, Row](const UScriptStruct* ColumnType)
				{
					PendingColumns.FindOrAdd(PendingColumnMappingKey(Row, ColumnType), nullptr);
					return true;
				};
			ColumnsToAdd.ExportTypes(AddColumn);

			auto RemoveColumn = [this, Row](const UScriptStruct* ColumnType)
				{
					PendingColumns.Remove(PendingColumnMappingKey(Row, ColumnType));
					return true;
				};
			ColumnsToRemove.ExportTypes(RemoveColumn);
		}

		int32 RowCount = Rows.Num();
		TArrayView<RowHandle> LocalRows = Environment.GetScratchBuffer().AllocateUninitializedArray<RowHandle>(RowCount);
		FMemory::Memcpy(LocalRows.GetData(), Rows.GetData(), RowCount * sizeof(RowHandle));

		AddCommand(FAddRemoveColumnsCommand
			{
				.Rows = LocalRows,
				.ElementsToAdd = MoveTemp(ColumnsToAdd),
				.ElementsToRemove = MoveTemp(ColumnsToRemove)
			});
	}

	//
	// Execute section
	//

	bool FCommandBuffer::Execute_IsRowAvailable(const FMassEntityManager& MassEntityManager, RowHandle Row)
	{
		return MassEntityManager.IsEntityValid(FMassEntityHandle::FromNumber(Row));
	}

	bool FCommandBuffer::Execute_IsRowAssigned(const FMassEntityManager& MassEntityManager, RowHandle Row)
	{
		return MassEntityManager.IsEntityActive(FMassEntityHandle::FromNumber(Row));
	}

	void FCommandBuffer::Execute_AddColumnCommand(
		FMassEntityManager& MassEntityManager,
		RowHandle Row,
		const UScriptStruct* ColumnType)
	{
		if (ColumnType && Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			if (UE::Mass::IsA<FMassTag>(ColumnType))
			{
				MassEntityManager.AddTagToEntity(Entity, ColumnType);
			}
			else if (UE::Mass::IsA<FMassFragment>(ColumnType))
			{
				FStructView Column = MassEntityManager.GetFragmentDataStruct(Entity, ColumnType);
				// Only add if not already added to avoid asserts from Mass.
				if (!Column.IsValid())
				{
					MassEntityManager.AddFragmentToEntity(Entity, ColumnType);
				}
			}
		}
	}

	void FCommandBuffer::Execute_AddDataColumnCommand(
		FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct* ColumnType,
		void* Data, ColumnCopyOrMoveCallback Relocator)
	{
		if (ColumnType && Execute_IsRowAssigned(MassEntityManager, Row))
		{
			checkf(UE::Mass::IsA<FMassFragment>(ColumnType),
				TEXT("Trying to create a data column for '%s' from a deferred command that isn't a data column."), *ColumnType->GetName());

			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			FStructView Column = MassEntityManager.GetFragmentDataStruct(Entity, ColumnType);
			// Only add if not already added to avoid asserts from Mass.
			if (!Column.IsValid())
			{
				MassEntityManager.AddFragmentToEntity(Entity, ColumnType,
					[Data, Relocator](void* Fragment, const UScriptStruct& FragmentType)
					{
						Relocator(FragmentType, Fragment, Data);
					});
			}
			else
			{
				Relocator(*ColumnType, Column.GetMemory(), Data);
			}
		}
	}

	void FCommandBuffer::Execute_AddSharedColumnCommand(FMassEntityManager& MassEntityManager, RowHandle Row, const FConstSharedStruct& SharedColumn)
	{
		if (SharedColumn.IsValid() && Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			MassEntityManager.AddConstSharedFragmentToEntity(Entity, SharedColumn);
		}
	}

	void FCommandBuffer::Execute_RemoveSharedColumnCommand(FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct& ColumnType)
	{
		if (ColumnType.IsChildOf(FTedsSharedColumn::StaticStruct()) && Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			MassEntityManager.RemoveConstSharedFragmentFromEntity(Entity, &ColumnType);
		}
	}

	void FCommandBuffer::Execute_AddColumnsCommand(
		FMassEntityManager& MassEntityManager,
		RowHandle Row,
		FMassFragmentBitSet FragmentsToAdd,
		FMassTagBitSet TagsToAdd)
	{
		if (Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassArchetypeCompositionDescriptor AddComposition(
				MoveTemp(FragmentsToAdd), MoveTemp(TagsToAdd), FMassChunkFragmentBitSet(), FMassSharedFragmentBitSet(), FMassConstSharedFragmentBitSet());
			MassEntityManager.AddCompositionToEntity_GetDelta(FMassEntityHandle::FromNumber(Row), AddComposition);
		}
	}

	void FCommandBuffer::Execute_RemoveColumnCommand(
		FMassEntityManager& MassEntityManager,
		RowHandle Row,
		const UScriptStruct* ColumnType)
	{
		if (ColumnType && Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(Row);
			if (UE::Mass::IsA<FMassTag>(ColumnType))
			{
				MassEntityManager.RemoveTagFromEntity(Entity, ColumnType);
			}
			else if (UE::Mass::IsA<FMassFragment>(ColumnType))
			{
				MassEntityManager.RemoveFragmentFromEntity(Entity, ColumnType);
			}
		}
	}

	void FCommandBuffer::Execute_RemoveColumnsCommand(
		FMassEntityManager& MassEntityManager,
		RowHandle Row,
		FMassFragmentBitSet FragmentsToRemove,
		FMassTagBitSet TagsToRemove)
	{
		if (Execute_IsRowAssigned(MassEntityManager, Row))
		{
			FMassArchetypeCompositionDescriptor RemoveComposition(
				MoveTemp(FragmentsToRemove), MoveTemp(TagsToRemove), FMassChunkFragmentBitSet(), FMassSharedFragmentBitSet(), FMassConstSharedFragmentBitSet());
			MassEntityManager.RemoveCompositionFromEntity(FMassEntityHandle::FromNumber(Row), RemoveComposition);
		}
	}

	void FCommandBuffer::Execute_AddRemoveColumnsCommand(FMassEntityManager& MassEntityManager, TArrayView<RowHandle> Rows,
		FMassElementBitSet ElementsToAdd, FMassElementBitSet ElementsToRemove)
	{
		using EntityHandleArray = TArray<FMassEntityHandle, TInlineAllocator<32>>;
		using EntityArchetypeLookup = TMap<FMassArchetypeHandle, EntityHandleArray, TInlineSetAllocator<32>>;
		using ArchetypeEntityArray = TArray<FMassArchetypeEntityCollection, TInlineAllocator<32>>;

		EntityArchetypeLookup LookupTable;
		for (RowHandle EntityId : Rows)
		{
			FMassEntityHandle Entity = FMassEntityHandle::FromNumber(EntityId);
			if (MassEntityManager.IsEntityActive(Entity))
			{
				FMassArchetypeHandle Archetype = MassEntityManager.GetArchetypeForEntity(Entity);
				EntityHandleArray& EntityCollection = LookupTable.FindOrAdd(Archetype);
				EntityCollection.Add(Entity);
			}
		}

		if (!LookupTable.IsEmpty())
		{
			// Construct table (archetype) specific row (entity) collections.
			ArchetypeEntityArray EntityCollections;
			EntityCollections.Reserve(LookupTable.Num());
			for (auto It = LookupTable.CreateConstIterator(); It; ++It)
			{
				EntityCollections.Emplace(It.Key(), It.Value(), FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates);
			}

			MassEntityManager.BatchChangeCompositionForEntities(EntityCollections, ElementsToAdd, ElementsToRemove);
		}
	}

	void FCommandBuffer::ProcessCommands()
	{
		struct FProcessor
		{
			FMassEntityManager& EntityManager;
			void operator()(FAddColumnCommand&& Command) 
			{ 
				Execute_AddColumnCommand(EntityManager, Command.Row, Command.ColumnType.Get());
			}
			void operator()(FAddDataColumnCommand&& Command) 
			{ 
				Execute_AddDataColumnCommand(EntityManager, Command.Row, Command.ColumnType.Get(), Command.Data, Command.Relocator);
			}
			void operator()(FAddColumnsCommand&& Command)
			{ 
				Execute_AddColumnsCommand(EntityManager, Command.Row, MoveTemp(Command.FragmentsToAdd), MoveTemp(Command.TagsToAdd));
			}
			void operator()(FRemoveColumnCommand&& Command) 
			{ 
				Execute_RemoveColumnCommand(EntityManager, Command.Row, Command.ColumnType.Get());
			}
			void operator()(FRemoveColumnsCommand&& Command) 
			{ 
				Execute_RemoveColumnsCommand(EntityManager, Command.Row, MoveTemp(Command.FragmentsToRemove), MoveTemp(Command.TagsToRemove));
			}
			void operator()(FAddRemoveColumnsCommand&& Command) 
			{ 
				Execute_AddRemoveColumnsCommand(EntityManager, Command.Rows, MoveTemp(Command.ElementsToAdd), MoveTemp(Command.ElementsToRemove));
			}
		};

		FMassEntityManager& EntityManager = Environment.GetMassEntityManager();
		FProcessor Processor{ .EntityManager = EntityManager };
		for (CommandData& Command : Commands)
		{
			Visit(Processor, MoveTemp(Command));
		}

		PendingColumns.Reset();
		Commands.Reset();
	}

	void FCommandBuffer::ClearCommands()
	{
		PendingColumns.Reset();
		Commands.Reset();
	}

	// 
	// misc
	//

	template<typename T>
	void FCommandBuffer::AddCommand(T&& Args)
	{
		CommandData Command;
		Command.Emplace<T>(Forward<T>(Args));
		Commands.Add(MoveTemp(Command));
	}
} // namespace UE::Editor::DataStorage::Legacy
