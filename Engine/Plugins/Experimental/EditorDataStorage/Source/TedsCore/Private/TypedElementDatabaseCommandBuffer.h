// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "DataStorage/Handles.h"
#include "DataStorage/CommonTypes.h"
#include "MassArchetypeTypes.h"
#include "Misc/TVariant.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FMassEntityManager;
struct FMassProcessingPhaseManager;

namespace UE::Editor::DataStorage
{
	class FEnvironment;

	namespace Legacy
	{
		class FCommandBuffer final
		{
		public:
			explicit FCommandBuffer(FEnvironment& InEnvironment);
	
			/** Returns a pointer to the queued data column, if it exists and hasn't been processed yet. Otherwise null is returned. */
			void* GetQueuedDataColumn(RowHandle Row, const UScriptStruct* ColumnType);
			/** Returns if the column on the provided row is pending processing. */
			bool HasColumn(RowHandle Row, const UScriptStruct* ColumnType) const;
			/** Returns a list of columns pending processing for a row.. */
			void ListColumns(RowHandle Row, ColumnListCallbackRef Callback) const;
			/** Returns a list of columns and their data addresses (can be null for tags) pending processing for a row.. */
			void ListColumns(RowHandle Row, ColumnListWithDataCallbackRef Callback);
			
			/**
			 * @section Queue_* functions create a command that can be stored for later execution in the provided command buffer.
			 * Typically the Execute_* counter part will be called for execution.
			 */
			void Queue_AddColumnCommand(RowHandle Row, const UScriptStruct* ColumnType);
			void* Queue_AddDataColumnCommandUnitialized(RowHandle Row, const UScriptStruct* ColumnType,
				ColumnCopyOrMoveCallback Relocator);
			void Queue_AddColumnsCommand(RowHandle Row, FMassFragmentBitSet FragmentsToAdd, FMassTagBitSet TagsToAdd);
	
			void Queue_RemoveColumnCommand(RowHandle Row, const UScriptStruct* ColumnType);
			void Queue_RemoveColumnsCommand(RowHandle Row, FMassFragmentBitSet FragmentsToRemove, FMassTagBitSet TagsToRemove);

			void Queue_AddRemoveColumnsCommand(TConstArrayView<RowHandle> Rows, FMassElementBitSet ColumnsToAdd, FMassElementBitSet ColumnsToRemove);
	
			/**
			 * @section Execute_* functions directly execute a command with limited validation checks.
			 */
	
			static bool Execute_IsRowAvailable(const FMassEntityManager& MassEntityManager, RowHandle Row);
			static bool Execute_IsRowAssigned(const FMassEntityManager& MassEntityManager, RowHandle Row);
	
			static void Execute_AddColumnCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct* ColumnType);
			static void Execute_AddDataColumnCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct* ColumnType,
				void* Data, ColumnCopyOrMoveCallback Relocator);
			static void Execute_AddSharedColumnCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row, const FConstSharedStruct& SharedColumn);
			static void Execute_RemoveSharedColumnCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct& ColumnType);
			static void Execute_AddColumnsCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row,
				FMassFragmentBitSet FragmentsToAdd, FMassTagBitSet TagsToAdd);
			static void Execute_RemoveColumnCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row, const UScriptStruct* ColumnType);
			static void Execute_RemoveColumnsCommand(
				FMassEntityManager& MassEntityManager, RowHandle Row,
				FMassFragmentBitSet FragmentsToRemove, FMassTagBitSet TagsToRemove);
			static void Execute_AddRemoveColumnsCommand(
				FMassEntityManager& MassEntityManager, TArrayView<RowHandle> Rows,
				FMassElementBitSet ElementsToAdd, FMassElementBitSet ElementsToRemove);
	
			void ProcessCommands();
			void ClearCommands();
	
		private:
			struct FAddColumnCommand
			{
				RowHandle Row;
				TWeakObjectPtr<const UScriptStruct> ColumnType;
			};
			struct FAddDataColumnCommand
			{
				RowHandle Row;
				TWeakObjectPtr<const UScriptStruct> ColumnType;
				ColumnCopyOrMoveCallback Relocator;
				void* Data;
	
				~FAddDataColumnCommand();
			};
			struct FAddColumnsCommand
			{
				RowHandle Row;
				FMassFragmentBitSet FragmentsToAdd;
				FMassTagBitSet TagsToAdd;
			};
			struct FRemoveColumnCommand
			{
				RowHandle Row;
				TWeakObjectPtr<const UScriptStruct> ColumnType;
			};
			struct FRemoveColumnsCommand
			{
				RowHandle Row;
				FMassFragmentBitSet FragmentsToRemove;
				FMassTagBitSet TagsToRemove;
			};
			struct FAddRemoveColumnsCommand
			{
				TArrayView<RowHandle> Rows;
				FMassElementBitSet ElementsToAdd;
				FMassElementBitSet ElementsToRemove;
			};
			using CommandData = TVariant<
				FAddColumnCommand,
				FAddDataColumnCommand,
				FAddColumnsCommand,
				FRemoveColumnCommand,
				FRemoveColumnsCommand,
				FAddRemoveColumnsCommand>;

			template<typename T>
			void AddCommand(T&& Args);
	
			using PendingColumnMappingKey = TPair<RowHandle, TWeakObjectPtr<const UScriptStruct>>;
			TMap<PendingColumnMappingKey, void*> PendingColumns;
			TArray<CommandData> Commands;
			FEnvironment& Environment;
		};
	} // namespace Legacy
} // namespace UE::Editor::DataStorage
