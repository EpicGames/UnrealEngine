// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseCompatibility.h"

#include <cinttypes>
#include <utility>
#include "Algo/Unique.h"
#include "Async/UniqueLock.h"
#include "DataStorage/Debug/Log.h"
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementTransformColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Memento/TypedElementMementoRowTypes.h"
#include "TypedElementDatabase.h"
#include "TypedElementDatabaseEnvironment.h"
#include "TypedElementDataStorageProfilingMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementDatabaseCompatibility)

namespace UE::Editor::DataStorage::Private
{
	bool bIntegrateWithGC = true;
	FAutoConsoleVariableRef CVarIntegrateWithGC(
		TEXT("TEDS.Feature.IntegrateWithGC"),
		bIntegrateWithGC,
		TEXT("Enables actors being removed through the garbage collection instead of requiring explicit removal."));

	bool bUseCommandBuffer = false;
	FAutoConsoleVariableRef CVarUseCommandBufferInCompat(
		TEXT("TEDS.Feature.UseCommandBufferInCompat"),
		bUseCommandBuffer,
		TEXT("Use the command buffer to defer TEDS Compatibility commands."));

	bool bUseDeferredRemovesInCompat = false;
	FAutoConsoleVariableRef CVarUseDeferredRemovesInCompat(
		TEXT("TEDS.Feature.UseDeferredRemovesInCompat"),
		bUseDeferredRemovesInCompat,
		TEXT("If the command buffer in TEDS Compatibility is enabled, setting this to true will cause removes to be queued instead "
			"of immediately executed."));

	bool bOptimizeCommandBuffer = true;
	FAutoConsoleVariableRef CVarOptimizeCommandBufferInCompat(
		TEXT("TEDS.Debug.OptimizeCommandBufferInCompat"),
		bOptimizeCommandBuffer,
		TEXT("If true, the command buffer used in TEDS Compat is optimized, otherwise the optimization phase is skipped."));

	int PrintCompatCommandBuffer = 0;
	FAutoConsoleVariableRef CVarPrintCompatCommandBuffer(
		TEXT("TEDS.Debug.PrintCompatCommandBuffer"),
		PrintCompatCommandBuffer,
		TEXT("If enabled and TEDS Compat uses the command buffer, then the list of pending commands is printed before being execute.\n"
			"0 - disable\n"
			"1 - summarize number of nops\n"
			"2 - include nops"));

	enum class EAsyncUObjectLoadStrategy : int32
	{
		// assert if object that is being asynchronously loaded is added
		// This strategy will cause assertions to occur with -asyncloadingthread enabled
		// as UObjects are added from the GameThread but are set with the AsyncLoading flags
		// indicating that the object is not supposed to be visible for use
		// Strategy has been provided to make it possible to better understand these situations
		Assert = 0,
		// defer registration of async loaded objects into TEDS
		// This strategy will check UObjects for the presence of the AsyncLoading flags
		// and will only make the UObject visible to TEDS if these flags are not present.
		// Registration will be attempted at the next Compatibility tick
		DeferRegistration = 1,
		END
	};
	
	int AsyncLoadedUObjectStrategy = 1;
	FAutoConsoleVariableRef CVarAsyncLoadedObjectStrategy(
		TEXT("TEDS.Feature.AsyncLoadedUObjectStrategy"),
		AsyncLoadedUObjectStrategy,
		TEXT("Strategy used to handle asynchronously loaded objects.\n"
			 "0 - assert if object that is being asynchronously loaded is added\n"
			 "1 - defer registration of async loaded objects into TEDS\n"));

	static EAsyncUObjectLoadStrategy GetAsyncLoadStrategy()
	{
		checkf(AsyncLoadedUObjectStrategy >= 0 && AsyncLoadedUObjectStrategy < static_cast<int32>(EAsyncUObjectLoadStrategy::END), TEXT("Invalid value for TEDS.Feature.AsyncLoadedUObjectStrategy [%d]"), AsyncLoadedUObjectStrategy);
		return static_cast<EAsyncUObjectLoadStrategy>(AsyncLoadedUObjectStrategy);
	}

	static const FName IntegrateWithGCName(TEXT("IntegrateWithGC"));
	static const FName CompatibilityUsesCommandBufferExtensionName(TEXT("CompatiblityUsesCommandBuffer"));
} // namespace UE::Editor::DataStorage::Private

thread_local UE::Editor::DataStorage::FObjectCompatibilityImmediateScope* UEditorDataStorageCompatibility::ImmediateModeTracker = nullptr;

UEditorDataStorageCompatibility::~UEditorDataStorageCompatibility()
{
	UE_LOGF(LogEditorDataStorage, Log, "Destroyed TEDS Compatibility.");
}

void UEditorDataStorageCompatibility::Initialize(UEditorDataStorage* InStorage)
{
	using namespace UE::Editor::DataStorage;

	UE_LOGF(LogEditorDataStorage, Log, "Initializing TEDS Compatibility.");
	checkf(InStorage, TEXT("TEDS Compatibility is being initialized with an invalid storage target."));
	
	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	Storage = InStorage;
	Environment = InStorage->GetEnvironmentPtr();
	QueuedCommands.Initialize(Environment->GetScratchBuffer());
	
	Prepare();

	InStorage->OnUpdate().AddUObject(this, &UEditorDataStorageCompatibility::Tick);

	PreEditChangePropertyDelegateHandle = FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddUObject(this, &UEditorDataStorageCompatibility::OnPrePropertyChanged);
	PostEditChangePropertyDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UEditorDataStorageCompatibility::OnPostEditChangeProperty);
	ObjectModifiedDelegateHandle = FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &UEditorDataStorageCompatibility::OnObjectModified);
	ObjectReinstancedDelegateHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &UEditorDataStorageCompatibility::OnObjectReinstanced);
	
	PostGcUnreachableAnalysisHandle = FCoreUObjectDelegates::PostReachabilityAnalysis.AddUObject(this, &UEditorDataStorageCompatibility::OnPostGcUnreachableAnalysis);

	UE_LOGF(LogEditorDataStorage, Log, "Initialized TEDS Compatibility.");
}

void UEditorDataStorageCompatibility::PostInitialize(UEditorDataStorage* InStorage)
{
}

void UEditorDataStorageCompatibility::Deinitialize()
{
	using namespace UE::Editor::DataStorage;

	UE_LOGF(LogEditorDataStorage, Log, "De-initializing TEDS Compatibility.");

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	FCoreUObjectDelegates::PostReachabilityAnalysis.Remove(PostGcUnreachableAnalysisHandle);

	FCoreUObjectDelegates::OnObjectsReinstanced.Remove(ObjectReinstancedDelegateHandle);
	FCoreUObjectDelegates::OnObjectModified.Remove(ObjectModifiedDelegateHandle);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PostEditChangePropertyDelegateHandle);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.Remove(PreEditChangePropertyDelegateHandle);

	if (Storage)
	{
		Storage->OnUpdate().RemoveAll(this);
	}

	Environment.Reset();
	Storage = nullptr;
	
	ObjectRegistrationFilters.Reset();
	ObjectToRowDialiasers.Reset();
	TypeToTableMap.Reset();
	ObjectAddedCallbackList.Reset();
	PreObjectRemovedCallbackList.Reset();

	StandardUObjectTable = InvalidTableHandle;
	StandardExternalObjectTable = InvalidTableHandle;

	UE_LOGF(LogEditorDataStorage, Log, "De-initialized TEDS Compatibility.");
}

void UEditorDataStorageCompatibility::RegisterRegistrationFilter(ObjectRegistrationFilter Filter)
{
	using namespace UE::Editor::DataStorage;

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);
	ObjectRegistrationFilters.Add(MoveTemp(Filter));
}

void UEditorDataStorageCompatibility::RegisterDealiaserCallback(ObjectToRowDealiaser Dealiaser)
{
	using namespace UE::Editor::DataStorage;

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);
	ObjectToRowDialiasers.Add(MoveTemp(Dealiaser));
}

void UEditorDataStorageCompatibility::RegisterTypeTableAssociation(
	TWeakObjectPtr<UStruct> TypeInfo, UE::Editor::DataStorage::TableHandle Table)
{
	using namespace UE::Editor::DataStorage;
	
	if (Storage)
	{
		checkf(Storage->IsTableValid(Table), TEXT("Attempting to associate invalid table %" PRIu64 " with object type %s"),
			Table, TypeInfo.IsValid() ? *TypeInfo->GetName() : TEXT("<unknown>"));
		if (Private::bUseCommandBuffer)
		{
			QueuedCommands.AddCommand(UE::Editor::DataStorage::FRegisterTypeTableAssociation{ .TypeInfo = TypeInfo, .Table = Table });
		}
		else
		{
			FScopedExclusiveLock Lock(EGlobalLockScope::Public);
			TypeToTableMap.Add(TypeInfo, Table);
		}
	}
}

FDelegateHandle UEditorDataStorageCompatibility::RegisterObjectAddedCallback(UE::Editor::DataStorage::ObjectAddedCallback&& OnObjectAdded)
{
	using namespace UE::Editor::DataStorage;

	FDelegateHandle Handle(FDelegateHandle::GenerateNewHandle);
	if (Private::bUseCommandBuffer)
	{
		QueuedCommands.AddCommand(UE::Editor::DataStorage::FRegisterObjectAddedCallback{ .Callback = MoveTemp(OnObjectAdded), .Handle = Handle });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);
		ObjectAddedCallbackList.Emplace(MoveTemp(OnObjectAdded), Handle);
	}
	return Handle;
}

void UEditorDataStorageCompatibility::UnregisterObjectAddedCallback(FDelegateHandle Handle)
{
	using namespace UE::Editor::DataStorage;

	if (Private::bUseCommandBuffer)
	{
		QueuedCommands.AddCommand(UE::Editor::DataStorage::FUnregisterObjectAddedCallback{ .Handle = Handle });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);
		ObjectAddedCallbackList.RemoveAll(
			[Handle](const TPair<UE::Editor::DataStorage::ObjectAddedCallback, FDelegateHandle>& Element)->bool
			{
				return Element.Value == Handle;
			});
	}
}

FDelegateHandle UEditorDataStorageCompatibility::RegisterObjectRemovedCallback(UE::Editor::DataStorage::ObjectRemovedCallback&& OnObjectAdded)
{
	using namespace UE::Editor::DataStorage;

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	// Since removing object has be immediately executed in some situation, adding the callback can not be delayed through the command buffer.
	FDelegateHandle Handle(FDelegateHandle::GenerateNewHandle);
	PreObjectRemovedCallbackList.Emplace(MoveTemp(OnObjectAdded), Handle);
	return Handle;
}

void UEditorDataStorageCompatibility::UnregisterObjectRemovedCallback(FDelegateHandle Handle)
{
	using namespace UE::Editor::DataStorage;

	// Since removing object has be immediately executed in some situation, adding the callback can not be delayed through the command buffer.
	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	PreObjectRemovedCallbackList.RemoveAll([Handle](const TPair<ObjectRemovedCallback, FDelegateHandle>& Element)->bool
	{
		return Element.Value == Handle;
	});
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::AddCompatibleObjectExplicit(UObject* Object)
{
	// Because AddCompatibleObjectExplicitTransactionable needs a finer grained control over the lock, there's no higher up lock here.

	bool bCanAddObject =
		ensureMsgf(Storage, TEXT("Trying to add a UObject to the editor's Data Storage before it's available.")) &&
		Object != nullptr &&
		ShouldAddObject(Object);
	return bCanAddObject ? AddCompatibleObjectExplicitTransactionable<true>(Object) : UE::Editor::DataStorage::InvalidRowHandle;
}


UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::AddCompatibleObjectExplicit(void* Object, TWeakObjectPtr<const UScriptStruct> TypeInfo)
{
	using namespace UE::Editor::DataStorage;

	if (Object == nullptr)
	{
		return InvalidRowHandle;
	}
	
	checkf(Storage, TEXT("Trying to add an object to the editor's Data Storage before it's available."));
	FScopedExclusiveLock Lock(EGlobalLockScope::Public);
	
	RowHandle Result = FindRowWithCompatibleObjectExplicit(Object);
	if (!Storage->IsRowAvailable(Result))
	{
		if (const UScriptStruct* TypeInfoPtr = TypeInfo.Get(true))
		{
			Result = Storage->ReserveRow();
			Storage->MapRow(ICompatibilityProvider::ObjectMappingDomain, FMapKey(Object), Result);
			if (ImmediateModeTracker)
			{
				FObjectCompatibilityImmediateScopeWriteAccess::RegisterObject(*ImmediateModeTracker, Object, TypeInfo.Get(true), Result);
			}
			else if (Private::bUseCommandBuffer)
			{
				QueuedCommands.AddCommand(FAddCompatibleExternalObject{ .Object = Object, .TypeInfo = TypeInfo, .Row = Result });
			}
			else
			{
				ExternalObjectsPendingRegistration.Add(Result, ExternalObjectRegistration{ .Object = Object, .TypeInfo = TypeInfo });
			}
		}
	}
	return Result;
}

void UEditorDataStorageCompatibility::RemoveCompatibleObjectExplicit(UObject* Object)
{
	RemoveCompatibleObjectExplicitTransactionable<true>(Object);
}

void UEditorDataStorageCompatibility::RemoveCompatibleObjectExplicit(void* Object)
{
	using namespace UE::Editor::DataStorage;

	checkf(Storage, TEXT("Removing compatible objects is not supported before Typed Element's Database compatibility manager has been initialized."));

	if (Private::bUseCommandBuffer && Private::bUseDeferredRemovesInCompat)
	{
		QueuedCommands.AddCommand(FRemoveCompatibleExternalObject{ .Object = Object });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		RowHandle Row = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Object));
		if (Storage->IsRowAvailable(Row))
		{
			const FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = Storage->GetColumn<FTypedElementScriptStructTypeInfoColumn>(Row);
			if (Storage->IsRowAssigned(Row) && ensureMsgf(TypeInfoColumn, TEXT("Missing type information for removed void* object at ptr 0x%p"), Object))
			{
				TriggerOnPreObjectRemoved(Object, TypeInfoColumn->TypeInfo.Get(), Row);
			}
			Storage->RemoveRow(Row);
		}
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::FindRowWithCompatibleObjectExplicit(const UObject* Object) const
{
	using namespace UE::Editor::DataStorage;

	if (Object && Storage && Storage->IsAvailable())
	{
		FScopedSharedLock Lock(EGlobalLockScope::Public);

		RowHandle Row = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Object));
		return Storage->IsRowAvailable(Row) ? Row : DealiasObject(Object);
	}
	return InvalidRowHandle;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::FindRowWithCompatibleObjectExplicit(const void* Object) const
{
	// Thread safety is only needed by FindIndexedRow which internally takes care of it.

	using namespace UE::Editor::DataStorage;

	return (Object && Storage && Storage->IsAvailable()) 
		? Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Object)) 
		: InvalidRowHandle;
}

bool UEditorDataStorageCompatibility::SupportsExtension(FName Extension) const
{
	// No thread safety needed.

	using namespace UE::Editor::DataStorage;

	if (Extension == Private::IntegrateWithGCName)
	{
		return Private::bIntegrateWithGC;
	}
	else if (Extension == Private::CompatibilityUsesCommandBufferExtensionName)
	{
		return Private::bUseCommandBuffer;
	}
	else
	{
		return false;
	}
}

void UEditorDataStorageCompatibility::ListExtensions(TFunctionRef<void(FName)> Callback) const
{
	// No thread safety needed.

	using namespace UE::Editor::DataStorage;

	if (Private::bIntegrateWithGC)
	{
		Callback(Private::IntegrateWithGCName);
	}
	if (Private::bUseCommandBuffer)
	{
		Callback(Private::CompatibilityUsesCommandBufferExtensionName);
	}
}

void UEditorDataStorageCompatibility::EnterImmediateMode(UE::Editor::DataStorage::FObjectCompatibilityImmediateScope& Scope)
{
	FObjectCompatibilityImmediateScopeWriteAccess::SetPrevious(Scope, ImmediateModeTracker);
	ImmediateModeTracker = &Scope;
}

void UEditorDataStorageCompatibility::LeaveImmediateMode(UE::Editor::DataStorage::FObjectCompatibilityImmediateScope& Scope)
{
	using namespace UE::Editor::DataStorage;

	FObjectCompatibilityImmediateScope* CurrentTracker = ImmediateModeTracker;
	checkf(CurrentTracker, 
		TEXT("Attempting to leave immediate mode in the editor data storage's compatibility layer, but immediate was never entered."));
	checkf(CurrentTracker == &Scope, TEXT("Attempting to leave immediate mode in the editor data storage's compatibility ")
		TEXT("layer, but the active scope doesn't match the provided scope."));

	if (Scope.GetObjectCount() > 0)
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		FObjectCompatibilityImmediateScopeWriteAccess::ListUObjects(*CurrentTracker,
			[this](RowHandle Row, UObject* Object, const UClass* TypeInfo)
			{
				TableHandle Table = FindBestMatchingTable(TypeInfo);
				checkf(Table != InvalidTableHandle,
					TEXT("Unable to find a matching table in the compatibility layer for the editor's data storage for type '%s'"),
					*Object->GetClass()->GetName());

				Storage->AddRow(Row, Table, [this, Object](RowHandle Row)
					{
						FCommandProcessor::SetupRow(*Storage, *this, Row, Object);
					});
			});

		FObjectCompatibilityImmediateScopeWriteAccess::ListExternalObjects(*CurrentTracker,
			[this](RowHandle Row, void* Object, const UScriptStruct* TypeInfo)
			{
				TableHandle Table = FindBestMatchingTable(TypeInfo);
				Table = (Table != InvalidTableHandle) ? Table : StandardExternalObjectTable;
				Storage->AddRow(Row, Table, [this, Object, TypeInfo](RowHandle Row)
					{
						FCommandProcessor::SetupRow(*Storage, *this, Row, Object, TypeInfo);
					});
			});
	}

	ImmediateModeTracker = FObjectCompatibilityImmediateScopeWriteAccess::GetPrevious(*CurrentTracker);
}

void UEditorDataStorageCompatibility::Prepare()
{
	// Thread-safe as this is only called from a function that has an exclusive lock.
	CreateStandardArchetypes();
	RegisterTypeInformationQueries();
}

void UEditorDataStorageCompatibility::CreateStandardArchetypes()
{
	using namespace UE::Editor::DataStorage::Queries;

	// Thread-safe as this is only called from a function that has an exclusive lock.
	using namespace UE::Editor::DataStorage;
	StandardUObjectTable = Storage->RegisterTable<
			FTypedElementUObjectColumn, FTypedElementUObjectIdColumn, FUObjectIdNameColumn,
			FTypedElementClassTypeInfoColumn, FTypedElementSyncFromWorldTag>
		(FName("Editor_StandardUObjectTable"));

	Storage->RegisterTableForeignKey(StandardUObjectTable, FName("Serialization"), 
			[](TConstQueryContext<RowBatchInfo> Context, TResult<FForeignKey>& Result, TConstBatch<FTypedElementUObjectColumn> Objects)
			{
				Context.ForEachRow([&Result](RowHandle Row, const FTypedElementUObjectColumn& Object)
					{
						const UObject* ObjectPtr = Object.Object.Get(true);
						Result.Add(Row, ObjectPtr
							? FForeignKey(ObjectPtr->GetPathName())
							: FForeignKey(FKeyIssue(TEXT("Row doesn't contain a valid UObject.")))
						);
					}, Objects);
			});
	Storage->RegisterTableForeignKey(StandardUObjectTable, FName("Memento"),
			[](TConstQueryContext<RowBatchInfo> Context, TResult<FForeignKey>& Result, TConstBatch<FTypedElementUObjectColumn> Objects)
			{
				Context.ForEachRow([&Result](RowHandle Row, const FTypedElementUObjectColumn& Object)
					{
						const UObject* ObjectPtr = Object.Object.Get(true);
						Result.Add(Row, ObjectPtr
							? FForeignKey(ObjectPtr)
							: FForeignKey(FKeyIssue(TEXT("Row doesn't contain a valid UObject.")))
						);
					}, Objects);
			});
	RegisterTypeTableAssociation(UObject::StaticClass(), StandardUObjectTable);

	StandardExternalObjectTable = Storage->RegisterTable<
			FTypedElementExternalObjectColumn, FTypedElementScriptStructTypeInfoColumn,
			FTypedElementSyncFromWorldTag>
		(FName("Editor_StandardExternalObjectTable"));
}

void UEditorDataStorageCompatibility::RegisterTypeInformationQueries()
{
	// Thread-safe as this is only called from a function that has an exclusive lock.

	using namespace UE::Editor::DataStorage::Queries;

	ClassTypeInfoQuery = Storage->RegisterQuery(
		Select()
			.ReadWrite<FTypedElementClassTypeInfoColumn>()
		.Compile());
	
	ScriptStructTypeInfoQuery = Storage->RegisterQuery(
		Select()
			.ReadWrite<FTypedElementScriptStructTypeInfoColumn>()
		.Compile());

	UObjectQuery = Storage->RegisterQuery(
		Select()
			.ReadWrite<FTypedElementUObjectIdColumn>()
		.Compile());
}

bool UEditorDataStorageCompatibility::ShouldAddObject(const UObject* Object) const
{
	using namespace UE::Editor::DataStorage;
	
	FScopedSharedLock Lock(EGlobalLockScope::Public);

	bool Include = true;
	if (!Storage->IsRowAvailable(Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Object))))
	{
		const ObjectRegistrationFilter* Filter = ObjectRegistrationFilters.GetData();
		const ObjectRegistrationFilter* FilterEnd = Filter + ObjectRegistrationFilters.Num();
		for (; Include && Filter != FilterEnd; ++Filter)
		{
			Include = (*Filter)(*this, Object);
		}
	}
	return Include;
}

UE::Editor::DataStorage::TableHandle UEditorDataStorageCompatibility::FindBestMatchingTable(const UStruct* TypeInfo) const
{
	using namespace UE::Editor::DataStorage;

	FScopedSharedLock Lock(EGlobalLockScope::Public);

	while (TypeInfo)
	{
		if (const TableHandle* Table = TypeToTableMap.Find(TypeInfo))
		{
			checkf(Storage && Storage->IsTableValid(*Table), TEXT("A previously registered table (%" PRIu64 ") has become invalid."), *Table);
			return *Table;
		}
		TypeInfo = TypeInfo->GetSuperStruct();
	}

	return InvalidTableHandle;
}

template<bool bEnableTransactions>
UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::AddCompatibleObjectExplicitTransactionable(UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	RowHandle Result = FindRowWithCompatibleObjectExplicit(Object);
	if (!Storage->IsRowAvailable(Result))
	{	
		if (ImmediateModeTracker)
		{
			checkf(
				!(Object->GetInternalFlags() & EInternalObjectFlags_AsyncLoading),
				TEXT("Cannot add object '%s' as it's being asynchronously loaded to the editor's data storage while in immediate mode."),
				*Object->GetName());

			Result = Storage->ReserveRow();
			Storage->MapRow(ICompatibilityProvider::ObjectMappingDomain, FMapKey(Object), Result);
			FObjectCompatibilityImmediateScopeWriteAccess::RegisterObject(*ImmediateModeTracker, Object, nullptr, Result);
		}
		else
		{
			if (Private::GetAsyncLoadStrategy() == Private::EAsyncUObjectLoadStrategy::Assert)
			{
				EInternalObjectFlags Flags = Object->GetInternalFlags();
				ensureMsgf(
					!(Flags & EInternalObjectFlags_AsyncLoading),
					TEXT("Cannot add object that is being asynchronously loaded to TEDS"));
				return InvalidRowHandle;
			}
			Result = Storage->ReserveRow();
			Storage->MapRow(ICompatibilityProvider::ObjectMappingDomain, FMapKey(Object), Result);

			if (Private::bUseCommandBuffer)
			{
				QueuedCommands.AddCommand(FAddCompatibleUObject{ .Object = Object, .Row = Result });
			}
			else
			{
				UObjectsPendingRegistration.Add(Result, Object);
			}
		}
		
		if constexpr (bEnableTransactions)
		{
			if (IsInGameThread() && GUndo)
			{
				GUndo->StoreUndo(this, MakeUnique<FRegistrationCommandChange>(this, Object));
			}
		}
	}
	return Result;
}

template<bool bEnableTransactions>
void UEditorDataStorageCompatibility::RemoveCompatibleObjectExplicitTransactionable(const UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	checkf(Storage,
		TEXT("Removing compatible objects is not supported before Typed Element's Database compatibility manager has been initialized."));
	
	if constexpr (!bEnableTransactions)
	{
		if (Private::bUseCommandBuffer && Private::bUseDeferredRemovesInCompat)
		{
			// There's no need for an transaction recording so the full operation can be done as part of the commands processing.
			QueuedCommands.AddCommand(FRemoveCompatibleUObject{ .Object = Object });
			return;
		}
	}

	// Do not lock while both buffered and non-buffered ways are still available. An exclusive lock is required here for the
	// non-buffered to reduce the additional locks/unlocks while the buffered version doesn't need any locking beyond the
	// shared lock FindIndexedRow does. Not adding an exclusive here means some additional lock/unlocking but doesn't make
	// the code thread unsafe.
	RowHandle Row = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Object));
	if (Storage->IsRowAvailable(Row))
	{
		RemoveCompatibleObjectExplicitTransactionable<bEnableTransactions>(Object, Row);
	}
}

template<bool bEnableTransactions>
void UEditorDataStorageCompatibility::RemoveCompatibleObjectExplicitTransactionable(
	const UObject* Object, UE::Editor::DataStorage::RowHandle ObjectRow)
{
	using namespace UE::Editor::DataStorage;

	checkf(Storage,
		TEXT("Removing compatible objects is not supported before Typed Element's Database compatibility manager has been initialized."));

	if (Private::bUseCommandBuffer && Private::bUseDeferredRemovesInCompat)
	{
		if constexpr (bEnableTransactions)
		{
			if (IsInGameThread() && GUndo)
			{
				GUndo->StoreUndo(this, MakeUnique<FDeregistrationCommandChange>(this, const_cast<UObject*>(Object)));
			}
		}
		QueuedCommands.AddCommand(FRemoveCompatibleUObject{ .Object = Object, .ObjectRow = ObjectRow });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		const FTypedElementClassTypeInfoColumn* TypeInfoColumn = Storage->GetColumn<FTypedElementClassTypeInfoColumn>(ObjectRow);
		if (Storage->IsRowAssigned(ObjectRow) &&
			ensureMsgf(TypeInfoColumn, TEXT("Missing type information for removed UObject at ptr 0x%p [%s]"), Object, *Object->GetName()))
		{
			TriggerOnPreObjectRemoved(Object, TypeInfoColumn->TypeInfo.Get(), ObjectRow);

			if constexpr (bEnableTransactions)
			{
				if (IsInGameThread() && GUndo)
				{
					GUndo->StoreUndo(this, MakeUnique<FDeregistrationCommandChange>(this, const_cast<UObject*>(Object)));
				}
			}
		}

		Storage->RemoveRow(ObjectRow);
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageCompatibility::DealiasObject(const UObject* Object) const
{
	// Thread safe because it's only called from functions that already lock.

	for (const ObjectToRowDealiaser& Dealiaser : ObjectToRowDialiasers)
	{
		if (UE::Editor::DataStorage::RowHandle Row = Dealiaser(*this, Object); Storage->IsRowAvailable(Row))
		{
			return Row;
		}
	}
	return UE::Editor::DataStorage::InvalidRowHandle;
}

void UEditorDataStorageCompatibility::Tick()
{
	using namespace UE::Editor::DataStorage;
	
	TEDS_EVENT_SCOPE(TEXT("Compatibility Tick"))
	
	FScopedExclusiveLock Lock(EGlobalLockScope::Public);

	// Delay processing until the required systems are available by not clearing any lists or doing any work.
	if (Storage && Storage->IsAvailable())
	{
		if (Private::bUseCommandBuffer)
		{
			TickPendingCommands();
		}
		else
		{
			PendingTypeInformationUpdate.Process(*this);
			TickPendingUObjectRegistration();
			TickPendingExternalObjectRegistration();
			TickObjectSync();
		}
	}
}



//
// FPendingTypeInformatUpdate
// 

UEditorDataStorageCompatibility::FPendingTypeInformationUpdate::FPendingTypeInformationUpdate()
	: PendingTypeInformationUpdatesActive(&PendingTypeInformationUpdates[0])
	, PendingTypeInformationUpdatesSwapped(&PendingTypeInformationUpdates[1])
{}

void UEditorDataStorageCompatibility::FPendingTypeInformationUpdate::AddTypeInformation(const TMap<UObject*, UObject*>& ReplacedObjects)
{
	UE::TUniqueLock Lock(Safeguard);

	for (TMap<UObject*, UObject*>::TConstIterator It = ReplacedObjects.CreateConstIterator(); It; ++It)
	{
		if (It->Key->IsA<UStruct>())
		{
			PendingTypeInformationUpdatesActive->Add(*It);
			bHasPendingUpdate = true;
		}
	}
}

void UEditorDataStorageCompatibility::FPendingTypeInformationUpdate::Process(UEditorDataStorageCompatibility& Compatibility)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (bHasPendingUpdate)
	{
		// Swap to release the lock as soon as possible.
		{
			UE::TUniqueLock Lock(Safeguard);
			std::swap(PendingTypeInformationUpdatesActive, PendingTypeInformationUpdatesSwapped);
			bHasPendingUpdate = false;
		}

		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		for (TypeToTableMapType::TIterator It = Compatibility.TypeToTableMap.CreateIterator(); It; ++It)
		{
			if (TOptional<TWeakObjectPtr<UObject>> NewObject = ProcessResolveTypeRecursively(It.Key()); NewObject.IsSet())
			{
				UpdatedTypeInfoScratchBuffer.Emplace(Cast<UStruct>(*NewObject), It.Value());
				It.RemoveCurrent();
			}
		}
		for (TPair<TWeakObjectPtr<UStruct>, TableHandle>& UpdatedEntry : UpdatedTypeInfoScratchBuffer)
		{
			checkf(UpdatedEntry.Key.IsValid(),
				TEXT("Type info column in data storage has been re-instanced to an object without type information"));
			Compatibility.TypeToTableMap.Add(UpdatedEntry);
		}
		UpdatedTypeInfoScratchBuffer.Reset();

		Compatibility.Storage->RunQuery(Compatibility.ClassTypeInfoQuery, CreateDirectQueryCallbackBinding(
			[this](IDirectQueryContext& Context, FTypedElementClassTypeInfoColumn& Type)
			{
				if (TOptional<TWeakObjectPtr<UObject>> NewObject = 
					// Using reintpret_Cast as TWeakObjectPtr<const UClass> and TWeakObjectPtr<UClass> are considered separate types by 
					// the compiler.
					ProcessResolveTypeRecursively(reinterpret_cast<TWeakObjectPtr<UClass>&>(Type.TypeInfo)); NewObject.IsSet())
				{
					Type.TypeInfo = Cast<UClass>(*NewObject);
					checkf(Type.TypeInfo.IsValid(),
						TEXT("Type info column in data storage has been re-instanced to an object without class type information"));
				}
			}));
		Compatibility.Storage->RunQuery(Compatibility.ScriptStructTypeInfoQuery, CreateDirectQueryCallbackBinding(
			[this](IDirectQueryContext& Context, FTypedElementScriptStructTypeInfoColumn& Type)
			{
				if (TOptional<TWeakObjectPtr<UObject>> NewObject = 
					ProcessResolveTypeRecursively(reinterpret_cast<TWeakObjectPtr<const UScriptStruct>&>(Type.TypeInfo)); NewObject.IsSet())
				{
					Type.TypeInfo = Cast<const UScriptStruct>(*NewObject);
					checkf(Type.TypeInfo.IsValid(),
						TEXT("Type info column in data storage has been re-instanced to an object without struct type information"));
				}
			}));

		Compatibility.ExternalObjectsPendingRegistration.ForEachAddress(
			[this](ExternalObjectRegistration& Entry)
			{
				if (TOptional<TWeakObjectPtr<UObject>> NewObject = ProcessResolveTypeRecursively(Entry.TypeInfo); NewObject.IsSet())
				{
					Entry.TypeInfo = Cast<const UScriptStruct>(*NewObject);
					checkf(Entry.TypeInfo.Get(),
						TEXT("Type info pending processing in data storage has been re-instanced to an object without struct type information"));
				}
			});

		PendingTypeInformationUpdatesSwapped->Reset();
	}
}

TOptional<TWeakObjectPtr<UObject>> UEditorDataStorageCompatibility::FPendingTypeInformationUpdate::ProcessResolveTypeRecursively(
	const TWeakObjectPtr<const UObject>& Target)
{
	// Thread-safety guaranteed because this is a private function that only gets called from functions that called inside a mutex.

	if (const TWeakObjectPtr<UObject>* NewObject = PendingTypeInformationUpdatesSwapped->Find(Target))
	{
		TWeakObjectPtr<UObject> LastNewObject = *NewObject;
		while (const TWeakObjectPtr<UObject>* NextNewObject = PendingTypeInformationUpdatesSwapped->Find(LastNewObject))
		{
			LastNewObject = *NextNewObject;
		}
		return LastNewObject;
	}
	return TOptional<TWeakObjectPtr<UObject>>();
}




//
// PendingRegistration
//

template<typename AddressType>
void UEditorDataStorageCompatibility::PendingRegistration<AddressType>::Add(UE::Editor::DataStorage::RowHandle ReservedRowHandle, AddressType Address)
{
	// Thread-safe as it's only called from functions that already lock using.
	Entries.Emplace(FEntry{ .Address = Address, .Row = ReservedRowHandle });
}

template<typename AddressType>
bool UEditorDataStorageCompatibility::PendingRegistration<AddressType>::IsEmpty() const
{
	// Thread-safe as it's only called from functions that already lock using.
	return Entries.IsEmpty();
}

template<typename AddressType>
int32 UEditorDataStorageCompatibility::PendingRegistration<AddressType>::Num() const
{
	// Thread-safe as it's only called from functions that already lock using.
	return Entries.Num();
}

template<typename AddressType>
void UEditorDataStorageCompatibility::PendingRegistration<AddressType>::ForEachAddress(const TFunctionRef<void(AddressType&)>& Callback)
{
	// Thread-safe as it's only called from functions that already lock using.
	for (FEntry& Entry : Entries)
	{
		Callback(Entry.Address);
	}
}

template<typename AddressType>
void UEditorDataStorageCompatibility::PendingRegistration<AddressType>::ProcessEntries(UE::Editor::DataStorage::ICoreProvider& StorageInterface,
	UEditorDataStorageCompatibility& Compatibility, const TFunctionRef<void(UE::Editor::DataStorage::RowHandle, const AddressType&)>& SetupRowCallback)
{
	// Thread-safe as it's only called from functions that already lock using.
	using namespace UE::Editor::DataStorage;

	// Double buffer because new entries can be added during processing of the queue. This can happen if there are observers that add
	// new UObjects or external objects in response to one being registered.
	Swap(Entries, ProcessingEntries);

	// Start by removing any entries that are no longer valid.
	for (auto It = ProcessingEntries.CreateIterator(); It; ++It)
	{
		bool bIsValid = StorageInterface.IsRowAvailable(It->Row);
		if constexpr (std::is_same_v<AddressType, TWeakObjectPtr<UObject>>)
		{
			bIsValid = bIsValid && It->Address.IsValid();
			if (bIsValid && Private::GetAsyncLoadStrategy() == Private::EAsyncUObjectLoadStrategy::Assert)
			{
				// Technically should never ensure due to this check being made when objects are added to the list
				// but left here for completeness
				ensureMsgf(
					!It->Address->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading),
					TEXT("UObject that is currently being asynchronously loaded has been queued to be registered with TEDS. Cannot process this entry"));
				bIsValid = false;
			}
		}
		else if constexpr (std::is_same_v<AddressType, ExternalObjectRegistration>)
		{
			bIsValid = bIsValid && (It->Address.Object != nullptr);
		}
		else
		{
			static_assert(sizeof(AddressType) == 0, "Unsupported type for pending object registration in data storage compatibility.");
		}

		if (!bIsValid)
		{
			It.RemoveCurrentSwap();
		}
	}

	// Check for empty here are the above code could potentially leave an empty array behind. This would result in break the assumption
	// that there is at least one entry later in this function.
	if (!ProcessingEntries.IsEmpty())
	{
		// Next resolve the required table handles.
		for (FEntry& Entry : ProcessingEntries)
		{
			if constexpr (std::is_same_v<AddressType, TWeakObjectPtr<UObject>>)
			{
				switch (Private::GetAsyncLoadStrategy())
				{
					case Private::EAsyncUObjectLoadStrategy::Assert:
					{
						Entry.Table = Compatibility.FindBestMatchingTable(Entry.Address->GetClass());
						checkf(Entry.Table != InvalidTableHandle, 
							TEXT("The data storage could not find any matching tables for object of type '%s'. "
							"This can mean that the object doesn't derive from UObject or that a table for UObject is no longer registered."), 
							*Entry.Address->GetClass()->GetFName().ToString());
						checkf(StorageInterface.IsTableValid(Entry.Table),
							TEXT("Retrieved table (%" PRIu64 ") for object %s of type %s is invalid."), Entry.Table,
							*Entry.Address->GetName(), *Entry.Address->GetClass()->GetFName().ToString());
						break;
					}
					case Private::EAsyncUObjectLoadStrategy::DeferRegistration:
					{
						if (Entry.Address->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
						{
							// Do nothing, we will check the entry again next frame
							Entry.Table = InvalidTableHandle;
						}
						else
						{
							Entry.Table = Compatibility.FindBestMatchingTable(Entry.Address->GetClass());
							checkf(Entry.Table != InvalidTableHandle, 
								TEXT("The data storage could not find any matching tables for object of type '%s'. "
								"This can mean that the object doesn't derive from UObject or that a table for UObject is no longer registered."), 
								*Entry.Address->GetClass()->GetFName().ToString());
							checkf(StorageInterface.IsTableValid(Entry.Table),
								TEXT("Retrieved table (%" PRIu64 ") for object %s of type %s is invalid."), Entry.Table,
								*Entry.Address->GetName(), *Entry.Address->GetClass()->GetFName().ToString());
						}
						break;
					}
					default:
					{
						checkf(false, TEXT("Unhandled async uobject load strategy"));
					}
				}
			}
			else if constexpr (std::is_same_v<AddressType, ExternalObjectRegistration>)
			{
				Entry.Table = Compatibility.FindBestMatchingTable(Entry.Address.TypeInfo.Get());
				Entry.Table = (Entry.Table != InvalidTableHandle) ? Entry.Table : Compatibility.StandardExternalObjectTable;
			}
			else
			{
				static_assert(sizeof(AddressType) == 0, "Unsupported type for pending object registration in data storage compatibility.");
			}
		}

		// Next sort them by table then by row handle to allow batch insertion.
		ProcessingEntries.Sort(
			[](const FEntry& Lhs, const FEntry& Rhs)
			{
				if (Lhs.Table < Rhs.Table)
				{
					return true;
				}
				else if (Lhs.Table > Rhs.Table)
				{
					return false;
				}
				else
				{
					return Lhs.Row < Rhs.Row;
				}
			});

		// Batch up the entries and add them to the storage.
		FEntry* Current = ProcessingEntries.GetData();
		FEntry* End = Current + ProcessingEntries.Num();
		
		FEntry* TableFront = Current;

		// It is expected that InvalidTableHandle should always be sorted at the end
		// This assertion ensures that we catch a change in this behavior should the constant
		// be modified. 
		static_assert(InvalidTableHandle == TNumericLimits<TableHandle>::Max()); 
		if (ProcessingEntries[0].Table == InvalidTableHandle)
		{
			Entries.Append(ProcessingEntries);
			ProcessingEntries.Reset();
			return;
		}
		TableHandle CurrentTable = ProcessingEntries[0].Table;
		checkf(CurrentTable == InvalidTableHandle || StorageInterface.IsTableValid(CurrentTable),
			TEXT("The table (%" PRIu64 ") starting the processing queue has an unsupported value."), CurrentTable);

		Compatibility.RowScratchBuffer.Reset();
		
		for (; Current != End; ++Current)
		{
			if (Current->Table != CurrentTable)
			{
				StorageInterface.BatchAddRow(CurrentTable, Compatibility.RowScratchBuffer,
					[&SetupRowCallback, &TableFront](RowHandle Row)
					{
						SetupRowCallback(Row, TableFront->Address);
						++TableFront;
					});

				CurrentTable = Current->Table;
				checkf(CurrentTable == InvalidTableHandle || StorageInterface.IsTableValid(CurrentTable),
					TEXT("The next table (%" PRIu64 ") in the processing queue is invalid."), CurrentTable);
				Compatibility.RowScratchBuffer.Reset();

				if (CurrentTable == InvalidTableHandle)
				{
					break;
				}
			}
			Compatibility.RowScratchBuffer.Add(Current->Row);
		}
		if (CurrentTable != InvalidTableHandle)
		{
			StorageInterface.BatchAddRow(CurrentTable, Compatibility.RowScratchBuffer,
				[&SetupRowCallback, &TableFront](RowHandle Row)
				{
					SetupRowCallback(Row, TableFront->Address);
					++TableFront;
				});
			// The last entry was able to be added to TEDS
			// therefore none remain in the list.
		}
		else
		{
			// There are some objects that couldn't be added this frame. Take that list
			// and add it to the active entries list so they get picked up next frame.
			const int32 RemainingEntries = static_cast<int32>(End - Current);
			Entries.Reserve(Entries.Num() + RemainingEntries);
			for (; Current != End; ++Current)
			{
				Entries.Add(*Current);
			}
		}
		ProcessingEntries.Reset();
		Compatibility.RowScratchBuffer.Reset();
	}
}

void UEditorDataStorageCompatibility::TickPendingCommands()
{
	using namespace UE::Editor::DataStorage;

	// Thread safe because it's only called from functions that already lock.
	SIZE_T CommandCount = QueuedCommands.Collect(PendingCommands);

	// First see if there's anything that needs to be patched to avoid any of the later steps using stale data.
	if (FPatchData::IsPatchingRequired(PendingCommands))
	{
		TEDS_EVENT_SCOPE(TEXT("Compatibility Tick - Patching"))
		FPatchData::RunPatch(PendingCommands, *this, Environment->GetScratchBuffer());
		CommandCount = PendingCommands.GetTotalCommandCount();
	}

	if (CommandCount > 0)
	{
		TEDS_EVENT_SCOPE(TEXT("Compatibility Tick - Preparation"))
		// Prepare data in the commands. Commands that can't or don't need to be executed will be nop-ed out.
		FPrepareCommands::RunPreparation(*Storage, *this, PendingCommands);
		CommandCount = PendingCommands.GetTotalCommandCount();
	}

	if (CommandCount > 0)
	{
		if (Private::bOptimizeCommandBuffer)
		{
			TEDS_EVENT_SCOPE(TEXT("Compatibility Tick - Optimization"))
			FSorter::SortCommands(PendingCommands);
			FCommandOptimizer::Run(PendingCommands, Environment->GetScratchBuffer());
		}

		if (Private::PrintCompatCommandBuffer > 0 )
		{
			TEDS_EVENT_SCOPE(TEXT("Compatibility Tick - Logging"))
			FString CommandsAsString = FRecordCommands::PrintToString(PendingCommands, Private::PrintCompatCommandBuffer == 2);
			UE_LOGF(LogEditorDataStorage, Log, "Pending Commands:\n%ls%u Nops",
				*CommandsAsString, PendingCommands.GetCommandCount<FNopCommand>());
		}

		TEDS_EVENT_SCOPE(TEXT("Compatibility Tick - Processing"))
		PendingCommands.Process(FCommandProcessor(*Storage, *this));
	}
	PendingCommands.Reset();
}

void UEditorDataStorageCompatibility::TickPendingUObjectRegistration()
{
	using namespace UE::Editor::DataStorage;

	// Thread safe because it's only called from functions that already lock.
	
	if (Storage && !UObjectsPendingRegistration.IsEmpty())
	{
		checkf(Storage->IsTableValid(StandardUObjectTable), TEXT("Default UObject table has become invalid."));
		UObjectsPendingRegistration.ProcessEntries(*Storage, *this,
			[this](UE::Editor::DataStorage::RowHandle Row, const TWeakObjectPtr<UObject>& Object)
			{
				FCommandProcessor::SetupRow(*Storage, *this, Row, Object.Get());
			});
	}
}

void UEditorDataStorageCompatibility::TickPendingExternalObjectRegistration()
{
	using namespace UE::Editor::DataStorage;

	// Thread safe because it's only called from functions that already lock.

	if (Storage && !ExternalObjectsPendingRegistration.IsEmpty())
	{
		checkf(Storage->IsTableValid(StandardExternalObjectTable), TEXT("Default external object table has become invalid."));
		ExternalObjectsPendingRegistration.ProcessEntries(*Storage, *this,
			[this](UE::Editor::DataStorage::RowHandle Row, const ExternalObjectRegistration& Object)
			{
				FCommandProcessor::SetupRow(*Storage, *this, Row, Object.Object, Object.TypeInfo);
			});
	}
}

void UEditorDataStorageCompatibility::TickObjectSync()
{
	using namespace UE::Editor::DataStorage;
	
	// Thread safe because it's only called from functions that already lock.

	if (!ObjectsNeedingSyncTags.IsEmpty())
	{
		TEDS_EVENT_SCOPE(TEXT("Process ObjectsNeedingSyncTags"));
		
		using ColumnArray = TArray<const UScriptStruct*, TInlineAllocator<MaxExpectedTagsForObjectSync>>;
		ColumnArray ColumnsToAdd;
		ColumnArray ColumnsToRemove;
		ColumnArray* ColumnsToAddPtr = &ColumnsToAdd;
		ColumnArray* ColumnsToRemovePtr = &ColumnsToRemove;
		bool bHasUpdates = false;
		for (TPair<ObjectsNeedingSyncTagsMapKey, ObjectsNeedingSyncTagsMapValue>& ObjectToSync : ObjectsNeedingSyncTags)
		{
			const RowHandle Row = FindRowWithCompatibleObject(ObjectToSync.Key);
			if (Storage->IsRowAvailable(Row))
			{
				for (FSyncTagInfo& Column : ObjectToSync.Value)
				{
					if (Column.ColumnType.IsValid())
					{
						ColumnArray* TargetColumn = Column.bAddColumn ? ColumnsToAddPtr : ColumnsToRemovePtr;
						TargetColumn->Add(Column.ColumnType.Get());
						bHasUpdates = true;
					}
				}
				if (bHasUpdates)
				{
					Storage->AddRemoveColumns(Row, ColumnsToAdd, ColumnsToRemove);
				}
			}
			bHasUpdates = false;
			ColumnsToAdd.Reset();
			ColumnsToRemove.Reset();
		}

		ObjectsNeedingSyncTags.Reset();
	}
}

void UEditorDataStorageCompatibility::OnPrePropertyChanged(UObject* Object, const FEditPropertyChain& PropertyChain)
{
	using namespace UE::Editor::DataStorage;

	if (Private::bUseCommandBuffer)
	{
		QueuedCommands.AddCommand(FAddInteractiveSyncFromWorldTag{ .Target = Object });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);
		ObjectsNeedingSyncTags.FindOrAdd(Object).AddUnique(
			FSyncTagInfo{ .ColumnType = FTypedElementSyncFromWorldInteractiveTag::StaticStruct(), .bAddColumn = true });
	}
}

void UEditorDataStorageCompatibility::OnPostEditChangeProperty(
	UObject* Object,
	FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace UE::Editor::DataStorage;

	if (Private::bUseCommandBuffer)
	{
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			QueuedCommands.AddCommand(FRemoveInteractiveSyncFromWorldTag{ .Target = Object });
		}
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		// Determining the object is being tracked in the database can't be done safely as it may be queued for addition.
		// It would also add a small bit of performance overhead as access the lookup table can be done faster as a
		// batch operation during the tick step.
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			ObjectsNeedingSyncTagsMapValue& SyncValue = ObjectsNeedingSyncTags.FindOrAdd(Object);
			SyncValue.AddUnique(FSyncTagInfo{ .ColumnType = FTypedElementSyncFromWorldTag::StaticStruct(), .bAddColumn = true });
			SyncValue.AddUnique(FSyncTagInfo{ .ColumnType = FTypedElementSyncFromWorldInteractiveTag::StaticStruct(), .bAddColumn = false });
		}
	}
}

void UEditorDataStorageCompatibility::OnObjectModified(UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	if (Private::bUseCommandBuffer)
	{
		QueuedCommands.AddCommand(FAddSyncFromWorldTag{ .Target = Object });
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		// Determining the object is being tracked in the database can't be done safely as it may be queued for addition.
		// It would also add a small bit of performance overhead as access the lookup table can be done faster as a
		// batch operation during the tick step.
		ObjectsNeedingSyncTags.FindOrAdd(Object).AddUnique(
			FSyncTagInfo{ .ColumnType = FTypedElementSyncFromWorldTag::StaticStruct(), .bAddColumn = true });
	}
}

void UEditorDataStorageCompatibility::TriggerOnObjectAdded(
	const void* Object, UE::Editor::DataStorage::FObjectTypeInfo TypeInfo, UE::Editor::DataStorage::RowHandle Row) const
{
	using namespace UE::Editor::DataStorage;

	// Thread safe because it's only called from functions that already lock.

	for (const TPair<ObjectAddedCallback, FDelegateHandle>& CallbackPair : ObjectAddedCallbackList)
	{
		const ObjectAddedCallback& Callback = CallbackPair.Key;
		Callback(Object, TypeInfo, Row);
	}
}

void UEditorDataStorageCompatibility::TriggerOnPreObjectRemoved(
	const void* Object, UE::Editor::DataStorage::FObjectTypeInfo TypeInfo, UE::Editor::DataStorage::RowHandle Row) const
{
	using namespace UE::Editor::DataStorage;

	// Thread safe because it's only called from functions that already lock.

	for (const TPair<ObjectRemovedCallback, FDelegateHandle>& CallbackPair : PreObjectRemovedCallbackList)
	{
		const ObjectRemovedCallback& Callback = CallbackPair.Key;
		Callback(Object, TypeInfo, Row);
	}
}

void UEditorDataStorageCompatibility::OnObjectReinstanced(const FCoreUObjectDelegates::FReplacementObjectMap& ReplacedObjects)
{
	using namespace UE::Editor::DataStorage;
	
	if (Private::bUseCommandBuffer)
	{
		bool bHasUpdatedTypeInformation = false;
		for (TMap<UObject*, UObject*>::TConstIterator It = ReplacedObjects.CreateConstIterator(); It; ++It)
		{
			UStruct* Original = Cast<UStruct>(It->Key);
			UStruct* Reinstanced = Cast<UStruct>(It->Value);
			if (Original && Reinstanced)
			{
				QueuedCommands.AddCommand(FTypeInfoReinstanced{ .Original = Original, .Reinstanced = Reinstanced });
				bHasUpdatedTypeInformation = true;
			}
		}
	}
	else
	{
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);
		PendingTypeInformationUpdate.AddTypeInformation(ReplacedObjects);
	}
}

void UEditorDataStorageCompatibility::OnPostGcUnreachableAnalysis()
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Private;

	if (bIntegrateWithGC)
	{
		TEDS_EVENT_SCOPE(TEXT("Post GC clean up"));
		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		if (bUseCommandBuffer)
		{
			Storage->RunQuery(UObjectQuery, CreateDirectQueryCallbackBinding(
				[this](RowHandle Row, const FTypedElementUObjectIdColumn& ObjectId)
				{
					FUObjectItem* Description = GUObjectArray.IndexToObject(ObjectId.Id);
					if (ensureMsgf(Description && Description->GetSerialNumber() == ObjectId.SerialNumber,
						TEXT("The UObject found in TEDS no longer exists. TEDS was likely not informed in an earlier GC pass.")))
						// Unable to provide additional information such as the UObject's name as the UObject will not be valid.
					{
						if (Description->HasAnyFlags(EInternalObjectFlags::Garbage | EInternalObjectFlags::Unreachable))
						{
							if (UObject* Object = static_cast<UObject*>(Description->GetObject())) // No need to delete if this isn't a full UObject.
							{
								QueuedCommands.AddCommand(FRemoveCompatibleUObject{ .Object = Object, .ObjectRow = Row });
							}
						}
					}
				}));
			// Forcefully execute all pending commands to make sure there are no commands left that reference deleted objects as well
			// as to make sure the added deletes are executed to guaranteed there are no stale objects in TEDS.
			TickPendingCommands();
		}
		else
		{
			TArray<TPair<FUObjectItem*, RowHandle>> DeletedObjects;
			Storage->RunQuery(UObjectQuery, CreateDirectQueryCallbackBinding(
				[&DeletedObjects](RowHandle Row, const FTypedElementUObjectIdColumn& ObjectId)
				{
					FUObjectItem* Description = GUObjectArray.IndexToObject(ObjectId.Id);
					if (ensureMsgf(Description && Description->GetSerialNumber() == ObjectId.SerialNumber,
						TEXT("The UObject found in TEDS no longer exists. TEDS was likely not informed in an earlier GC pass.")))
						// Unable to provide additional information such as the UObject's name as the UObject will not be valid.
					{
						if (Description->HasAnyFlags(EInternalObjectFlags::Garbage | EInternalObjectFlags::Unreachable))
						{
							DeletedObjects.Emplace(Description, Row);
						}
					}
				}));

			for (TPair<FUObjectItem*, RowHandle>& ObjectItem : DeletedObjects)
			{
				if (UObject* Object = static_cast<UObject*>(ObjectItem.Key->GetObject())) // No need to delete if this isn't a full UObject.
				{
					RemoveCompatibleObjectExplicitTransactionable<false>(Object, ObjectItem.Value);
				}
			}
		}
	}
}

SIZE_T GetTypeHash(const UEditorDataStorageCompatibility::FSyncTagInfo& Column)
{
	// Thread safe as it only uses local data.
	return HashCombine(Column.ColumnType.GetWeakPtrTypeHash(), Column.bAddColumn);
}

//
// UEditorDataStorageCompatibility::FRegistrationCommandChange
//

UEditorDataStorageCompatibility::FRegistrationCommandChange::FRegistrationCommandChange(
	UEditorDataStorageCompatibility* InOwner, UObject* InTargetObject)
	: Owner(InOwner)
	, TargetObject(InTargetObject)
{
}

UEditorDataStorageCompatibility::FRegistrationCommandChange::~FRegistrationCommandChange()
{
	// Does not require any thread locking as IsRowAvailable is thread safe and DestroyMemento will lock.
	using namespace UE::Editor::DataStorage;
	
	// If there has been no revert operation, there's also no memento.
	if (UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get(); 
		DataStorageCompat && DataStorageCompat->Storage && DataStorageCompat->Storage->IsRowAvailable(MementoRow))
	{
		if (Private::bUseCommandBuffer)
		{
			DataStorageCompat->QueuedCommands.AddCommand(UE::Editor::DataStorage::FDestroyMemento{ .MementoRow = MementoRow});
		}
		else
		{
			DataStorageCompat->Environment->GetMementoSystem().DestroyMemento(MementoRow);
		}
	}
}

void UEditorDataStorageCompatibility::FRegistrationCommandChange::Apply(UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	checkf(Owner.IsValid() && Owner.Get() == Object, 
		TEXT("Applying registration transaction command within TEDS Compat was called after TEDS is not longer available."));
	UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get();
	if (UObject* TargetRetrieved = TargetObject.Get(/*bEvenIfPendingKill=*/ true))
	{
		if (Private::bUseCommandBuffer)
		{
			RowHandle ObjectRow = DataStorageCompat->AddCompatibleObjectExplicitTransactionable<false>(TargetRetrieved);
			DataStorageCompat->QueuedCommands.AddCommand(FRestoreMemento{ .MementoRow = MementoRow, .TargetRow  = ObjectRow });
		}
		else
		{
			// Lock here because the next two functions would otherwise lock multiple times.
			FScopedExclusiveLock Lock(EGlobalLockScope::Public);
			RowHandle ObjectRow = DataStorageCompat->AddCompatibleObjectExplicitTransactionable<false>(TargetRetrieved);
			DataStorageCompat->Environment->GetMementoSystem().RestoreMemento(MementoRow, ObjectRow);
		}
	}
}

void UEditorDataStorageCompatibility::FRegistrationCommandChange::Revert(UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	checkf(Owner.IsValid() && Owner.Get() == Object,
		TEXT("Reverting registration transaction command within TEDS Compat was called after TEDS is not longer available."));

	FScopedExclusiveLock Lock(EGlobalLockScope::Public);
	if (UObject* TargetRetrieved = TargetObject.Get(/*bEvenIfPendingKill=*/ true))
	{
		UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get(); 
		ICoreProvider* DataStorage = DataStorageCompat->Storage;
			
		RowHandle ObjectRow = DataStorageCompat->FindRowWithCompatibleObjectExplicit(TargetRetrieved);
		if (DataStorage->IsRowAvailable(ObjectRow))
		{
			if (Private::bUseCommandBuffer && Private::bUseDeferredRemovesInCompat)
			{
				MementoRow = DataStorage->ReserveRow();
				DataStorageCompat->QueuedCommands.AddCommand(FCreateMemento{ .ReservedMementoRow = MementoRow, .TargetRow = ObjectRow });
			}
			else
			{
				MementoRow = DataStorageCompat->Environment->GetMementoSystem().CreateMemento(ObjectRow);
			}
			DataStorageCompat->RemoveCompatibleObjectExplicitTransactionable<false>(TargetRetrieved, ObjectRow);
		}
	}
}

FString UEditorDataStorageCompatibility::FRegistrationCommandChange::ToString() const
{
	return TEXT("Typed Element Data Storage Compatibility - Registration");
}


//
// UEditorDataStorageCompatibility::FDeregistrationCommandChange
//

UEditorDataStorageCompatibility::FDeregistrationCommandChange::FDeregistrationCommandChange(
	UEditorDataStorageCompatibility* InOwner, UObject* InTargetObject)
	: Owner(InOwner)
	, TargetObject(InTargetObject)
{
	using namespace UE::Editor::DataStorage;

	ICoreProvider* DataStorage = InOwner->Storage;

	RowHandle ObjectRow = InOwner->FindRowWithCompatibleObjectExplicit(InTargetObject);
	if (DataStorage->IsRowAvailable(ObjectRow))
	{
		if (Private::bUseCommandBuffer && Private::bUseDeferredRemovesInCompat)
		{
			MementoRow = DataStorage->ReserveRow();
			InOwner->QueuedCommands.AddCommand(FCreateMemento{ .ReservedMementoRow = MementoRow, .TargetRow = ObjectRow });
		}
		else
		{
			MementoRow = InOwner->Environment->GetMementoSystem().CreateMemento(ObjectRow);
		}
	}
}

UEditorDataStorageCompatibility::FDeregistrationCommandChange::~FDeregistrationCommandChange()
{
	using namespace UE::Editor::DataStorage;

	// There's no memento row if target object was never registered with TEDS Compat.
	if (UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get();
		DataStorageCompat && DataStorageCompat->Storage && DataStorageCompat->Storage->IsRowAvailable(MementoRow))
	{
		if (Private::bUseCommandBuffer)
		{
			DataStorageCompat->QueuedCommands.AddCommand(FDestroyMemento{ .MementoRow = MementoRow });
		}
		else
		{
			DataStorageCompat->Environment->GetMementoSystem().DestroyMemento(MementoRow);
		}
	}
}

void UEditorDataStorageCompatibility::FDeregistrationCommandChange::Apply(UObject* Object)
{
	// All function calls are guaranteed to be thread safe.

	checkf(Owner.IsValid() && Owner.Get() == Object,
		TEXT("Applying deregistration transaction command within TEDS Compat was called after TEDS is not longer available."));
	UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get();
	if (UObject* TargetRetrieved = TargetObject.Get(/*bEvenIfPendingKill=*/ true))
	{
		DataStorageCompat->RemoveCompatibleObjectExplicitTransactionable<false>(TargetRetrieved);
	}
}

void UEditorDataStorageCompatibility::FDeregistrationCommandChange::Revert(UObject* Object)
{
	using namespace UE::Editor::DataStorage;

	checkf(Owner.IsValid() && Owner.Get() == Object,
		TEXT("Reverting deregistration transaction command within TEDS Compat was called after TEDS is not longer available."));
	
	UEditorDataStorageCompatibility* DataStorageCompat = Owner.Get();
	if (UObject* TargetRetrieved = TargetObject.Get(/*bEvenIfPendingKill=*/ true))
	{
		if(Private::bUseCommandBuffer)
		{
			RowHandle ObjectRow = DataStorageCompat->AddCompatibleObjectExplicitTransactionable<false>(TargetRetrieved);
			DataStorageCompat->QueuedCommands.AddCommand(FRestoreMemento{ .MementoRow = MementoRow, .TargetRow = ObjectRow });
		}
		else
		{
			// Lock here because the next two functions would otherwise lock multiple times.
			FScopedExclusiveLock Lock(EGlobalLockScope::Public);
			RowHandle ObjectRow = DataStorageCompat->AddCompatibleObjectExplicitTransactionable<false>(TargetRetrieved);
			DataStorageCompat->Environment->GetMementoSystem().RestoreMemento(MementoRow, ObjectRow);
		}
	}
}

FString UEditorDataStorageCompatibility::FDeregistrationCommandChange::ToString() const
{
	return TEXT("Typed Element Data Storage Compatibility - Deregistration");
}
