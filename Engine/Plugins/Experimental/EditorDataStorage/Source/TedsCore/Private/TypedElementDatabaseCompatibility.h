// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Async/Mutex.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Commands/EditorDataStorageCommandBuffer.h"
#include "Commands/EditorDataStorageCompatibilityCommands.h"
#include "Compatibility/TypedElementObjectReinstancingManager.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Engine/World.h"
#include "Misc/Change.h"
#include "Misc/Optional.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"

#include "TypedElementDatabaseCompatibility.generated.h"

#define UE_API TEDSCORE_API

struct FMassActorManager;
class UTypedElementMementoSystem;

namespace UE::Editor::DataStorage
{
	class FEnvironment;
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

UCLASS(MinimalAPI)
class UEditorDataStorageCompatibility
	: public UObject
	, public UE::Editor::DataStorage::ICompatibilityProvider
{
	GENERATED_BODY()

	friend struct UE::Editor::DataStorage::FCommandProcessor;
	friend struct UE::Editor::DataStorage::FPatchData;
	friend struct UE::Editor::DataStorage::FPrepareCommands;
public:
	~UEditorDataStorageCompatibility() override;

	UE_API void Initialize(UEditorDataStorage* InStorage);
	UE_API void PostInitialize(UEditorDataStorage* InStorage);
	UE_API void Deinitialize();

	UE_API void RegisterRegistrationFilter(ObjectRegistrationFilter Filter) override;
	UE_API void RegisterDealiaserCallback(ObjectToRowDealiaser Dealiaser) override;
	UE_API void RegisterTypeTableAssociation(TWeakObjectPtr<UStruct> TypeInfo, UE::Editor::DataStorage::TableHandle Table) override;
	UE_API FDelegateHandle RegisterObjectAddedCallback(UE::Editor::DataStorage::ObjectAddedCallback&& OnObjectAdded);
	UE_API void UnregisterObjectAddedCallback(FDelegateHandle Handle);
	UE_API FDelegateHandle RegisterObjectRemovedCallback(UE::Editor::DataStorage::ObjectRemovedCallback&& OnObjectRemoved);
	UE_API void UnregisterObjectRemovedCallback(FDelegateHandle Handle);
	
	UE_API UE::Editor::DataStorage::RowHandle AddCompatibleObjectExplicit(UObject* Object) override;
	UE_API UE::Editor::DataStorage::RowHandle AddCompatibleObjectExplicit(void* Object, TWeakObjectPtr<const UScriptStruct> TypeInfo) override;
	
	UE_API void RemoveCompatibleObjectExplicit(UObject* Object) override;
	UE_API void RemoveCompatibleObjectExplicit(void* Object) override;

	UE_API UE::Editor::DataStorage::RowHandle FindRowWithCompatibleObjectExplicit(const UObject* Object) const override;
	UE_API UE::Editor::DataStorage::RowHandle FindRowWithCompatibleObjectExplicit(const void* Object) const override;

	UE_API bool SupportsExtension(FName Extension) const override;
	UE_API void ListExtensions(TFunctionRef<void(FName)> Callback) const override;

protected:
	UE_API void EnterImmediateMode(UE::Editor::DataStorage::FObjectCompatibilityImmediateScope& Scope) override;
	UE_API void LeaveImmediateMode(UE::Editor::DataStorage::FObjectCompatibilityImmediateScope& Scope) override;
	
private:
	// The below changes expect UEditorDataStorageCompatibility to be the object passed in to StoreUndo.
	// Note that we cannot pass in TargetObject to StoreUndo because doing so seems to stomp regular Modify()
	// changes for that object.
	class FRegistrationCommandChange final : public FCommandChange
	{
	public:
		FRegistrationCommandChange(UEditorDataStorageCompatibility* InOwner, UObject* InTargetObject);
		~FRegistrationCommandChange() override;

		void Apply(UObject* Object) override;
		void Revert(UObject* Object) override;
		FString ToString() const override;

	private:
		TWeakObjectPtr<UEditorDataStorageCompatibility> Owner;
		TWeakObjectPtr<UObject> TargetObject;
		UE::Editor::DataStorage::RowHandle MementoRow = UE::Editor::DataStorage::InvalidRowHandle;
	};
	class FDeregistrationCommandChange final : public FCommandChange
	{
	public:
		FDeregistrationCommandChange(UEditorDataStorageCompatibility* InOwner, UObject* InTargetObject);
		~FDeregistrationCommandChange() override;

		void Apply(UObject* Object) override;
		void Revert(UObject* Object) override;
		FString ToString() const override;

	private:
		TWeakObjectPtr<UEditorDataStorageCompatibility> Owner;
		TWeakObjectPtr<UObject> TargetObject;
		UE::Editor::DataStorage::RowHandle MementoRow = UE::Editor::DataStorage::InvalidRowHandle;
	};

	UE_API void Prepare();
	UE_API void CreateStandardArchetypes();
	UE_API void RegisterTypeInformationQueries();
	
	UE_API bool ShouldAddObject(const UObject* Object) const;
	UE_API UE::Editor::DataStorage::TableHandle FindBestMatchingTable(const UStruct* TypeInfo) const;
	template<bool bEnableTransactions>
	UE::Editor::DataStorage::RowHandle AddCompatibleObjectExplicitTransactionable(UObject* Object);
	template<bool bEnableTransactions>
	void RemoveCompatibleObjectExplicitTransactionable(const UObject* Object);
	template<bool bEnableTransactions>
	void RemoveCompatibleObjectExplicitTransactionable(const UObject* Object, UE::Editor::DataStorage::RowHandle ObjectRow);
	UE_API UE::Editor::DataStorage::RowHandle DealiasObject(const UObject* Object) const;

	UE_API void Tick();
	UE_API void TickPendingCommands();
	UE_API void TickPendingUObjectRegistration();
	UE_API void TickPendingExternalObjectRegistration();
	UE_API void TickObjectSync();

	UE_API void OnPrePropertyChanged(UObject* Object, const FEditPropertyChain& PropertyChain);
	UE_API void OnPostEditChangeProperty(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void OnObjectModified(UObject* Object);
	UE_API void TriggerOnObjectAdded(const void* Object, UE::Editor::DataStorage::FObjectTypeInfo TypeInfo, UE::Editor::DataStorage::RowHandle Row) const;
	UE_API void TriggerOnPreObjectRemoved(const void* Object, UE::Editor::DataStorage::FObjectTypeInfo TypeInfo, UE::Editor::DataStorage::RowHandle Row) const;
	UE_API void OnObjectReinstanced(const FCoreUObjectDelegates::FReplacementObjectMap& ReplacedObjects);

	UE_API void OnPostGcUnreachableAnalysis();

	struct FPendingTypeInformationUpdate
	{
	public:
		FPendingTypeInformationUpdate();

		void AddTypeInformation(const TMap<UObject*, UObject*>& ReplacedObjects);
		void Process(UEditorDataStorageCompatibility& Compatibility);

	private:
		TOptional<TWeakObjectPtr<UObject>> ProcessResolveTypeRecursively(const TWeakObjectPtr<const UObject>& Target);

		struct FTypeInfoEntryKeyFuncs : TDefaultMapHashableKeyFuncs<TWeakObjectPtr<UObject>, TWeakObjectPtr<UObject>, false>
		{
			static inline bool Matches(KeyInitType Lhs, KeyInitType Rhs) { return Lhs.HasSameIndexAndSerialNumber(Rhs); }
		};
		using PendingTypeInformationMap = TMap<TWeakObjectPtr<UObject>, TWeakObjectPtr<UObject>, FDefaultSetAllocator, FTypeInfoEntryKeyFuncs>;

		PendingTypeInformationMap PendingTypeInformationUpdates[2];
		PendingTypeInformationMap* PendingTypeInformationUpdatesActive;
		PendingTypeInformationMap* PendingTypeInformationUpdatesSwapped;
		TArray<TTuple<TWeakObjectPtr<UStruct>, UE::Editor::DataStorage::TableHandle>> UpdatedTypeInfoScratchBuffer;
		UE::FMutex Safeguard;
		std::atomic<bool> bHasPendingUpdate = false;
	};
	FPendingTypeInformationUpdate PendingTypeInformationUpdate;

	struct ExternalObjectRegistration
	{
		void* Object;
		TWeakObjectPtr<const UScriptStruct> TypeInfo;
	};
	
	template<typename AddressType>
	struct PendingRegistration
	{
	private:
		struct FEntry
		{
			AddressType Address;
			UE::Editor::DataStorage::RowHandle Row;
			UE::Editor::DataStorage::TableHandle Table;
		};
		TArray<FEntry> Entries;
		TArray<FEntry> ProcessingEntries;

	public:
		void Add(UE::Editor::DataStorage::RowHandle ReservedRowHandle, AddressType Address);
		bool IsEmpty() const;
		int32 Num() const;
		
		void ForEachAddress(const TFunctionRef<void(AddressType&)>& Callback);
		// Processes the entries and return the entries remaining in the list. They will be at the back of the list
		// Once this returns, the list will contain only entries that were not processed
		void ProcessEntries(UE::Editor::DataStorage::ICoreProvider& Storage, UEditorDataStorageCompatibility& Compatibility,
			const TFunctionRef<void(UE::Editor::DataStorage::RowHandle, const AddressType&)>& SetupRowCallback);
	};

	UE::Editor::DataStorage::CompatibilityCommandBuffer QueuedCommands;
	UE::Editor::DataStorage::CompatibilityCommandBuffer::FCollection PendingCommands;
	PendingRegistration<TWeakObjectPtr<UObject>> UObjectsPendingRegistration;
	PendingRegistration<ExternalObjectRegistration> ExternalObjectsPendingRegistration;
	TArray<UE::Editor::DataStorage::RowHandle> RowScratchBuffer;
	
	TArray<ObjectRegistrationFilter> ObjectRegistrationFilters;
	TArray<ObjectToRowDealiaser> ObjectToRowDialiasers;
	using TypeToTableMapType = TMap<TWeakObjectPtr<UStruct>, UE::Editor::DataStorage::TableHandle>;
	TypeToTableMapType TypeToTableMap;
	TArray<TPair<UE::Editor::DataStorage::ObjectAddedCallback, FDelegateHandle>> ObjectAddedCallbackList;
	TArray<TPair<UE::Editor::DataStorage::ObjectRemovedCallback, FDelegateHandle>> PreObjectRemovedCallbackList;

	static thread_local UE::Editor::DataStorage::FObjectCompatibilityImmediateScope* ImmediateModeTracker;
	
	UE::Editor::DataStorage::TableHandle StandardUObjectTable{ UE::Editor::DataStorage::InvalidTableHandle };
	UE::Editor::DataStorage::TableHandle StandardExternalObjectTable{ UE::Editor::DataStorage::InvalidTableHandle };
	UE::Editor::DataStorage::ICoreProvider* Storage{ nullptr };

	/**
	 * Reference of objects (UObject and AActor) that need to be fully synced from the world to the database.
	 * Caution: Could point to objects that have been GC-ed
	 */
	struct FSyncTagInfo
	{
		TWeakObjectPtr<const UScriptStruct> ColumnType;
		bool bAddColumn;

		bool operator==(const FSyncTagInfo& Rhs) const = default;
		bool operator!=(const FSyncTagInfo& Rhs) const = default;
	};
	friend SIZE_T GetTypeHash(const FSyncTagInfo& Column);
	static constexpr uint32 MaxExpectedTagsForObjectSync = 2;
	using ObjectsNeedingSyncTagsMapKey = TObjectKey<const UObject>;
	using ObjectsNeedingSyncTagsMapValue = TArray<FSyncTagInfo, TInlineAllocator<MaxExpectedTagsForObjectSync>>;
	using ObjectsNeedingSyncTagsMap = TMap<ObjectsNeedingSyncTagsMapKey, ObjectsNeedingSyncTagsMapValue>;
	ObjectsNeedingSyncTagsMap ObjectsNeedingSyncTags;

	FDelegateHandle PreEditChangePropertyDelegateHandle;
	FDelegateHandle PostEditChangePropertyDelegateHandle;
	FDelegateHandle ObjectModifiedDelegateHandle;
	FDelegateHandle ObjectReinstancedDelegateHandle;
	FDelegateHandle PostGcUnreachableAnalysisHandle;
	
	TSharedPtr<UE::Editor::DataStorage::FEnvironment> Environment;
	UE::Editor::DataStorage::QueryHandle ClassTypeInfoQuery;
	UE::Editor::DataStorage::QueryHandle ScriptStructTypeInfoQuery;
	UE::Editor::DataStorage::QueryHandle UObjectQuery;
};

SIZE_T GetTypeHash(const UEditorDataStorageCompatibility::FSyncTagInfo& Column);

#undef UE_API
