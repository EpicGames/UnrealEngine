// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseEnvironment.h"

#include "MassCommandBuffer.h"
#include "TypedElementDatabase.h"

namespace UE::Editor::DataStorage
{
	FEnvironment::FEnvironment(UEditorDataStorage& InDataStorage,
		FMassEntityManager& InMassEntityManager, FMassProcessingPhaseManager& InMassPhaseManager)
		: DirectDeferredCommands(*this)
		, TableManager(InMassEntityManager, DynamicColumnGenerator)
		, MappingTable(InDataStorage)
		, Queries(DynamicColumnGenerator)
		, MementoSystem(InDataStorage, TableManager)
		, ScopeSystem(InDataStorage)
		, ValueTagManager(InMassEntityManager, DynamicColumnGenerator)
		, ActiveCommandQueue(CommandQueues)
		, MassEntityManager(InMassEntityManager)
		, MassPhaseManager(InMassPhaseManager)
		, SharedMassCommandBuffer(MakeShared<FMassCommandBuffer>())
	{
		DynamicColumnGenerator.SetQueryStore(Queries);
	}

	FEnvironment::~FEnvironment()
	{
		DirectDeferredCommands.ClearCommands();
	}

	Legacy::FCommandBuffer& FEnvironment::GetDirectDeferredCommands()
	{
		return DirectDeferredCommands;
	}

	const Legacy::FCommandBuffer& FEnvironment::GetDirectDeferredCommands() const
	{
		return DirectDeferredCommands;
	}

	FMappingTable& FEnvironment::GetMappingTable()
	{
		return MappingTable;
	}

	const FMappingTable& FEnvironment::GetMappingTable() const
	{
		return MappingTable;
	}

	FScratchBuffer& FEnvironment::GetScratchBuffer()
	{
		return ScratchBuffer;
	}

	const FScratchBuffer& FEnvironment::GetScratchBuffer() const
	{
		return ScratchBuffer;
	}

	FExtendedQueryStore& FEnvironment::GetQueryStore()
	{
		return Queries;
	}

	const FExtendedQueryStore& FEnvironment::GetQueryStore() const
	{
		return Queries;
	}

	FMementoSystem& FEnvironment::GetMementoSystem()
	{
		return MementoSystem;
	}

	const FMementoSystem& FEnvironment::GetMementoSystem() const
	{
		return MementoSystem;
	}

	FScopeSystem& FEnvironment::GetScopeSystem()
	{
		return ScopeSystem;
	}

	const FScopeSystem& FEnvironment::GetScopeSystem() const
	{
		return ScopeSystem;
	}

	FMassEntityManager& FEnvironment::GetMassEntityManager()
	{
		return MassEntityManager;
	}

	const FMassEntityManager& FEnvironment::GetMassEntityManager() const
	{
		return MassEntityManager;
	}

	FTedsHierarchyRegistrar& FEnvironment::GetHierarchyRegistrar()
	{
		return HierarchyInterface;
	}

	const FTedsHierarchyRegistrar& FEnvironment::GetHierarchyRegistrar() const
	{
		return HierarchyInterface;
	}

	FTedsRelationAdapter& FEnvironment::GetRelationAdapter()
	{
		return RelationAdapter;
	}

	const FTedsRelationAdapter& FEnvironment::GetRelationAdapter() const
	{
		return RelationAdapter;
	}

	FTedsHierarchicalRelationManager& FEnvironment::GetHierarchicalRelationManager()
	{
		return HierarchicalRelationManager;
	}

	const FTedsHierarchicalRelationManager& FEnvironment::GetHierarchicalRelationManager() const
	{
		return HierarchicalRelationManager;
	}

	FMassProcessingPhaseManager& FEnvironment::GetMassPhaseManager()
	{
		return MassPhaseManager;
	}

	const FMassProcessingPhaseManager& FEnvironment::GetMassPhaseManager() const
	{
		return MassPhaseManager;
	}

	TSharedPtr<FMassCommandBuffer> FEnvironment::GetSharedMassCommandBuffer()
	{
		return SharedMassCommandBuffer;
	}

	TSharedPtr<const FMassCommandBuffer> FEnvironment::GetSharedMassCommandBuffer() const
	{
		return SharedMassCommandBuffer;
	}

	FTableManager& FEnvironment::GetTableManager()
	{
		return TableManager;
	}

	const FTableManager& FEnvironment::GetTableManager() const
	{
		return TableManager;
	}

	const UScriptStruct* FEnvironment::FindDynamicColumn(const UScriptStruct& Template, const FName& Identifier) const
	{
		return DynamicColumnGenerator.FindByTemplateId(Template, Identifier);
	}

	const UScriptStruct* FEnvironment::GenerateDynamicColumn(const UScriptStruct& Template, const FName& Identifier)
	{
		return DynamicColumnGenerator.GenerateColumn(Template, Identifier).Type;
	}

	FConstSharedStruct FEnvironment::GenerateValueTag(const FValueTag& Tag, const FName& Value)
	{
		return ValueTagManager.GenerateValueTag(Tag, Value);
	}

	const UScriptStruct* FEnvironment::GenerateColumnType(const FValueTag& Tag)
	{
		return ValueTagManager.GenerateColumnType(Tag);
	}

	void FEnvironment::ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const UScriptStruct& Type)> Callback) const
	{
		DynamicColumnGenerator.ForEachDynamicColumn(Template, [Callback](const FDynamicColumnGeneratorInfo& Info)
    		{
    			if (Info.Type)
    			{
    				Callback(*Info.Type);
    			}
    		});
	}

	void FEnvironment::NextUpdateCycle()
	{
		Queries.UpdateActivatableQueries();
		FlushCommands();
		ScratchBuffer.BatchDelete();
		
		UpdateCycleId++;
	}

	uint64 FEnvironment::GetUpdateCycleId() const
	{
		return UpdateCycleId;
	}

	void FEnvironment::PushCommands(TConstArrayView<const FEnvironmentCommand> Commands)
	{
		TUniqueLock<FMutex> Lock(CommandQueueMutex);
		ActiveCommandQueue->Append(Commands);
	}

	void FEnvironment::FlushCommands()
	{
		while (true)
		{
			TSharedPtr<FMassCommandBuffer> MassCommandBuffer = GetSharedMassCommandBuffer();
			bool bFlushMassCommandBuffer = MassCommandBuffer.IsValid() && MassCommandBuffer->HasPendingCommands();
			
			TArray<FEnvironmentCommand>* ProcessingQueue;
			{
				TUniqueLock<FMutex> Lock(CommandQueueMutex);
				if (!bFlushMassCommandBuffer && ActiveCommandQueue->IsEmpty())
				{
					return;
				}
				ProcessingQueue = ActiveCommandQueue;
				ActiveCommandQueue = (ActiveCommandQueue == &(CommandQueues[0]))
					? &(CommandQueues[1])
					: &(CommandQueues[0]);
			}

			if (bFlushMassCommandBuffer)
			{
				MassEntityManager.FlushCommands(GetSharedMassCommandBuffer());
			}
			for (FEnvironmentCommand& Command : *ProcessingQueue)
			{
				Command.CommandFunction(Command.CommandData);
			}
			ProcessingQueue->SetNum(0, EAllowShrinking::No);
		}
	}
} // namespace UE::Editor::DataStorage
