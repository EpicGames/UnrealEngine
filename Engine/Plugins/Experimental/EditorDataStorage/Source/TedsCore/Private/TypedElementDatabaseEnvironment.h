// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "DynamicColumnGenerator.h"
#include "MassEntityManager.h"
#include "MassProcessingPhaseManager.h"
#include "TableManager.h"
#include "Templates/SharedPointer.h"
#include "TypedElementDatabaseCommandBuffer.h"
#include "TypedElementDatabaseScratchBuffer.h"
#include "TypedElementDatabaseIndexTable.h"
#include "Hierarchy/EditorDataStorageHierarchyImplementation.h"
#include "Relations/EditorDataStorageRelationsImplementation.h"
#include "Memento/TypedElementMementoSystem.h"
#include "Queries/TypedElementExtendedQueryStore.h"
#include "Scope/TypedElementScopeSystem.h"

class UEditorDataStorage;
struct FMassCommandBuffer;

namespace UE::Editor::DataStorage
{
	class FEnvironment final
	{
	public:
		FEnvironment(UEditorDataStorage& InDataStorage,
			FMassEntityManager& InMassEntityManager, FMassProcessingPhaseManager& InMassPhaseManager);

		~FEnvironment();

		Legacy::FCommandBuffer& GetDirectDeferredCommands();
		const Legacy::FCommandBuffer& GetDirectDeferredCommands() const;

		FMappingTable& GetMappingTable();
		const FMappingTable& GetMappingTable() const;

		FScratchBuffer& GetScratchBuffer();
		const FScratchBuffer& GetScratchBuffer() const;

		FExtendedQueryStore& GetQueryStore();
		const FExtendedQueryStore& GetQueryStore() const;

		FMementoSystem& GetMementoSystem();
		const FMementoSystem& GetMementoSystem() const;

		FScopeSystem& GetScopeSystem();
		const FScopeSystem& GetScopeSystem() const;

		FMassEntityManager& GetMassEntityManager();
		const FMassEntityManager& GetMassEntityManager() const;

		FTedsHierarchyRegistrar& GetHierarchyRegistrar();
		const FTedsHierarchyRegistrar& GetHierarchyRegistrar() const;

		FTedsRelationAdapter& GetRelationAdapter();
		const FTedsRelationAdapter& GetRelationAdapter() const;

		FTedsHierarchicalRelationManager& GetHierarchicalRelationManager();
		const FTedsHierarchicalRelationManager& GetHierarchicalRelationManager() const;

		FMassProcessingPhaseManager& GetMassPhaseManager();
		const FMassProcessingPhaseManager& GetMassPhaseManager() const;

		TSharedPtr<FMassCommandBuffer> GetSharedMassCommandBuffer();
		TSharedPtr<const FMassCommandBuffer> GetSharedMassCommandBuffer() const;

		FTableManager& GetTableManager();
		const FTableManager& GetTableManager() const;

		// Finds the type information for the dynamic column
		// Dynamic columns are specified by a template layout and a FName Identifier
		const UScriptStruct* FindDynamicColumn(const UScriptStruct& Template, const FName& Identifier) const;
		// Generates or returns an existing type for the dynamic column
		// Dynamic columns are specified by a template layout and a FName Identifier
		const UScriptStruct* GenerateDynamicColumn(const UScriptStruct& Template, const FName& Identifier);
		
		// Creates or Finds the column type associated with the value tag
		const UScriptStruct* GenerateColumnType(const FValueTag& Tag);

		// Executes the given callback for each known dynamic column that derives from the base template provided
		void ForEachDynamicColumn(const UScriptStruct& Template, TFunctionRef<void(const UScriptStruct& Type)> Callback) const;
		
		// Creates an instance of a value tag
		FConstSharedStruct GenerateValueTag(const FValueTag& Tag, const FName& Value);

		void NextUpdateCycle();
		uint64 GetUpdateCycleId() const;
		
		struct FEnvironmentCommand
		{
			void (*CommandFunction)(void*);
			// If this is not static data or null, it should be a pointer into the scratch buffer
			void* CommandData = nullptr;
		};

		// Commands are automatically flushed on NextUpdateCycle or when explicitly flushed with `FlushCommands`.
		void PushCommands(TConstArrayView<const FEnvironmentCommand> Commands);
		void FlushCommands();

	private:
		Legacy::FCommandBuffer DirectDeferredCommands;
		FDynamicColumnGenerator DynamicColumnGenerator;
		FTableManager TableManager;
		FMappingTable MappingTable;
		FScratchBuffer ScratchBuffer;
		FExtendedQueryStore Queries;
		FMementoSystem MementoSystem;
		FScopeSystem ScopeSystem;
		FValueTagManager ValueTagManager;
		FTedsHierarchyRegistrar HierarchyInterface;
		FTedsRelationAdapter RelationAdapter;
		FTedsHierarchicalRelationManager HierarchicalRelationManager;

		FMutex CommandQueueMutex;
		TArray<FEnvironmentCommand> CommandQueues[2];
		TArray<FEnvironmentCommand>* ActiveCommandQueue;

		FMassEntityManager& MassEntityManager;
		FMassProcessingPhaseManager& MassPhaseManager;
		TSharedPtr<FMassCommandBuffer> SharedMassCommandBuffer;

		uint64 UpdateCycleId = 0;
	};

	struct FEmptyEnvironmentRetrievalBase {};

	template<typename Base = FEmptyEnvironmentRetrievalBase>
	class IQueryEnvironmentRetrieval : public Base
	{
	public:
		virtual ~IQueryEnvironmentRetrieval() = default;

	protected:
		virtual FEnvironment& GetEnvironment() = 0;
		virtual const FEnvironment& GetEnvironment() const = 0;
	};

	using QueryEnvironmentRetrieval = IQueryEnvironmentRetrieval<>;
} // namespace UE::Editor::DataStorage
