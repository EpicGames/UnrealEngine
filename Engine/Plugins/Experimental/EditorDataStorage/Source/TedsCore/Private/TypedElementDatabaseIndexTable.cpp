// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseIndexTable.h"

#include "Async/ParallelFor.h"
#include "DataStorage/Debug/Log.h"
#include "GenericPlatform/GenericPlatformTime.h"
#include "Hash/CityHash.h"
#include "TypedElementDatabase.h"
#include "TypedElementDataStorageProfilingMacros.h"

namespace UE::Editor::DataStorage
{
	const FTimespan FMappingTable::FCleanUpInfo::JobShrinkThreshold = FTimespan::FromMicroseconds(100);
	const FTimespan FMappingTable::FCleanUpInfo::JobGrowthThreshold = FTimespan::FromMicroseconds(500);

	const FTimespan FMappingTable::FCleanUpInfo::MaxBatchDuration = FTimespan::FromMicroseconds(500);
	const FTimespan FMappingTable::FCleanUpInfo::MinBatchDuration = FTimespan::FromMicroseconds(350);

	FMappingTable::FMappingTable(UEditorDataStorage& DataStorage)
		: DataStorage(DataStorage)
	{
		DataStorage.RegisterCooperativeUpdate(FName("Mapping Table GC"), ICoreProvider::ECooperativeTaskPriority::Medium,
			[this](FTimespan TimeAllowance)
			{
				RemoveInvalidRows(TimeAllowance);
			});
	}

	FMappingTable::~FMappingTable()
	{
		DataStorage.UnregisterCooperativeUpdate(FName("Mapping Table GC"));
	}

	RowHandle FMappingTable::Lookup(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& Key) const
	{
		FScopedSharedLock Lock(LockScope);

		if (const FDomain* DomainInfo = Domains.Find(Domain))
		{
			if (int32 Index = DomainInfo->FindIndexUnguarded(Key); Index >= 0)
			{
				RowHandle Result = DomainInfo->Rows[Index];
				if (IsDirty() && !DataStorage.IsRowAvailableUnsafe(Result))
				{
					Result = InvalidRowHandle;
				}
				return Result;
			}
		}
		return InvalidRowHandle;
	}

	void FMappingTable::Map(EGlobalLockScope LockScope, const FName& Domain, FMapKey Key, RowHandle Row)
	{
		FScopedExclusiveLock Lock(LockScope);
		
		FDomain& DomainInfo = Domains.FindOrAdd(Domain);
		DomainInfo.IndexRowUnguarded(DataStorage, MoveTemp(Key), Row, IsDirty());
	}

	void FMappingTable::BatchMap(EGlobalLockScope LockScope, const FName& Domain, TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs)
	{
		FScopedExclusiveLock Lock(LockScope);

		FDomain& DomainInfo = Domains.FindOrAdd(Domain);
		DomainInfo.IndexLookupMap.ReserveAdditional(MapRowPairs);

		checkf(DomainInfo.Rows.Num() >= DomainInfo.FreeList.Num(),
			TEXT("There can't be less rows than there are rows stored in the free list as free list is a subset of rows."));
		int32 ArraySize = DomainInfo.Rows.Num() - DomainInfo.FreeList.Num() + MapRowPairs.Num();
		DomainInfo.Rows.Reserve(ArraySize);
		DomainInfo.Keys.Reserve(ArraySize);

		for (TPair<FMapKey, RowHandle>& IndexAndRow : MapRowPairs)
		{
			DomainInfo.IndexRowUnguarded(DataStorage, MoveTemp(IndexAndRow.Key), IndexAndRow.Value, IsDirty());
		}
	}

	void FMappingTable::Remap(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& OriginalKey, FMapKey NewKey)
	{
		FScopedExclusiveLock Lock(LockScope);

		if (FDomain* DomainInfo = Domains.Find(Domain))
		{
			DomainInfo->RemapUnguarded(DataStorage, OriginalKey, MoveTemp(NewKey), IsDirty());
		}
	}

	void FMappingTable::Remove(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& Key)
	{
		FScopedExclusiveLock Lock(LockScope);
		
		if (FDomain* DomainInfo = Domains.Find(Domain))
		{
			DomainInfo->RemoveUnguarded(Key);
		}
	}

	void FMappingTable::MarkDirty()
	{
		DirtyDueToRemoval = true;
		CleanUpFrameDelay = DefaultCleanUpFrameDelay;
	}

	FMappingTable::FDomain::ERemoveInvalidRowsResult FMappingTable::FDomain::RemoveInvalidRows(
		const FName& DomainName, UEditorDataStorage& InDataStorage, FTimespan& RemainingTime)
	{
		FCleanUpInfo& Info = *CleanUpInfo;

		// Only do work if there's any left
		if (Info.RemainingJobs > 0 || !Info.DeletionQueue.IsEmpty())
		{
			uint64 DomainStartTime = FPlatformTime::Cycles64();
			FTimespan DomainRemainingTime = RemainingTime;

			// If there are still batches left from the previous frame or a new pass was just started, then collect invalid rows.
			if (Info.RemainingJobs > 0)
			{
				int32 NumBatches = FMath::Min(Info.RemainingJobs, Info.MaxNumJobs);
				ParallelForTemplate(NumBatches, [this, &InDataStorage](int32 Block)
					{
						InspectRowBlockForCleanUp(InDataStorage, Block);
					}, EParallelForFlags::Unbalanced);

				Info.RemainingJobs -= NumBatches;

				// Follow up with any adjustments needed.
				DomainRemainingTime -= FTimespan(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - DomainStartTime)));
				if (NumBatches == Info.MaxNumJobs) // Only adjust if a full set of batches was used to avoid skewing.
				{
					AdjustJobCount(DomainRemainingTime);
				}
				if (Info.RemainingJobs == 0) // Don't adjust the batches when there are more to be run.
				{
					AdjustBatchSize(Info.JobCount);
				}
			}

			// If there's time left and there are rows to remove, start removing rows.
			if (Info.RemainingJobs == 0 && DomainRemainingTime > FTimespan::Zero() && !Info.DeletionQueue.IsEmpty())
			{
				DrainDeletionQueue(DomainRemainingTime);
			}

			UE_LOGF(LogEditorDataStorage, Verbose,
				"Mapping cleanup - %7.2fms - Domain %ls has%lsremaining rows, Batch size: %i, Job count: %i, Remaining jobs: %i, Max batches: %i",
				FTimespan(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - DomainStartTime))).GetTotalMilliseconds(),
				*DomainName.ToString(),
				Info.DeletionQueue.IsEmpty() ? TEXT(" no ") : TEXT(" "),
				Info.BatchSize,
				Info.JobCount,
				Info.RemainingJobs,
				Info.MaxNumJobs);

			// See if there's time left for another pass for the next domain.
			RemainingTime -= FTimespan(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - DomainStartTime)));
			
			// If there are no more remaining jobs and nothing queued for deletion that the domain has been fully cleaned up and should return false.
			return (Info.RemainingJobs > 0 || !Info.DeletionQueue.IsEmpty()) 
				? ERemoveInvalidRowsResult::HasRemainingWork 
				: ERemoveInvalidRowsResult::CompletedWork;
		}
		else
		{
			return ERemoveInvalidRowsResult::NoWork;
		}
	}

	void FMappingTable::RemoveInvalidRows(FTimespan RemainingTime)
	{
		TEDS_EVENT_SCOPE("Mapping clean up");

		FScopedExclusiveLock Lock(EGlobalLockScope::Public);

		// If there's no work from a previous pass, then start a new pass if needed.
		if (RemainingUnverifiedDomains == 0)
		{
			if (DirtyDueToRemoval.load() && CleanUpFrameDelay-- == 0)
			{
				DirtyDueToRemoval = false;
				RemainingUnverifiedDomains = Domains.Num();

				// Setup all domains for cleanup.
				for (DomainMap::ElementType& It : Domains)
				{
					It.Value.CleanUpInfo->JobCount = (It.Value.Rows.Num() / It.Value.CleanUpInfo->BatchSize) + 1;
					It.Value.CleanUpInfo->RemainingJobs = It.Value.CleanUpInfo->JobCount;
				}
			}
			return;
		}

		if (RemainingUnverifiedDomains > 0)
		{
			for (DomainMap::ElementType& It : Domains)
			{
				switch (It.Value.RemoveInvalidRows(It.Key, DataStorage, RemainingTime))
				{
				case FDomain::ERemoveInvalidRowsResult::NoWork:
					continue;
				case FDomain::ERemoveInvalidRowsResult::CompletedWork:
					RemainingUnverifiedDomains--;
					// Fall through.
				case FDomain::ERemoveInvalidRowsResult::HasRemainingWork:
					if (RemainingTime <= 0)
					{
						return;
					}
				}
			}
		}
	}

	bool FMappingTable::IsDirty() const
	{
		return RemainingUnverifiedDomains > 0 || DirtyDueToRemoval;
	}


	//
	// FCollisionSafeHashMap
	//

	
	template<typename KeyType>
	int32* FMappingTable::FCollisionSafeHashMap::FindTemplate(uint64 Hash, const KeyType& Key, const TArray<FMapKey>& Keys)
	{
		if (int32* Index = Map.Find(Hash))
		{
			if (*Index != HashCollisionId)
			{
				return Index;
			}
			else
			{
				for (IndexLookupMultiMapType::TKeyIterator It = MapBackup.CreateKeyIterator(Hash); It; ++It)
				{
					if (Keys[It->Value] == Key)
					{
						return &(It->Value);
					}
				}
			}
		}
		return nullptr;
	}

	const int32* FMappingTable::FCollisionSafeHashMap::Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys) const
	{
		return const_cast<FMappingTable::FCollisionSafeHashMap*>(this)->FindTemplate(Hash, Key, Keys);
	}

	const int32* FMappingTable::FCollisionSafeHashMap::Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys) const
	{
		return const_cast<FMappingTable::FCollisionSafeHashMap*>(this)->FindTemplate(Hash, Key, Keys);
	}

	int32* FMappingTable::FCollisionSafeHashMap::Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys)
	{
		return FindTemplate(Hash, Key, Keys);
	}

	int32* FMappingTable::FCollisionSafeHashMap::Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys)
	{
		return FindTemplate(Hash, Key, Keys);
	}

	void FMappingTable::FCollisionSafeHashMap::ReserveAdditional(int32 Count)
	{
		// Collisions are rare so assume that all requested entries go into the main map.
		Map.Reserve(Map.Num() + Count);
	}

	void FMappingTable::FCollisionSafeHashMap::Add(uint64 Hash, int32 Index)
	{
		int32* StoredIndex = Map.Find(Hash);
		if (StoredIndex == nullptr)
		{
			Map.Add(Hash, Index);
		}
		else
		{
			MapBackup.Add(Hash, Index);
			// If this is the first conflict, move the index from the main map into the backup map and
			// leave a breadcrumb to indicate the backup map needs to be searched.
			if (*StoredIndex != HashCollisionId)
			{
				MapBackup.Add(Hash, *StoredIndex);
				*StoredIndex = HashCollisionId;
			}
		}
	}

	template<typename KeyType>
	int32 FMappingTable::FCollisionSafeHashMap::RemoveTemplate(uint64 Hash, const KeyType& Key, const TArray<FMapKey>& Keys)
	{
		if (IndexLookupMapType::TKeyIterator It = Map.CreateKeyIterator(Hash))
		{
			if (It->Value != HashCollisionId)
			{
				int32 Index = It->Value;
				It.RemoveCurrent();
				return Index;
			}
			else
			{
				int32 Index = InvalidId;
				int32 EntriesLeft = 0;
				for (IndexLookupMultiMapType::TKeyIterator ItBackup = MapBackup.CreateKeyIterator(Hash);
					ItBackup; ++ItBackup)
				{
					if (Keys[ItBackup->Value] == Key)
					{
						Index = ItBackup->Value;
						ItBackup.RemoveCurrent();
					}
					else
					{
						++EntriesLeft;
					}
				}
				// If there's exactly one key left there are no more collisions so switch back to the main map instead of the backup
				// multi-map.
				checkf(EntriesLeft > 0, TEXT("Only one entry can be removed and there should never be just one entry in the backup map."));
				if (EntriesLeft == 1)
				{
					It->Value = MapBackup.FindChecked(Hash);
					MapBackup.Remove(Hash);
				}
				return Index;
			}
		}
		return InvalidId;
	}

	int32 FMappingTable::FCollisionSafeHashMap::Remove(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys)
	{
		return RemoveTemplate(Hash, Key, Keys);
	}

	int32 FMappingTable::FCollisionSafeHashMap::Remove(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys)
	{
		return RemoveTemplate(Hash, Key, Keys);
	}

	void FMappingTable::FCollisionSafeHashMap::Remove(uint64 Hash, int32 Index)
	{
		if (IndexLookupMapType::TKeyIterator It = Map.CreateKeyIterator(Hash))
		{
			if (It->Value != HashCollisionId)
			{
				It.RemoveCurrent();
			}
			else
			{
				int32 EntriesLeft = 0;
				for (IndexLookupMultiMapType::TKeyIterator ItBackup = MapBackup.CreateKeyIterator(Hash);
					ItBackup; ++ItBackup)
				{
					if (ItBackup->Value == Index)
					{
						ItBackup.RemoveCurrent();
					}
					else
					{
						++EntriesLeft;
					}
				}
				// If there's exactly one key left there are no more collisions so switch back to the main map instead of the backup
				// multi-map.
				checkf(EntriesLeft > 0, TEXT("Only one entry can be removed and there should never be just one entry in the backup map."));
				if (EntriesLeft == 1)
				{
					It->Value = MapBackup.FindChecked(Hash);
					MapBackup.Remove(Hash);
				}
			}
		}
	}


	// 
	// FShardedMap
	//

	int32 FMappingTable::FShardedMap::GetShard(uint64 Hash) const
	{
		return Hash & ((1 << ShardBits) - 1);
	}

	uint64 FMappingTable::FShardedMap::ModifyHash(uint64 Hash) const
	{
		return Hash >> ShardBits;
	}

	const int32* FMappingTable::FShardedMap::Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys) const
	{
		return Shards[GetShard(Hash)].Find(ModifyHash(Hash), Key, Keys);
	}

	const int32* FMappingTable::FShardedMap::Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys) const
	{
		return Shards[GetShard(Hash)].Find(ModifyHash(Hash), Key, Keys);
	}

	int32* FMappingTable::FShardedMap::Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys)
	{
		return Shards[GetShard(Hash)].Find(ModifyHash(Hash), Key, Keys);
	}

	int32* FMappingTable::FShardedMap::Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys)
	{
		return Shards[GetShard(Hash)].Find(ModifyHash(Hash), Key, Keys);
	}

	void FMappingTable::FShardedMap::ReserveAdditional(TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs)
	{
		int32 Sizes[1 << ShardBits] = {};

		for (TPair<FMapKey, RowHandle>& It : MapRowPairs)
		{
			Sizes[GetShard(It.Key.CalculateHash())]++;
		}

		for (int32 ShardSizeIndex = 0; ShardSizeIndex < (1 << ShardBits); ++ShardSizeIndex)
		{
			int32 Count = Sizes[ShardSizeIndex];
			if (Count > 0)
			{
				Shards[ShardSizeIndex].ReserveAdditional(Count);
			}
		}
	}

	void FMappingTable::FShardedMap::Add(uint64 Hash, int32 Index)
	{
		Shards[GetShard(Hash)].Add(ModifyHash(Hash), Index);
	}

	int32 FMappingTable::FShardedMap::Remove(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys)
	{
		return Shards[GetShard(Hash)].Remove(ModifyHash(Hash), Key, Keys);
	}

	int32 FMappingTable::FShardedMap::Remove(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys)
	{
		return Shards[GetShard(Hash)].Remove(ModifyHash(Hash), Key, Keys);
	}

	void FMappingTable::FShardedMap::Remove(uint64 Hash, int32 Index)
	{
		return Shards[GetShard(Hash)].Remove(ModifyHash(Hash), Index);
	}


	//
	// FDomain
	//


	int32 FMappingTable::FDomain::FindIndexUnguarded(const FMapKey& Key) const
	{
		const int32* Result = IndexLookupMap.Find(Key.CalculateHash(), Key, Keys);
		return Result ? *Result : InvalidId;
	}

	int32 FMappingTable::FDomain::FindIndexUnguarded(uint64 Hash, const FMapKey& Key) const
	{
		const int32* Result = IndexLookupMap.Find(Hash, Key, Keys);
		return Result ? *Result : InvalidId;
	}

	int32 FMappingTable::FDomain::FindIndexUnguarded(const FMapKeyView& Key) const
	{
		const int32* Result = IndexLookupMap.Find(Key.CalculateHash(), Key, Keys);
		return Result ? *Result : InvalidId;
	}

	int32 FMappingTable::FDomain::FindIndexUnguarded(uint64 Hash, const FMapKeyView& Key) const
	{
		const int32* Result = IndexLookupMap.Find(Hash, Key, Keys);
		return Result ? *Result : InvalidId;
	}

	void FMappingTable::FDomain::IndexRowUnguarded(UEditorDataStorage& InDataStorage, FMapKey&& Key, RowHandle Row, bool bIsDirty)
	{
		uint64 Hash = Key.CalculateHash();
		if (int32 Index = FindIndexUnguarded(Hash, Key); Index >= 0)
		{
			bool bUpdatingAllowed = bIsDirty && !InDataStorage.IsRowAvailableUnsafe(Rows[Index]);
			if (ensureMsgf(bUpdatingAllowed || Row == Rows[Index], TEXT("Another row has already been registered under key '%s'."), *Key.ToString()))
			{
				Rows[Index] = Row; // Update the stored row to the new row.
				return;
			}
		}
		
		// There's no existing row stored under the given key, so create a new one.
		int32 RowIndex;
		if (FreeList.IsEmpty())
		{
			RowIndex = Rows.Add(Row);
			Keys.Add(MoveTemp(Key));
		}
		else
		{
			RowIndex = FreeList.Last();
			FreeList.PopLast();
			Rows[RowIndex] = Row;
			Keys[RowIndex] = MoveTemp(Key);
		}
		
		IndexLookupMap.Add(Hash, RowIndex);
	}

	int32 FMappingTable::FDomain::RemoveUnguarded(const FMapKeyView& Key)
	{
		uint64 Hash = Key.CalculateHash();
		if (int32 Index = IndexLookupMap.Remove(Hash, Key, Keys); Index != InvalidId)
		{
			Rows[Index] = InvalidRowHandle;
			Keys[Index].Clear();
			FreeList.PushFirst(Index);
			return Index;
		}
		return InvalidId;
	}

	void FMappingTable::FDomain::RemapUnguarded(UEditorDataStorage& InDataStorage, const FMapKeyView& OriginalKey, FMapKey NewKey, bool bIsDirty)
	{
		uint64 Hash = OriginalKey.CalculateHash();
		if (int32 Index = IndexLookupMap.Remove(Hash, OriginalKey, Keys); Index != InvalidId)
		{
			RowHandle StoredRow = Rows[Index];
			Rows[Index] = InvalidRowHandle;
			Keys[Index].Clear();
			FreeList.PushFirst(Index);
			IndexRowUnguarded(InDataStorage, MoveTemp(NewKey), StoredRow, bIsDirty);
		}
		else
		{
			checkf(false, TEXT("Unable to remove key '%s' from TEDS Mapping."), *OriginalKey.ToString());
		}
	}

	void FMappingTable::FDomain::InspectRowBlockForCleanUp(UEditorDataStorage& InDataStorage, int32 Block)
	{
		TEDS_EVENT_SCOPE("Mapping inspect rows");

		int32 Index = ((CleanUpInfo->JobCount - CleanUpInfo->RemainingJobs + Block) * CleanUpInfo->BatchSize);

		const RowHandle* Begin = Rows.GetData();
		const RowHandle* Front = Rows.GetData() + Index;
		const RowHandle* End = FMath::Min(Begin + Rows.Num(), Front + CleanUpInfo->BatchSize);
		bool bIsFullBatch = (End - Front) == CleanUpInfo->BatchSize;

		uint64 StartTime = FPlatformTime::Cycles64();
		
		FCleanUpInfo::InvalidRowContainer Result;
		for (; Front != End; ++Front)
		{
			// Skip rows with an invalid handle as those are already freed.
			if (*Front != InvalidRowHandle && !InDataStorage.IsRowAvailableUnsafe(*Front))
			{
				Result.Add(static_cast<int32>(Front - Begin));
			}
		}

		if (!Result.IsEmpty())
		{
			CleanUpInfo->DeletionQueue.Enqueue(MoveTemp(Result));
		}

		// Only adjust if this is a full batch, otherwise partial batches skew performance stats.
		if (bIsFullBatch)
		{
			FTimespan Duration(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime)));
			if (Duration >= FCleanUpInfo::MaxBatchDuration)
			{
				++CleanUpInfo->BatchWentOverTime;
			}
			else if (Duration <= FCleanUpInfo::MinBatchDuration)
			{
				++CleanUpInfo->BatchWentUnderTime;
			}
		}
	}

	void FMappingTable::FDomain::DrainDeletionQueue(FTimespan RemainingFrameTime)
	{
		TEDS_EVENT_SCOPE("Mapping drain deletion queue");

		uint64 StartTime = FPlatformTime::Cycles64();

		while (!CleanUpInfo->DeletionQueue.IsEmpty() && 
			FTimespan(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime))) < RemainingFrameTime)
		{
			TOptional<FCleanUpInfo::InvalidRowContainer> OptionalContainer = CleanUpInfo->DeletionQueue.Dequeue();
			checkf(OptionalContainer.IsSet(),
				TEXT("Retrieved an invalid row container from a non-empty deletion queue, but no result was returned"));
			
			FCleanUpInfo::InvalidRowContainer& Container = OptionalContainer.GetValue();
			for (int32 RowIndex : Container)
			{
				ClearRow(RowIndex);
			}
		}
	}

	void FMappingTable::FDomain::ClearRow(int32 Index)
	{
		FMapKey& Key = Keys[Index];
		uint64 Hash = Key.CalculateHash();
		
		IndexLookupMap.Remove(Hash, Index);
		
		Rows[Index] = InvalidRowHandle;
		Keys[Index].Clear();
		FreeList.PushFirst(Index);
	}

	void FMappingTable::FDomain::AdjustJobCount(FTimespan RemainingFrameTime)
	{
		if (RemainingFrameTime >= FCleanUpInfo::JobGrowthThreshold)
		{
			CleanUpInfo->MaxNumJobs = FMath::RoundToInt32(static_cast<float>(CleanUpInfo->MaxNumJobs) * FCleanUpInfo::JobGrowthFactor);
		}
		else if (RemainingFrameTime <= FCleanUpInfo::JobShrinkThreshold)
		{
			CleanUpInfo->MaxNumJobs = FMath::RoundToInt32(static_cast<float>(CleanUpInfo->MaxNumJobs) * FCleanUpInfo::JobShrinkFactor);
		}

		CleanUpInfo->MaxNumJobs = FMath::Clamp(CleanUpInfo->MaxNumJobs, FCleanUpInfo::MinJobCount, FCleanUpInfo::MaxJobCount);
	}

	void FMappingTable::FDomain::AdjustBatchSize(int32 JobCount)
	{
		if (CleanUpInfo->BatchWentOverTime.load() >= FCleanUpInfo::BatchShrinkThreshold)
		{
			// The batch size was too big to complete in the allotted time so take a sizable chunk off.
			CleanUpInfo->BatchSize = FMath::RoundToInt32(static_cast<float>(CleanUpInfo->BatchSize) * FCleanUpInfo::BatchShrinkFactor);
		}
		// Check if more than the minimum percentage of jobs took less time than currently allocated.
		else if (CleanUpInfo->BatchWentUnderTime.load() >=
			static_cast<uint32>(FMath::RoundToInt32(static_cast<float>(JobCount) * FCleanUpInfo::BatchIncreaseThreshold)))
		{
			// The batch size fitted within the allotted time so slightly increase it.
			CleanUpInfo->BatchSize = FMath::RoundToInt32(static_cast<float>(CleanUpInfo->BatchSize) * FCleanUpInfo::BatchGrowthFactor);
		}
		// Clamp the job batch size within reasonable sizes to avoid extremes.
		CleanUpInfo->BatchSize = FMath::Clamp(CleanUpInfo->BatchSize, FCleanUpInfo::MinBatchSize, FCleanUpInfo::MaxBatchSize);
		CleanUpInfo->BatchWentOverTime = 0;
		CleanUpInfo->BatchWentUnderTime = 0;
	}
} // namespace UE::Editor::DataStorage
