// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Deque.h"
#include "Containers/MpscQueue.h"
#include "DataStorage/Handles.h"
#include "DataStorage/MapKey.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "GlobalLock.h"
#include "Misc/Timespan.h"
#include "Templates/UniquePtr.h"

class UEditorDataStorage;

namespace UE::Editor::DataStorage
{
	/**
	 * Storage for a key to row mapping.
	 * Access to the mapping table is thread safe and guarded by the global lock.
	 */

	class FMappingTable final
	{
	public:
		explicit FMappingTable(UEditorDataStorage& DataStorage);
		~FMappingTable();

		RowHandle Lookup(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& Key) const;
		void Map(EGlobalLockScope LockScope, const FName& Domain, FMapKey Key, RowHandle Row);
		void BatchMap(EGlobalLockScope LockScope, const FName& Domain, TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs);
		void Remap(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& OriginalKey, FMapKey NewKey);
		void Remove(EGlobalLockScope LockScope, const FName& Domain, const FMapKeyView& Key);

		void MarkDirty();

	private:
		struct FCleanUpInfo final
		{
			using InvalidRowContainer = TArray<int32>;

			// If less than this time is available after running jobs in a frame, reduce the number of jobs.
			static const FTimespan JobShrinkThreshold;
			// If more than this time is left per frame, increase the number of jobs.
			static const FTimespan JobGrowthThreshold;
			// The rate at which to grow the number of jobs run per frame.
			static constexpr float JobShrinkFactor = 0.7f;
			// The rate at which to shrink the number of jobs run per frame.
			static constexpr float JobGrowthFactor = 1.2f;
			// The minimum required number of jobs.
			static constexpr int32 MinJobCount = 4;
			// The maximum number of allowed jobs.
			static constexpr int32 MaxJobCount = 1024;
			
			// If enough batches exceed this limit, the batch size for jobs is reduced. Aim for 4 batches per thread. Increasing this number
			// increases the spikes that can happen when batches are too big. Decreasing this number increases the overhead of scheduling jobs
			// and makes it more expensive overall.
			static const FTimespan MaxBatchDuration;
			// If enough batches are under this limit, the batch size for jobs is increased.
			static const FTimespan MinBatchDuration;
			// The absolute minimum number of rows to check for validity per batch.
			static constexpr int32 MinBatchSize = 1000;
			// The absolute maximum number of rows to check for validity per batch.
			static constexpr int32 MaxBatchSize = 100'000;
			// The number of jobs that need to take longer than MaxBatchDuration before the batch size gets shrunken down. This is
			// used to mitigate the occasional spike due to for instance the OS handing off work to other programs.
			static constexpr uint32 BatchShrinkThreshold = 2;
			// The portion of jobs that need to be below the MinBatchDuration threshold before the batch size gets increased.
			static constexpr float BatchIncreaseThreshold = 0.9f;
			// The rate at which the batch size gets reduced if enough batches went over time.
			static constexpr float BatchShrinkFactor = 0.8f;
			// The rate at which the batch size gets increased if there's time left.
			static constexpr float BatchGrowthFactor = 1.1f;

			// Queue containing arrays with removed row indices.
			TMpscQueue<InvalidRowContainer> DeletionQueue;
			
			// The maximum number of rows that a single job will process.
			int32 BatchSize = 4096;
			// The number of jobs that had too many batches to process within MaxBatchDuration.
			std::atomic<uint32> BatchWentOverTime = 0;
			// The number of jobs that processed their batches faster than MinBatchDuration.
			std::atomic<uint32> BatchWentUnderTime = 0;
			
			// The maximum number of jobs to run per frame.
			int32 MaxNumJobs = 8;
			// The total number of jobs needed to inspect all indexed rows.
			int32 JobCount = 0;
			// The number of jobs that were not completed in the previous frame.
			int32 RemainingJobs = 0;
		};

		static constexpr int32 InvalidId = -1;
		static constexpr int32 HashCollisionId = -2;

		struct FCollisionSafeHashMap
		{
			using IndexLookupMultiMapType = TMultiMap<uint64, int32>;
			using IndexLookupMapType = TMap<uint64, int32>;

			IndexLookupMultiMapType MapBackup; // Backup map in case there's a valid hash collision.
			IndexLookupMapType Map;

			const int32* Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys) const;
			const int32* Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys) const;
			int32* Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys);
			int32* Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys);

			void ReserveAdditional(int32 Count);
			void Add(uint64 Hash, int32 Index);
			int32 Remove(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys);
			int32 Remove(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys);
			void Remove(uint64 Hash, int32 Index);

		private:
			template<typename KeyType>
			int32* FindTemplate(uint64 Hash, const KeyType& Key, const TArray<FMapKey>& Keys);
			template<typename KeyType>
			int32 RemoveTemplate(uint64 Hash, const KeyType& Key, const TArray<FMapKey>& Keys);
		};

		struct FShardedMap final
		{
			static constexpr int32 ShardBits = 8;

			FCollisionSafeHashMap Shards[1 << ShardBits];

			const int32* Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys) const;
			const int32* Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys) const;
			int32* Find(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys);
			int32* Find(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys);

			void ReserveAdditional(TArrayView<TPair<FMapKey, RowHandle>> MapRowPairs);
			void Add(uint64 Hash, int32 Index);
			int32 Remove(uint64 Hash, const FMapKey& Key, const TArray<FMapKey>& Keys);
			int32 Remove(uint64 Hash, const FMapKeyView& Key, const TArray<FMapKey>& Keys);
			void Remove(uint64 Hash, int32 Index);

		private:
			int32 GetShard(uint64 Hash) const;
			uint64 ModifyHash(uint64 Hash) const;
		};
		
		struct FDomain final
		{
			FShardedMap IndexLookupMap;
			TArray<RowHandle> Rows;
			TArray<FMapKey> Keys;
			TDeque<int32> FreeList;

			TUniquePtr<FCleanUpInfo> CleanUpInfo = MakeUnique<FCleanUpInfo>();

			int32 FindIndexUnguarded(const FMapKey& Key) const;
			int32 FindIndexUnguarded(uint64 Hash, const FMapKey& Key) const;
			int32 FindIndexUnguarded(const FMapKeyView& Key) const;
			int32 FindIndexUnguarded(uint64 Hash, const FMapKeyView& Key) const;

			void IndexRowUnguarded(UEditorDataStorage& DataStorage, FMapKey&& Key, RowHandle Row, bool bIsDirty);
			int32 RemoveUnguarded(const FMapKeyView& Key);
			void RemapUnguarded(UEditorDataStorage& InDataStorage, const FMapKeyView& OriginalKey, FMapKey NewKey, bool bIsDirty);

			enum class ERemoveInvalidRowsResult
			{
				NoWork,
				HasRemainingWork,
				CompletedWork
			};
			ERemoveInvalidRowsResult RemoveInvalidRows(const FName& DomainName, UEditorDataStorage& DataStorage, FTimespan& RemainingTime);

			void InspectRowBlockForCleanUp(UEditorDataStorage& DataStorage, int32 Block);
			void DrainDeletionQueue(FTimespan RemainingFrameTime);
			void ClearRow(int32 Index);
			void AdjustJobCount(FTimespan RemainingFrameTime);
			void AdjustBatchSize(int32 JobCount);
		};
		using DomainMap = TMap<FName, FDomain>;
		DomainMap Domains;

		int32 RemainingUnverifiedDomains = 0;
		// The number of frames to wait to start the cleanup. This delay is added as several operations in the editor do a delete spread out of over
		// a few frames.
		static constexpr int8 DefaultCleanUpFrameDelay = 5;
		std::atomic<int8> CleanUpFrameDelay = 0;
		std::atomic<bool> DirtyDueToRemoval = false;
		
		UEditorDataStorage& DataStorage;
		
		void RemoveInvalidRows(FTimespan RemainingTime); 
		bool IsDirty() const;
	};
} // namespace UE::Editor::DataStorage
