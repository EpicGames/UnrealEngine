// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseScratchBuffer.h"

namespace UE::Editor::DataStorage
{
	FScratchBuffer::FScratchBuffer()
		: CurrentAllocator(&Allocators[0])
		, PreviousAllocator(&Allocators[1])
		, LeastRecentAllocator(&Allocators[2])
	{
	}

	void* FScratchBuffer::AllocateUninitialized(SIZE_T Size, uint32 Alignment)
	{
		return CurrentAllocator.load()->Malloc(Size, Alignment);
	}

	void* FScratchBuffer::AllocateZeroInitialized(SIZE_T Size, uint32 Alignment)
	{
		return CurrentAllocator.load()->MallocAndMemset(Size, Alignment, 0);
	}

	void FScratchBuffer::BatchDelete()
	{
		MemoryAllocator* Current = CurrentAllocator.exchange(LeastRecentAllocator);
		LeastRecentAllocator = PreviousAllocator;
		PreviousAllocator = Current;
		LeastRecentAllocator->BulkDelete();
	}
} // namespace UE::Editor::DataStorage
