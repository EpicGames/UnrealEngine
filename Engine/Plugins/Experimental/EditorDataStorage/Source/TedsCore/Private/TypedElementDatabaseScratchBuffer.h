// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include <type_traits>
#include "Experimental/ConcurrentLinearAllocator.h"

// A thread-safe memory allocator that uses linear allocation. This provides a fast and lightweight way to allocate temporary memory 
// for intermediate values that will live at best until the end of the frame.
namespace UE::Editor::DataStorage
{
	class FScratchBuffer
	{
	private:
		struct FAllocationTag
		{
			static constexpr uint32 BlockSize = 64 * 1024;		// Blocksize used to allocate from
			static constexpr bool AllowOversizedBlocks = true;  // The allocator supports oversized Blocks and will store them in a separate Block with counter 1
			static constexpr bool RequiresAccurateSize = false;  // GetAllocationSize returning the accurate size of the allocation otherwise it could be relaxed to return the size to the end of the Block
			static constexpr bool InlineBlockAllocation = true;  // Inline or Noinline the BlockAllocation which can have an impact on Performance
			static constexpr const char* TagName = "TedsScratchBuffer";
	
			using Allocator = TBlockAllocationLockFreeCache<BlockSize, FAlignedAllocator>;
		};
		using MemoryAllocator = TConcurrentLinearBulkObjectAllocator<FAllocationTag>;
	
	public:
		/** Whether or not functions that do not handle destruction of typed objects allow classes with non-trivial destructor. */
		enum class ERequiresTrivialDestructor : bool
		{
			Yes,
			No
		};

		FScratchBuffer();
	
		/** Allocate uninitialized memory. */
		void* AllocateUninitialized(SIZE_T Size, uint32 Alignment);

		/** 
		 * Allocates memory for a single object but does not initialize the memory. Memory is only reserved and the destructor 
		 * will not be called.
		 */
		template<typename T, ERequiresTrivialDestructor RequiresTrivialDestructors = ERequiresTrivialDestructor::Yes>
		T* AllocateUninitialized();
		
		/** 
		 * Allocates memory for an array of object but does not initialize the memory. Memory is only reserved and the 
		 * destructor will not be called.
		 */
		template<typename T, ERequiresTrivialDestructor RequiresTrivialDestructors = ERequiresTrivialDestructor::Yes>
		TArrayView<T> AllocateUninitializedArray(int32 Count);

		/** Allocate memory and sets it to zero. */
		void* AllocateZeroInitialized(SIZE_T Size, uint32 Alignment);

		/** 
		 * Allocates memory for a single object and sets the memory to zero. This is not the same as calling a constructor, which can be 
		 * done using Emplace. Memory is only reserved and the destructor will not be called.
		 */
		template<typename T, ERequiresTrivialDestructor RequiresTrivialDestructors = ERequiresTrivialDestructor::Yes>
		T* AllocateZeroInitialized();

		/** 
		 * Allocates memory for an array of object and sets the memory to zero. This is not the same as calling a constructor, which can be 
		 * done using Emplace. Memory is only reserved and the destructor will not be called.
		 */
		template<typename T, ERequiresTrivialDestructor RequiresTrivialDestructors = ERequiresTrivialDestructor::Yes>
		TArrayView<T> AllocateZeroInitializedArray(int32 Count);

		/** 
		 * Allocate memory for one object and construct it with the provided arguments. The object created by this call will
		 * has its constructor and destructor called.
		 */
		template<typename T, typename... ArgTypes>
		T* Emplace(ArgTypes... Args);

		/** 
		 * Allocate memory for an array of objects and initialize each object with the provided arguments. Objects created by this 
		 * call will have their constructor and destructor called.
		 */
		template<typename T, typename... ArgTypes>
		TArrayView<T> EmplaceArray(int32 Count, const ArgTypes&... Args);

	
		// Activates a new allocator and deletes all commands and objects in the least recently touched scratch buffer.
		void BatchDelete();
	constexpr static int32 MaxAllocationSize();
	
	private:
		// Using a triple buffered approach because the direct API in TEDS (those calls that can be made directly to the API and don't go
		// through a context) are not required to be atomic. As such it's possible that data for a command is stored in allocator A while
		// the command is in allocator B if those calls are issued while TEDS is closing it's processing cycle. With double buffering this
		// would result in allocator A being flushed thus clearing out the data for the command. Using a triple buffered approach will
		// cause the clearing to be delayed by a frame, avoid this problem. This however does assume that all data and command issuing happens
		// within a single tick, though for the direct API this should always be true.
	
		MemoryAllocator Allocators[3];
		std::atomic<MemoryAllocator*> CurrentAllocator;
		MemoryAllocator* PreviousAllocator;
		MemoryAllocator* LeastRecentAllocator;
	};
	
	template<typename T, FScratchBuffer::ERequiresTrivialDestructor RequiresTrivialDestructors>
	T* FScratchBuffer::AllocateUninitialized()
	{
		if constexpr (RequiresTrivialDestructors == ERequiresTrivialDestructor::Yes)
		{
			static_assert(std::is_trivially_destructible_v<T>, 
				"Scratch buffer allocator requires a trivially destructible class type or to be explicitly told it's safe to construct.");
		}
		return CurrentAllocator.load()->Malloc<T>();
	}

	template<typename T, FScratchBuffer::ERequiresTrivialDestructor RequiresTrivialDestructors>
	TArrayView<T> FScratchBuffer::AllocateUninitializedArray(int32 Count)
	{
		if constexpr (RequiresTrivialDestructors == ERequiresTrivialDestructor::Yes)
		{
			static_assert(std::is_trivially_destructible_v<T>,
				"Scratch buffer allocator requires a trivially destructible class type or to be explicitly told it's safe to construct.");
		}
		T* Result = CurrentAllocator.load()->MallocArray<T>(Count);
		return TArrayView<T>(Result, Result ? Count : 0);
	}

	template<typename T, FScratchBuffer::ERequiresTrivialDestructor RequiresTrivialDestructors>
	T* FScratchBuffer::AllocateZeroInitialized()
	{
		if constexpr (RequiresTrivialDestructors == ERequiresTrivialDestructor::Yes)
		{
			static_assert(std::is_trivially_destructible_v<T>,
				"Scratch buffer allocator requires a trivially destructible class type or to be explicitly told it's safe to construct.");
		}
		return CurrentAllocator.load()->MallocAndMemset<T>(0);
	}

	template<typename T, FScratchBuffer::ERequiresTrivialDestructor RequiresTrivialDestructors>
	TArrayView<T> FScratchBuffer::AllocateZeroInitializedArray(int32 Count)
	{
		if constexpr (RequiresTrivialDestructors == ERequiresTrivialDestructor::Yes)
		{
			static_assert(std::is_trivially_destructible_v<T>,
				"Scratch buffer allocator requires a trivially destructible class type or to be explicitly told it's safe to construct.");
		}
		T* Result = CurrentAllocator.load()->MallocAndMemsetArray<T>(Count, 0);
		return TArrayView<T>(Result, Result ? Count : 0);
	}

	template<typename T, typename... ArgTypes>
	T* FScratchBuffer::Emplace(ArgTypes... Args)
	{
		return CurrentAllocator.load()->Create<T>(Forward<ArgTypes>(Args)...);
	}
	
	template<typename T, typename... ArgTypes>
	TArrayView<T> FScratchBuffer::EmplaceArray(int32 Count, const ArgTypes&... Args)
	{
		T* Result = CurrentAllocator.load()->CreateArray<T>(Count, Args...);
		return TArrayView<T>(Result, Result ? Count : 0);
	}

	constexpr int32 FScratchBuffer::MaxAllocationSize()
	{
		return FAllocationTag::BlockSize;
	}
} // namespace UE::Editor::DataStorage
