// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementDatabaseUI.h"

#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Elements/Columns/DecoratorWidgetColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/WidgetPurposeColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementColumnUtils.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Interfaces/DecoratorWidgetConstructor.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "Widgets/SlateControlledConstruction.h"
#include "Widgets/STedsWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementDatabaseUI)

DEFINE_LOG_CATEGORY(LogEditorDataStorageUI);

#define LOCTEXT_NAMESPACE "TypedElementDatabaseUI"

namespace Internal
{
	// Source: https://en.cppreference.com/w/cpp/utility/variant/visit
	template<class... Ts>
	struct TOverloaded : Ts...
	{ 
		using Ts::operator()...; 
	};

	template<class... Ts> TOverloaded(Ts...) -> TOverloaded<Ts...>;

	// Check if the two columns are equal, or if InRequestedColumn is a dynamic specialization of InMatchedColumn
	bool CheckSingleColumnMatch(const UScriptStruct* InMatchedColumn, const UScriptStruct* InRequestedColumn)
	{
		if (InMatchedColumn == InRequestedColumn)
		{
			return true;
		}
		
		else if (UE::Editor::DataStorage::ColumnUtils::IsDynamicTemplate(InMatchedColumn) &&
			UE::Editor::DataStorage::ColumnUtils::IsDerivedFromDynamicTemplate(InRequestedColumn))
		{
			return InRequestedColumn->IsChildOf(InMatchedColumn);
		}

		return false;
	}

	// Custom deleter for a widget constructor since we allocate using FMemory::Malloc and then pass ownership to a TSharedPtr
	struct FWidgetConstructorDeleter
	{
		FWidgetConstructorDeleter(const UScriptStruct* InConstructorType)
			: ConstructorType(InConstructorType)
		{}
		
		void operator()(FTypedElementWidgetConstructor* Constructor) const
		{
			if (Constructor)
			{
				ConstructorType->DestroyStruct(Constructor);
				FMemory::Free(Constructor);
			}
		}

	private:
		const UScriptStruct* ConstructorType;
	};

	UE::Editor::DataStorage::IUiProvider::FWidgetConstructorPtr CreateWidgetConstructorInstance(const UScriptStruct* ConstructorType)
	{
		FTypedElementWidgetConstructor* ConstructorData = reinterpret_cast<FTypedElementWidgetConstructor*>(
			FMemory::Malloc(ConstructorType->GetStructureSize(), ConstructorType->GetMinAlignment()));
		
		ConstructorType->InitializeStruct(ConstructorData);
		
		return MakeShareable(ConstructorData, FWidgetConstructorDeleter(ConstructorType));
	}

	// Custom deleter for a decorator widget constructor since we allocate using FMemory::Malloc and then store in a TUniquePtr
	struct FDecoratorWidgetConstructorDeleter
	{
		FDecoratorWidgetConstructorDeleter(const UScriptStruct* InConstructorType)
			: ConstructorType(InConstructorType)
		{}
		
		void operator()(FTedsDecoratorWidgetConstructor* Constructor) const
		{
			if (Constructor)
			{
				ConstructorType->DestroyStruct(Constructor);
				FMemory::Free(Constructor);
			}
		}

	private:
		const UScriptStruct* ConstructorType;
	};

	TSharedPtr<FTedsDecoratorWidgetConstructor> CreateDecoratorWidgetConstructorInstance(const UScriptStruct* ConstructorType)
	{
		FTedsDecoratorWidgetConstructor* ConstructorData = reinterpret_cast<FTedsDecoratorWidgetConstructor*>(
			FMemory::Malloc(ConstructorType->GetStructureSize(), ConstructorType->GetMinAlignment()));
		
		ConstructorType->InitializeStruct(ConstructorData);

		return MakeShareable(ConstructorData, FDecoratorWidgetConstructorDeleter(ConstructorType));
	}

	static const UE::Editor::DataStorage::IUiProvider::FPurposeID DefaultWidgetPurposeID(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "Default").GeneratePurposeID());
	
	static const UE::Editor::DataStorage::IUiProvider::FPurposeID GeneralWidgetPurposeID(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", NAME_None).GeneratePurposeID());
	
	static FName WidgetPurposesHierarchyName("WidgetPurposesHierarchy");
	
}

void UEditorDataStorageUi::Initialize(
	UE::Editor::DataStorage::ICoreProvider* StorageInterface,
	UE::Editor::DataStorage::ICompatibilityProvider* StorageCompatibilityInterface)
{
	checkf(StorageInterface, TEXT("TEDS' compatibility manager is being initialized with an invalid storage target."));

	Storage = StorageInterface;
	StorageCompatibility = StorageCompatibilityInterface;
	CreateStandardArchetypes();
	RegisterQueries();
	RegisterHierarchies();

	Storage->OnUpdateCompleted().AddUObject(this, &UEditorDataStorageUi::PostDataStorageUpdate);
}

void UEditorDataStorageUi::PostInitialize(UE::Editor::DataStorage::ICoreProvider* StorageInterface,
	UE::Editor::DataStorage::ICompatibilityProvider* StorageCompatibilityInterface)
{
	// Now that all the default widget purposes have been fully initialized, we can try to resolve the parenting information manually
	// to ensure it is available for any tools that restored from the layout, which happens before TEDS has a chance to tick and resolve the
	// information through a processor
	Storage->ResolveHierarchy(WidgetPurposesHierarchyHandle);
}

void UEditorDataStorageUi::Deinitialize()
{
	Storage->OnUpdateCompleted().RemoveAll(this);
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageUi::RegisterWidgetPurpose(const FPurposeID& PurposeID, const FPurposeInfo& InPurposeInfo)
{
	using namespace UE::Editor::DataStorage;

	// If a purpose is already registered against this name, let the user know
	const FMapKey Key = FMapKey(PurposeID);
	RowHandle ExistingRow = Storage->LookupMappedRow(PurposeMappingDomain, Key);
	if (Storage->IsRowAvailable(ExistingRow))
	{
		ensureMsgf(false, TEXT("Existing purpose found registered with name: %s"), *PurposeID.ToString());
		return InvalidRowHandle;
	}

	// Add the row and register the mapping
	RowHandle PurposeRowHandle = Storage->AddRow(WidgetPurposeTable);
	Storage->MapRow(PurposeMappingDomain, Key, PurposeRowHandle);

	// Setup the relevant columns
	if (FWidgetPurposeColumn* PurposeColumn = Storage->GetColumn<FWidgetPurposeColumn>(PurposeRowHandle))
	{
		PurposeColumn->PurposeType = InPurposeInfo.Type;
		PurposeColumn->PurposeID = PurposeID;
	}

	if (FWidgetPurposeNameColumn* PurposeNameColumn = Storage->GetColumn<FWidgetPurposeNameColumn>(PurposeRowHandle))
	{
		PurposeNameColumn->Namespace = InPurposeInfo.Namespace;
		PurposeNameColumn->Name = InPurposeInfo.Name;
		PurposeNameColumn->Frame = InPurposeInfo.Frame;
	}

	if (!InPurposeInfo.Description.IsEmpty())
	{
		Storage->AddColumn(PurposeRowHandle, FDescriptionColumn{ .Description = InPurposeInfo.Description });
	}

	// If the parent purpose already exists, simply reference it. Otherwise add an unresolved parent column to resolve it later
	if (InPurposeInfo.ParentPurposeID.IsSet())
	{
		RowHandle ParentRowHandle = FindPurpose(InPurposeInfo.ParentPurposeID);

		if (Storage->IsRowAvailable(ParentRowHandle))
		{
			Storage->SetParentRow(WidgetPurposesHierarchyHandle, PurposeRowHandle, ParentRowHandle);
		}
		else
		{
			Storage->SetUnresolvedParent(WidgetPurposesHierarchyHandle, PurposeRowHandle, UE::Editor::DataStorage::FMapKey(InPurposeInfo.ParentPurposeID), PurposeMappingDomain);
		}
	}
	return PurposeRowHandle;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageUi::RegisterWidgetPurpose(const FPurposeInfo& InPurposeInfo)
{
	return RegisterWidgetPurpose(InPurposeInfo.GeneratePurposeID(), InPurposeInfo);
}

bool UEditorDataStorageUi::RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor)
{
	checkf(Constructor->IsChildOf(FTypedElementWidgetConstructor::StaticStruct()),
		TEXT("Attempting to register a widget constructor '%s' that isn't derived from FTypedElementWidgetConstructor."),
		*Constructor->GetFullName());
	
	if (FWidgetPurposeColumn* PurposeInfo = Storage->GetColumn<FWidgetPurposeColumn>(PurposeRow))
	{
		switch (PurposeInfo->PurposeType)
		{
		case EPurposeType::Generic:
			{
				UE::Editor::DataStorage::RowHandle FactoryRow = RegisterWidgetFactoryRow(PurposeRow);
				Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
				return true;
			}
		case EPurposeType::UniqueByName:
			{
				UE::Editor::DataStorage::RowHandle FactoryRow = RegisterUniqueWidgetFactoryRow(PurposeRow);
				Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
				return true;
			}
		case EPurposeType::UniqueByNameAndColumn:
			{
				UE_LOGF(LogEditorDataStorageUI, Warning,
				"Unable to register widget factory '%ls' as purpose '%llu' requires at least one column for matching.", 
				*Constructor->GetName(), PurposeRow);
				return false;
			}
		default:
			{
				checkf(false, TEXT("Unexpected UE::Editor::DataStorage::IUiProvider::EPurposeType found provided when registering widget factory."));
				return false;
			}
		}
	}
	
	UE_LOGF(LogEditorDataStorageUI, Warning, 
				"Unable to register widget factory '%ls' as purpose '%llu' isn't registered.", *Constructor->GetName(), PurposeRow);
	return false;
}

bool UEditorDataStorageUi::RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor,
	UE::Editor::DataStorage::Queries::FConditions Columns)
{
	if (!Columns.IsEmpty())
	{
		checkf(Constructor->IsChildOf(FTypedElementWidgetConstructor::StaticStruct()),
			TEXT("Attempting to register a widget constructor '%s' that isn't deriving from FTypedElementWidgetConstructor."),
			*Constructor->GetFullName());
		
		if (FWidgetPurposeColumn* PurposeInfo = Storage->GetColumn<FWidgetPurposeColumn>(PurposeRow))
		{
			switch (PurposeInfo->PurposeType)
			{
			case EPurposeType::Generic:
				{
					UE::Editor::DataStorage::RowHandle FactoryRow = RegisterWidgetFactoryRow(PurposeRow);
					Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
					return true;
				}
			case EPurposeType::UniqueByName:
				if (!Columns.IsEmpty())
				{
					UE::Editor::DataStorage::RowHandle FactoryRow = RegisterUniqueWidgetFactoryRow(PurposeRow);
					Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
				}
				else
				{
					return false;
				}
			case EPurposeType::UniqueByNameAndColumn:
				{
					if (!Columns.IsEmpty())
					{
						Columns.Compile(UE::Editor::DataStorage::Queries::FEditorStorageQueryConditionCompileContext(Storage));
							
						UE::Editor::DataStorage::RowHandle FactoryRow = RegisterWidgetFactoryRow(PurposeRow);
						Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
						Storage->AddColumn(FactoryRow, FWidgetFactoryConditionsColumn{.Conditions = Columns});
							
						return true;
					}
					else
					{
						return false;
					}
				}
			default:
				checkf(false, TEXT("Unexpected UE::Editor::DataStorage::IUiProvider::EPurposeType found provided when registering widget factory."));
				return false;
			}
		}
		
		UE_LOGF(LogEditorDataStorageUI, Warning,
				"Unable to register widget factory '%ls' as purpose '%llu' isn't registered.", *Constructor->GetName(), PurposeRow);
		return false;
	}
	else
	{
		return RegisterWidgetFactory(PurposeRow, Constructor);
	}
}

bool UEditorDataStorageUi::RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow,
	TUniquePtr<FTypedElementWidgetConstructor>&& Constructor)
{
	checkf(Constructor->GetTypeInfo(), TEXT("Widget constructor being registered that doesn't have valid type information."));

	return RegisterWidgetFactory(PurposeRow, Constructor->GetTypeInfo());
}

bool UEditorDataStorageUi::RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow,
	TUniquePtr<FTypedElementWidgetConstructor>&& Constructor, UE::Editor::DataStorage::Queries::FConditions Columns)
{
	checkf(Constructor->GetTypeInfo(), TEXT("Widget constructor being registered that doesn't have valid type information."));

	return RegisterWidgetFactory(PurposeRow, Constructor->GetTypeInfo(), Columns);
}

bool UEditorDataStorageUi::RegisterDecoratorWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor,
	const UScriptStruct* Column)
{
	checkf(Constructor->IsChildOf(FTedsDecoratorWidgetConstructor::StaticStruct()),
		TEXT("Attempting to register a decorator widget constructor '%s' that isn't derived from FTedsDecoratorWidgetConstructor."),
		*Constructor->GetFullName());

	using namespace UE::Editor::DataStorage::Queries;
		
	if (FWidgetPurposeColumn* PurposeInfo = Storage->GetColumn<FWidgetPurposeColumn>(PurposeRow))
	{
		constexpr bool bIsDecoratorWidget = true;

		switch (PurposeInfo->PurposeType)
		{
		case EPurposeType::Generic:
			{
				RowHandle FactoryRow = RegisterWidgetFactoryRow(PurposeRow, bIsDecoratorWidget);
				Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
			}
			break;
		case EPurposeType::UniqueByName:
			{
				if (Column)
				{
					RowHandle FactoryRow = RegisterUniqueWidgetFactoryRow(PurposeRow, bIsDecoratorWidget);
					Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
				}
				else
				{
					return false;
				}
			}
			break;
		case EPurposeType::UniqueByNameAndColumn:
			{
				if (Column)
				{
					FConditions ColumnConditions{TColumn(Column)};
					ColumnConditions.Compile(FEditorStorageQueryConditionCompileContext(Storage));
						
					RowHandle FactoryRow = RegisterWidgetFactoryRow(PurposeRow, bIsDecoratorWidget);
					Storage->AddColumn(FactoryRow, FWidgetFactoryConstructorTypeInfoColumn{.Constructor = Constructor});
					Storage->AddColumn(FactoryRow, FWidgetFactoryConditionsColumn{.Conditions =ColumnConditions});
				}
				else
				{
					return false;
				}
				break;
			}
		default:
			checkf(false, TEXT("Unexpected UE::Editor::DataStorage::IUiProvider::EPurposeType found provided when registering widget factory."));
			return false;
		}
	}
	else
	{
		UE_LOGF(LogEditorDataStorageUI, Warning, 
			"Unable to register decorator widget factory '%ls' as purpose '%llu' isn't registered.", *Constructor->GetName(), PurposeRow);
		return false;
	}

	// Register observers for this decorator so any widget rows that add or remove the column post construction are updated
	RegisterDecoratorWidgetObservers(PurposeRow, Column);
	
	return true;
}

void UEditorDataStorageUi::CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructionCallback& Callback)
{
	while (Storage->HasColumns<FWidgetPurposeColumn>(PurposeRow))
	{
		TArray<UE::Editor::DataStorage::RowHandle> Factories;
		GetWidgetFactories(PurposeRow, Factories);

		// If no factories were found for this purpose, move on to the parent purpose
		if (Factories.IsEmpty())
		{
			PurposeRow = Storage->GetParentRow(WidgetPurposesHierarchyHandle, PurposeRow);
			continue;
		}

		for (UE::Editor::DataStorage::RowHandle FactoryRow : Factories)
		{
			if (!CreateSingleWidgetConstructor(FactoryRow, Arguments, {}, Callback))
			{
				return;
			}
		}

		// Don't want to go up the parent chain if we created any widgets for this purpose
		break;
	}
}

void UEditorDataStorageUi::CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, EMatchApproach MatchApproach,
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetConstructionCallback& Callback)
{
	// Sort by name so that removing the matched columns can be done in a single pass.
	// Sorting by ptr does not work because dynamic column ptrs are different than their base template
	Columns.Sort(
		[](const TWeakObjectPtr<const UScriptStruct>& Lhs, const TWeakObjectPtr<const UScriptStruct>& Rhs)
		{
			return Lhs->GetName() < Rhs->GetName();
		});
	
	while (Storage->HasColumns<FWidgetPurposeColumn>(PurposeRow))
	{
		TArray<UE::Editor::DataStorage::RowHandle> Factories;
		GetWidgetFactories(PurposeRow, Factories);

		if (!Factories.IsEmpty())
		{
			// There is currently no way to cache the sorted results back into TEDS, so we sort every time this function is called
			Factories.StableSort(
				[this](UE::Editor::DataStorage::RowHandle Lhs, UE::Editor::DataStorage::RowHandle Rhs)
				{
					const UE::Editor::DataStorage::Queries::FConditions& LhsConditions = GetFactoryConditions(Lhs);
					const UE::Editor::DataStorage::Queries::FConditions& RhsConditions = GetFactoryConditions(Rhs);
				
					int32 LeftSize = LhsConditions.MinimumColumnMatchRequired();
					int32 RightSize = RhsConditions.MinimumColumnMatchRequired();

					// If two factories are the same size, we want factories containing dynamic templates to be at the end so
					// they are de-prioritized when matching and factories with dynamic specializations are matched first.
					// e.g A widget factory for ColumnA("Apple") or ColumnA("Orange") should be considered before a generic one for ColumnA.
					if (LeftSize == RightSize)
					{
						return !LhsConditions.UsesDynamicTemplates();
					}
				
					return LeftSize > RightSize;
				});

			FWidgetConstructionMethod ConstructionCallback = [this, Arguments, Callback]
				(UE::Editor::DataStorage::RowHandle FactoryRow, TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumns)
				{
					return CreateSingleWidgetConstructor(FactoryRow, Arguments, MatchedColumns, Callback);
				};

			switch (MatchApproach)
			{
			case EMatchApproach::LongestMatch:

				// For longest match, we don't want to continue matching with the parent purpose if the user requested us to stop
				if (!CreateWidgetConstructors_LongestMatch(Factories, Columns, ConstructionCallback))
				{
					return;
				}
				break;
			case EMatchApproach::ExactMatch:
				CreateWidgetConstructors_ExactMatch(Factories, Columns, ConstructionCallback);
				break;
			case EMatchApproach::SingleMatch:
				CreateWidgetConstructors_SingleMatch(Factories, Columns, ConstructionCallback);
				break;
			default:
				checkf(false, TEXT("Unsupported match type (%i) for CreateWidgetConstructors."), 
					static_cast<std::underlying_type_t<EMatchApproach>>(MatchApproach));
			}
		}

		// No need to go up the parent chain if there are no more columns to match
		if (Columns.IsEmpty())
		{
			return;
		}

		// If we have a parent purpose, try matching against factories belonging to it next
		PurposeRow = Storage->GetParentRow(WidgetPurposesHierarchyHandle, PurposeRow);
	}
}

void UEditorDataStorageUi::ConstructWidgets(UE::Editor::DataStorage::RowHandle PurposeRow, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetCreatedCallback& ConstructionCallback)
{
	// Find the first purpose in the parent chain with at least one registered factory
	TArray<UE::Editor::DataStorage::RowHandle> Factories;

	while (Storage->HasColumns<FWidgetPurposeColumn>(PurposeRow) && Factories.IsEmpty())
	{
		GetWidgetFactories(PurposeRow, Factories);
		
		// If no factories were found for this purpose, move on to the parent purpose
		PurposeRow = Storage->GetParentRow(WidgetPurposesHierarchyHandle, PurposeRow);
	}
	
	for (UE::Editor::DataStorage::RowHandle FactoryRow : Factories)
	{
		if (FWidgetFactoryConstructorTypeInfoColumn* ConstructorTypeInfoColumn = Storage->GetColumn<FWidgetFactoryConstructorTypeInfoColumn>(FactoryRow))
		{
			if (const UScriptStruct* ConstructorType = ConstructorTypeInfoColumn->Constructor.Get())
			{
				CreateWidgetInstance(ConstructorType, Arguments, ConstructionCallback);
			}
		}
	}
}

bool UEditorDataStorageUi::CreateSingleWidgetConstructor(
	UE::Editor::DataStorage::RowHandle FactoryRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments,
	TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes,
	const WidgetConstructionCallback& Callback)
{
	if (FWidgetFactoryConstructorTypeInfoColumn* ConstructorTypeInfoColumn = Storage->GetColumn<FWidgetFactoryConstructorTypeInfoColumn>(FactoryRow))
	{
		if (const UScriptStruct* Target = ConstructorTypeInfoColumn->Constructor.Get())
		{
			FWidgetConstructorPtr Result(Internal::CreateWidgetConstructorInstance(Target));
			if (Result)
			{
				Result->Initialize(Arguments, MoveTemp(MatchedColumnTypes), FactoryRow);
				const TArray<TWeakObjectPtr<const UScriptStruct>>& MatchedColumns = Result->GetMatchedColumns();
				return Callback(MoveTemp(Result), MatchedColumns);
			}
			return true;
		}
	}
	return false;
}

TSharedPtr<FTedsDecoratorWidgetConstructor> UEditorDataStorageUi::CreateSingleDecoratorWidget(UE::Editor::DataStorage::RowHandle FactoryRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments, TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes)
{
	if (FWidgetFactoryConstructorTypeInfoColumn* ConstructorTypeInfoColumn = Storage->GetColumn<FWidgetFactoryConstructorTypeInfoColumn>(FactoryRow))
	{
		if (const UScriptStruct* Target = ConstructorTypeInfoColumn->Constructor.Get())
		{
			TSharedPtr<FTedsDecoratorWidgetConstructor> Result(Internal::CreateDecoratorWidgetConstructorInstance(Target));
			if (Result)
			{
				Result->Initialize(Arguments, MoveTemp(MatchedColumnTypes), FactoryRow);
				return MoveTemp(Result);
			}
		}
	}
	return nullptr;
}

void UEditorDataStorageUi::CreateWidgetInstance(
	FTypedElementWidgetConstructor& Constructor, 
	const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetCreatedCallback& ConstructionCallback)
{
	UE::Editor::DataStorage::RowHandle Row = Storage->AddRow(WidgetTable);
	if (TSharedPtr<SWidget> Widget = ConstructWidget(Row, Constructor, Arguments))
	{
		ConstructionCallback(Widget.ToSharedRef(), Row);
	}
	else
	{
		Storage->RemoveRow(Row);
	}
}

void UEditorDataStorageUi::CreateWidgetInstance(const UScriptStruct* ConstructorType, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetCreatedCallback& ConstructionCallback)
{
	FTypedElementWidgetConstructor* Constructor = reinterpret_cast<FTypedElementWidgetConstructor*>(
						FMemory_Alloca_Aligned(ConstructorType->GetStructureSize(), ConstructorType->GetMinAlignment()));
	if (Constructor)
	{
		ConstructorType->InitializeStruct(Constructor);
		CreateWidgetInstance(*Constructor, Arguments, ConstructionCallback);
		ConstructorType->DestroyStruct(&Constructor);
	}
	else
	{
		checkf(false, TEXT("Remaining memory is too small to create a widget constructor from a description."));
	}
}

TSharedPtr<SWidget> UEditorDataStorageUi::ConstructWidget(UE::Editor::DataStorage::RowHandle Row, FTypedElementWidgetConstructor& Constructor,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	TSharedPtr<SWidget> InternalWidget = ConstructInternalWidget(Row, Constructor, Arguments);

	// Create a container widget to hold the content
	if (TSharedPtr<UE::Editor::DataStorage::ITedsWidget> ContainerWidget = CreateContainerTedsWidget(Row))
	{
		ContainerWidget->SetContent(InternalWidget.ToSharedRef());
		Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(Row)->TedsWidget = ContainerWidget;
		return ContainerWidget->AsWidget();
	}

	return InternalWidget;
}

TSharedPtr<SWidget> UEditorDataStorageUi::ConstructInternalWidget(UE::Editor::DataStorage::RowHandle Row, FTypedElementWidgetConstructor& Constructor,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	TSharedPtr<SWidget> InternalWidget = Constructor.ConstructFinalWidget(Row, Storage, this, Arguments);

	// Create any decorator widgets
	InternalWidget = CreateDecoratorWidgets(Row, InternalWidget, Constructor, Arguments);

	return InternalWidget;
}

void UEditorDataStorageUi::ListWidgetPurposes(const WidgetPurposeCallback& Callback) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	static QueryHandle PurposeQueryHandle =
		Storage->RegisterQuery(Select().ReadOnly<FWidgetPurposeNameColumn, FWidgetPurposeColumn, FDescriptionColumn>().Compile());

	Storage->RunQuery(PurposeQueryHandle, CreateDirectQueryCallbackBinding(
		[Callback](const FWidgetPurposeNameColumn& NameColumn, const FWidgetPurposeColumn& PurposeColumn, const FDescriptionColumn& DescriptionColumn)
	{
		FPurposeInfo PurposeInfo(NameColumn);
		Callback(PurposeInfo.ToString(), PurposeColumn.PurposeType, DescriptionColumn.Description);
	}));
}

bool UEditorDataStorageUi::SupportsExtension(FName Extension) const
{
	return false;
}

void UEditorDataStorageUi::ListExtensions(TFunctionRef<void(FName)> Callback) const
{
	
}

TSharedPtr<UE::Editor::DataStorage::ITedsWidget> UEditorDataStorageUi::CreateContainerTedsWidget(UE::Editor::DataStorage::RowHandle UiRowHandle) const
{
	return SNew(UE::Editor::DataStorage::Widgets::STedsWidget)
			.UiRowHandle(UiRowHandle);
}

UE::Editor::DataStorage::TableHandle UEditorDataStorageUi::GetWidgetTable() const
{
	return WidgetTable;
}

void UEditorDataStorageUi::GeneratePropertySearchers(TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>>& Results,
		TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const
{
	using namespace UE::Editor::DataStorage;

	TArray<const PropertySearcherConstructorCallback*, TInlineAllocator<8>> UsedConstructors;

	for (TWeakObjectPtr<const UScriptStruct>& ColumnTypePointer : Columns)
	{
		if (const UScriptStruct* ColumnType = ColumnTypePointer.Get())
		{
			for (TFieldIterator<FProperty> PropIt(ColumnType); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				if (Property->HasMetaData(TEXT("Searchable")))
				{
					if (const PropertySearcherConstructorCallback* Callback = PropertySearcherConstructors.Find(Property->GetClass()))
					{
						if (!UsedConstructors.Contains(Callback))
						{
							UsedConstructors.Add(Callback);
							Results.Add((*Callback)(ColumnTypePointer, *Property));
						}
					}
					else
					{
						UE_LOGF(LogEditorDataStorageUI, Warning, 
							"Property '%ls' in class '%ls' is marked as searchable, but no property searcher for type '%ls' was registered.",
							*Property->GetName(), *Property->GetFullGroupName(true), *Property->GetClass()->GetName());
					}
				}
			}
		}
	}
}

void UEditorDataStorageUi::RegisterSearcherGeneratorForProperty(const FFieldClass* PropertyType, 
	PropertySearcherConstructorCallback PropertySearcherConstructor)
{
	checkf(!PropertySearcherConstructors.Contains(PropertyType), 
		TEXT("Double registration of property searcher for type '%s'."), *PropertyType->GetName());
	PropertySearcherConstructors.Add(PropertyType, MoveTemp(PropertySearcherConstructor));
}

void UEditorDataStorageUi::UnregisterSearcherGeneratorForProperty(const FFieldClass* PropertyType)
{
	PropertySearcherConstructors.Remove(PropertyType);
}

void UEditorDataStorageUi::GeneratePropertySorters(TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>>& Results,
	TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const
{
	using namespace UE::Editor::DataStorage;

	TArray<const PropertySorterConstructorCallback*, TInlineAllocator<8>> UsedConstructors;

	for (TWeakObjectPtr<const UScriptStruct>& ColumnTypePointer : Columns)
	{
		if (const UScriptStruct* ColumnType = ColumnTypePointer.Get())
		{
			for (TFieldIterator<FProperty> PropIt(ColumnType); PropIt; ++PropIt)
			{
				FProperty* Property = *PropIt;
				if (Property->HasMetaData(TEXT("Sortable")))
				{
					if (const PropertySorterConstructorCallback* Callback = PropertySorterConstructors.Find(Property->GetClass()))
					{
						if (!UsedConstructors.Contains(Callback))
						{
							UsedConstructors.Add(Callback);
							Results.Add((*Callback)(ColumnTypePointer, *Property));
						}
					}
					else
					{
						UE_LOGF(LogEditorDataStorageUI, Warning, 
							"Property '%ls' in class '%ls' is marked as sortable, but no property sorter for type '%ls' was registered.",
							*Property->GetName(), *Property->GetFullGroupName(true), *Property->GetClass()->GetName());
					}
				}
			}
		}
	}
}

void UEditorDataStorageUi::RegisterSorterGeneratorForProperty(const FFieldClass* PropertyType, 
	PropertySorterConstructorCallback PropertySorterConstructor)
{
	checkf(!PropertySorterConstructors.Contains(PropertyType), 
		TEXT("Double registration of property sorter for type '%s'."), *PropertyType->GetName());
	PropertySorterConstructors.Add(PropertyType, MoveTemp(PropertySorterConstructor));
}

void UEditorDataStorageUi::UnregisterSorterGeneratorForProperty(const FFieldClass* PropertyType)
{
	PropertySorterConstructors.Remove(PropertyType);
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UEditorDataStorageUi::GetDefaultWidgetPurposeID() const
{
	return Internal::DefaultWidgetPurposeID;	
}

UE::Editor::DataStorage::IUiProvider::FPurposeID UEditorDataStorageUi::GetGeneralWidgetPurposeID() const
{
	return Internal::GeneralWidgetPurposeID;
}

UE::Editor::DataStorage::FHierarchyHandle UEditorDataStorageUi::GetWidgetPurposeHierarchyHandle() const
{
	return WidgetPurposesHierarchyHandle;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageUi::FindPurpose(const FPurposeID& PurposeID) const
{
	return Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(PurposeID));
}

void UEditorDataStorageUi::CreateStandardArchetypes()
{
	using namespace UE::Editor::DataStorage::Ui;
	
	WidgetTable = Storage->RegisterTable(MakeArrayView(
		{
			FTypedElementSlateWidgetReferenceColumn::StaticStruct(),
			FTypedElementSlateWidgetReferenceDeletesRowTag::StaticStruct(),
			FSlateColorColumn::StaticStruct()
		}), FName(TEXT("Editor_WidgetTable")));

	WidgetPurposeTable = Storage->RegisterTable(UE::Editor::DataStorage::TTypedElementColumnTypeList<FWidgetPurposeColumn,
		FWidgetPurposeNameColumn>(), FName("Editor_WidgetPurposeTable"));
	
	WidgetFactoryTable = Storage->RegisterTable(UE::Editor::DataStorage::TTypedElementColumnTypeList<FWidgetFactoryColumn>(),
		FName("Editor_WidgetFactoryTable"));
	
	DecoratorWidgetFactoryTable = Storage->RegisterTable(
		UE::Editor::DataStorage::TTypedElementColumnTypeList<FWidgetFactoryColumn, FDecoratorWidgetFactoryTag>(),
		FName("Editor_DecoratorWidgetFactoryTable"));
}

void UEditorDataStorageUi::RegisterQueries()
{
	using namespace UE::Editor::DataStorage::Queries;
	
	Storage->RegisterQuery(
		Select(
			TEXT("Add the name to widget factory with constructor"),
			FObserver::OnAdd<FWidgetFactoryConstructorTypeInfoColumn>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FWidgetFactoryConstructorTypeInfoColumn& ConstructorColumn)
			{
				Context.AddColumn(Row, FWidgetConstructorNameColumn{ .Name = ConstructorColumn.Constructor->GetFName() });
			})
		.Where()
			.All<FWidgetFactoryColumn>()
		.Compile());
}

void UEditorDataStorageUi::RegisterHierarchies()
{
	using namespace UE::Editor::DataStorage;
	
	UE::Editor::DataStorage::FHierarchyRegistrationParams WidgetPurposeHierarchyParams
		{
			.Name = Internal::WidgetPurposesHierarchyName
		};
	
	WidgetPurposesHierarchyHandle = Storage->RegisterHierarchy(WidgetPurposeHierarchyParams);
}

void UEditorDataStorageUi::PostDataStorageUpdate()
{
	// For any widgets that need their decorators updated, re-create decorators based on the columns on the widget row
	for (UE::Editor::DataStorage::RowHandle WidgetRow : WidgetsToUpdateDecorators)
	{
		if (FTypedElementSlateWidgetReferenceColumn* WidgetReferenceColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(WidgetRow))
		{
			if (WidgetReferenceColumn->WidgetConstructor.IsValid())
			{
				if (TSharedPtr<UE::Editor::DataStorage::ITedsWidget> ContainerWidget = WidgetReferenceColumn->TedsWidget.Pin())
				{
					TSharedPtr<SWidget> InternalWidget = CreateDecoratorWidgets(WidgetRow, WidgetReferenceColumn->Widget.Pin(),
						*WidgetReferenceColumn->WidgetConstructor.Pin(), UE::Editor::DataStorage::FMetaDataView());

					// Add the final composite widget that contains all the decorators and the actual widget to the container ITedsWidget
					ContainerWidget->SetContent(InternalWidget.ToSharedRef());
				}
			}
		}
	}

	WidgetsToUpdateDecorators.Empty();
}

bool UEditorDataStorageUi::CreateWidgetConstructors_LongestMatch(const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories,
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, FWidgetConstructionMethod ConstructionFunction)
{
	TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumns;
	for (auto FactoryIt = WidgetFactories.CreateConstIterator(); FactoryIt && !Columns.IsEmpty();)
	{
		const UE::Editor::DataStorage::Queries::FConditions& Conditions = GetFactoryConditions(*FactoryIt);
		
		if (Conditions.MinimumColumnMatchRequired() > Columns.Num())
		{
			// There are more columns required for this factory than there are in the requested columns list so skip this
			// factory.
			++FactoryIt;
			continue;
		}

		MatchedColumns.Reset();
		
		if (Conditions.Verify(MatchedColumns, Columns))
		{
			// Empty conditions match against everything - so we force a match against the first column in the list
			if (Conditions.IsEmpty() || MatchedColumns.IsEmpty())
			{
				MatchedColumns = {Columns[0]};
			}
			
			// Remove the found columns from the requested list.
			MatchedColumns.Sort(
				[](const TWeakObjectPtr<const UScriptStruct>& Lhs, const TWeakObjectPtr<const UScriptStruct>& Rhs)
				{
					return Lhs->GetName() < Rhs->GetName();
				});
			
			MatchedColumns.SetNum(Algo::Unique(MatchedColumns), EAllowShrinking::No);

			// We need to keep track of the columns the user requested that ended up matching separately because MatchedColumns could contain the
			// base template for a dynamic column in the requested columns that actually matched - and the widget constructor wants the latter
			TArray<TWeakObjectPtr<const UScriptStruct>> RequestedColumnsThatMatched;
			
			TWeakObjectPtr<const UScriptStruct>* ColumnsIt = Columns.GetData();
			TWeakObjectPtr<const UScriptStruct>* ColumnsEnd = ColumnsIt + Columns.Num();
			int32 ColumnIndex = 0;
			for (const TWeakObjectPtr<const UScriptStruct>& MatchedColumn : MatchedColumns)
			{
				// Remove all the columns that were matched from the provided column list.
				while (!Internal::CheckSingleColumnMatch(MatchedColumn.Get(), ColumnsIt->Get()))
				{
					++ColumnIndex;
					++ColumnsIt;
					if (ColumnsIt == ColumnsEnd)
					{
						ensureMsgf(false, TEXT("A previously found matching column can't be found in the original array."));
						return false;
					}
				}
				RequestedColumnsThatMatched.Add(*ColumnsIt);
				Columns.RemoveAt(ColumnIndex, EAllowShrinking::No);
				--ColumnsEnd;
			}
			
			if (!ConstructionFunction(*FactoryIt, MoveTemp(RequestedColumnsThatMatched)))
			{
				return false;
			}
		}
		else
		{
			// If this factory matched against a subset of the columns, we don't want to increment the iterator so we can check the same factory
			// against the columns again to see if it matches another subset of the remaining columns. Otherwise a single factory only gets checked
			// against the columns once even if it would match multiple subsets of the column list.
			++FactoryIt;
		}
	}

	return true;
}

void UEditorDataStorageUi::CreateWidgetConstructors_ExactMatch(const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories,
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, FWidgetConstructionMethod ConstructionFunction)
{
	int32 ColumnCount = Columns.Num();
	TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumns;
	for (UE::Editor::DataStorage::RowHandle FactoryRow : WidgetFactories)
	{
		const UE::Editor::DataStorage::Queries::FConditions& Conditions = GetFactoryConditions(FactoryRow);
		
		// If there are more matches required that there are columns, then there will never be an exact match.
		// Less than the column count can still result in a match that covers all columns.
		if (Conditions.MinimumColumnMatchRequired() > ColumnCount)
		{
			continue;
		}

		MatchedColumns.Reset();

		if (Conditions.Verify(MatchedColumns, Columns))
		{
			// Empty conditions match against everything - so we update the matched columns list to reflect that
			if (Conditions.IsEmpty())
			{
				MatchedColumns = Columns;
			}
			
			Algo::SortBy(MatchedColumns, [](const TWeakObjectPtr<const UScriptStruct>& Column) { return Column.Get(); });
			MatchedColumns.SetNum(Algo::Unique(MatchedColumns), EAllowShrinking::No);
			if (MatchedColumns.Num() == Columns.Num())
			{
				Columns.Reset();
				ConstructionFunction(FactoryRow, MoveTemp(MatchedColumns));
				return;
			}
		}
	}
}

void UEditorDataStorageUi::CreateWidgetConstructors_SingleMatch(const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories,
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, FWidgetConstructionMethod ConstructionFunction)
{
	// Start from the back as the widgets with lower counts will be last.
	for (int32 ColumnIndex = Columns.Num() - 1; ColumnIndex >= 0; --ColumnIndex)
	{
		auto FactoryIt = WidgetFactories.rbegin();
		auto FactoryEnd = WidgetFactories.rend();

		for (; FactoryIt != FactoryEnd; ++FactoryIt)
		{
			const UE::Editor::DataStorage::Queries::FConditions& Conditions = GetFactoryConditions(*FactoryIt);
			
			TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnData = Conditions.GetColumns();
			if (ColumnData.Num() > 1)
			{
				// Moved passed the point where factories only have a single column.
				return;
			}
			else if (ColumnData.Num() == 0)
			{
				// Need to move further to find factories with exactly one column.
				continue;
			}

			if (Internal::CheckSingleColumnMatch(ColumnData[0].Get(), Columns[ColumnIndex].Get()))
			{
				// We need to keep a copy of the actually requested column because the matched column could be the base template for the actually matched
				// dynamic column and the widget constructor wants the latter
				TWeakObjectPtr<const UScriptStruct> RequestedColumn = Columns[ColumnIndex];
				Columns.RemoveAt(ColumnIndex);
				ConstructionFunction(*FactoryIt, {RequestedColumn});
				// Match was found so move on to the next column in the column.
				break;
			} 
		}
	}
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageUi::RegisterWidgetFactoryRow(UE::Editor::DataStorage::RowHandle PurposeRowHandle, bool bDecoratorWidgetFactory) const
{
	UE::Editor::DataStorage::RowHandle FactoryRowHandle = bDecoratorWidgetFactory ? Storage->AddRow(DecoratorWidgetFactoryTable) : Storage->AddRow(WidgetFactoryTable);
	Storage->GetColumn<FWidgetFactoryColumn>(FactoryRowHandle)->PurposeRowHandle = PurposeRowHandle;
	return FactoryRowHandle;
}

UE::Editor::DataStorage::RowHandle UEditorDataStorageUi::RegisterUniqueWidgetFactoryRow(UE::Editor::DataStorage::RowHandle InPurposeRowHandle, bool bDecoratorWidgetFactory) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Ui;
	
	static QueryHandle FactoryQueryHandle = Storage->RegisterQuery(Select().ReadOnly<FWidgetFactoryColumn>().Compile());

	RowHandle FactoryRowHandle = InvalidRowHandle;

	// Find the first matching factory belonging to this purpose, we know there is only going to be one
	Storage->RunQuery(FactoryQueryHandle, CreateDirectQueryCallbackBinding(
		[&FactoryRowHandle, InPurposeRowHandle](RowHandle FoundFactoryRowHandle, const FWidgetFactoryColumn& PurposeReferenceColumn)
		{
			if (PurposeReferenceColumn.PurposeRowHandle == InPurposeRowHandle)
			{
				FactoryRowHandle = FoundFactoryRowHandle;
			}
		}));

	// If there was a factory already registered for this purpose, overwrite its information
	if (Storage->IsRowAvailable(FactoryRowHandle))
	{
		Storage->RemoveColumns<FWidgetFactoryConstructorTypeInfoColumn>(FactoryRowHandle);
		Storage->GetColumn<FWidgetFactoryColumn>(FactoryRowHandle)->PurposeRowHandle = InPurposeRowHandle;

		// If we are registering a decorator widget factory, ensure the correct tag is added (and remove if not). 
		bDecoratorWidgetFactory ? Storage->AddColumn<FDecoratorWidgetFactoryTag>(FactoryRowHandle) : Storage->RemoveColumn<FDecoratorWidgetFactoryTag>(FactoryRowHandle);
		return FactoryRowHandle;
	}
	// Otherwise just register the factory row as usual
	else
	{
		return RegisterWidgetFactoryRow(InPurposeRowHandle, bDecoratorWidgetFactory);
	}
}

void UEditorDataStorageUi::GetWidgetFactories(UE::Editor::DataStorage::RowHandle PurposeRowHandle,
	TArray<UE::Editor::DataStorage::RowHandle>& OutFactories) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Ui;
	
	static QueryHandle FactoryQueryHandle =
		Storage->RegisterQuery(
			Select()
				.ReadOnly<FWidgetFactoryColumn>()
			.Where()
				.None<FDecoratorWidgetFactoryTag>()
			.Compile());
	
	Storage->RunQuery(FactoryQueryHandle, CreateDirectQueryCallbackBinding(
		[PurposeRowHandle, &OutFactories](RowHandle RowHandle, const FWidgetFactoryColumn& PurposeReferenceColumn)
		{
			if (PurposeReferenceColumn.PurposeRowHandle == PurposeRowHandle)
			{
				OutFactories.Add(RowHandle);
			}
		}));
}

void UEditorDataStorageUi::GetDecoratorWidgetFactories(UE::Editor::DataStorage::RowHandle PurposeRowHandle,
	TArray<UE::Editor::DataStorage::RowHandle>& OutFactories) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Ui;
	
	static QueryHandle FactoryQueryHandle =
		Storage->RegisterQuery(
			Select()
				.ReadOnly<FWidgetFactoryColumn>()
			.Where()
				.All<FDecoratorWidgetFactoryTag>()
			.Compile());
	
	Storage->RunQuery(FactoryQueryHandle, CreateDirectQueryCallbackBinding(
		[PurposeRowHandle, &OutFactories](RowHandle RowHandle, const FWidgetFactoryColumn& PurposeReferenceColumn)
		{
			if (PurposeReferenceColumn.PurposeRowHandle == PurposeRowHandle)
			{
				OutFactories.Add(RowHandle);
			}
		}));
}

const UE::Editor::DataStorage::Queries::FConditions& UEditorDataStorageUi::GetFactoryConditions(UE::Editor::DataStorage::RowHandle FactoryRow) const
{
	using namespace UE::Editor::DataStorage::Queries;

	if (FWidgetFactoryConditionsColumn* FactoryColumn = Storage->GetColumn<FWidgetFactoryConditionsColumn>(FactoryRow))
	{
		FactoryColumn->Conditions.Compile(FEditorStorageQueryConditionCompileContext(Storage));
		return FactoryColumn->Conditions;
	}
	// If this factory does not have any query conditions, just return a default empty FConditions struct
	else
	{
		static FConditions DefaultConditions;
		DefaultConditions.Compile(FEditorStorageQueryConditionCompileContext(Storage));
		return DefaultConditions;
	}
	
}

TSharedPtr<SWidget> UEditorDataStorageUi::CreateDecoratorWidgets(UE::Editor::DataStorage::RowHandle WidgetRowHandle, TSharedPtr<SWidget> InternalWidget, FTypedElementWidgetConstructor& Constructor,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	RowHandle FactoryRow = Constructor.GetWidgetFactoryRow();

	if (FWidgetFactoryColumn* FactoryColumn = Storage->GetColumn<FWidgetFactoryColumn>(FactoryRow))
	{
		RowHandle PurposeRow = FactoryColumn->PurposeRowHandle;

		if (Storage->IsRowAvailable(PurposeRow))
		{
			// First get all the decorator widgets for this purpose and all the columns on the UI row so we can figure out which decorators are
			// actually needed
			TArray<RowHandle> Factories;
			GetDecoratorWidgetFactories(PurposeRow, Factories);
			
			TArray<TWeakObjectPtr<const UScriptStruct>> Columns;
			Storage->ListColumns(WidgetRowHandle, [&Columns](const UScriptStruct& ColumnType)
			{
				Columns.Emplace(&ColumnType);
			});

			Columns.Sort(
				[](const TWeakObjectPtr<const UScriptStruct>& Lhs, const TWeakObjectPtr<const UScriptStruct>& Rhs)
				{
					return Lhs->GetName() < Rhs->GetName();
				});

			TArray<TSharedPtr<FTedsDecoratorWidgetConstructor>> Decorators;

			FWidgetConstructionMethod ConstructionCallback = [this, Arguments, &Decorators]
				(UE::Editor::DataStorage::RowHandle FactoryRow, TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumns)
			{
				TSharedPtr<FTedsDecoratorWidgetConstructor> Constructor = CreateSingleDecoratorWidget(FactoryRow, Arguments, MatchedColumns);
				Decorators.Add(MoveTemp(Constructor));
				return true;
			};

			// Decorator widgets can only match against a single column, so we can re-use the single match algorithm to figure out which ones
			// match the columns on the widget row
			CreateWidgetConstructors_SingleMatch(Factories, Columns, ConstructionCallback);

			// If this widget was created for a specific data row, also let the decorator know about that
			RowHandle TargetRow = InvalidRowHandle;
			if (FTypedElementRowReferenceColumn* RowReferenceColumn = Storage->GetColumn<FTypedElementRowReferenceColumn>(WidgetRowHandle))
			{
				TargetRow = RowReferenceColumn->Row;
			}

			// Create each decorator widget, compounding the result widget of the previous operation onto the current widget
			for (TSharedPtr<FTedsDecoratorWidgetConstructor>& ConstructorIt : Decorators)
			{
				InternalWidget = ConstructorIt->CreateDecoratorWidget(InternalWidget, Storage, this, TargetRow, WidgetRowHandle, Arguments);
			}
		}
	}

	return InternalWidget;
}

void UEditorDataStorageUi::RegisterDecoratorWidgetObservers(UE::Editor::DataStorage::RowHandle PurposeRowHandle, const UScriptStruct* DecoratorColumn)
{
	using namespace UE::Editor::DataStorage::Queries;

	FString OnAddName = TEXT("TEDS UI Decorator Monitor: OnAdd - ");
	DecoratorColumn->GetFName().AppendString(OnAddName);

	FQueryDescription OnAddObserver =
		Select(
			FName(OnAddName),
			FObserver(FObserver::EEvent::Add, DecoratorColumn).SetExecutionMode(EExecutionMode::GameThread),
			[PurposeRowHandle, this](IQueryContext& Context, RowHandle Row, const FWidgetPurposeReferenceColumn& PurposeColumn)
			{
				if (PurposeColumn.PurposeRowHandle == PurposeRowHandle)
				{
					// Defer the actual decorator update since widget construction could be accessing the data storage directly for data
					WidgetsToUpdateDecorators.Add(Row);
				}
			})
		.Where()
			.All<FTypedElementSlateWidgetReferenceColumn>()
		.Compile();

	DecoratorObservers.Add(Storage->RegisterQuery(MoveTemp(OnAddObserver)));

	FString OnRemoveName = TEXT("TEDS UI Decorator Monitor: OnRemove - ");
	DecoratorColumn->GetFName().AppendString(OnRemoveName);

	FQueryDescription OnRemoveObserver =
		Select(
			FName(OnRemoveName),
			FObserver(FObserver::EEvent::Remove, DecoratorColumn).SetExecutionMode(EExecutionMode::GameThread),
			[PurposeRowHandle, this](IQueryContext& Context, RowHandle Row, const FWidgetPurposeReferenceColumn& PurposeColumn)
			{
				if (PurposeColumn.PurposeRowHandle == PurposeRowHandle)
				{
					// Defer the actual decorator update since widget construction could be accessing the data storage directly for data
					WidgetsToUpdateDecorators.Add(Row);
				}
			})
		.Where()
			.All<FTypedElementSlateWidgetReferenceColumn>()
		.Compile();

	DecoratorObservers.Add(Storage->RegisterQuery(MoveTemp(OnRemoveObserver)));
}

// Deprecated functions use a deprecated alias as an argument so we need to disable warnings for their definitions
PRAGMA_DISABLE_DEPRECATION_WARNINGS

void UEditorDataStorageUi::CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback)
{
	auto CallbackWrapper = [Callback](FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumns) -> bool
		{
			// This is technically safe since nothing else is using the shared ptr yet as it was just created,
			// plus we are manually resetting the only instance which is used in this callback
			TUniquePtr<FTypedElementWidgetConstructor> ConstructorPtr(Constructor.Get());
			Constructor.Reset();

			return Callback(MoveTemp(ConstructorPtr), MatchedColumns);
		};

	CreateWidgetConstructors(PurposeRow, Arguments, CallbackWrapper);
}

void UEditorDataStorageUi::CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, EMatchApproach MatchApproach,
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetConstructorCallback& Callback)
{
	auto CallbackWrapper = [Callback](FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumns) -> bool
		{
			// This is technically safe since nothing else is using the shared ptr yet as it was just created,
			// plus we are manually resetting the only instance which is used in this callback
			TUniquePtr<FTypedElementWidgetConstructor> ConstructorPtr(Constructor.Get());
			Constructor.Reset();

			return Callback(MoveTemp(ConstructorPtr), MatchedColumns);
		};

	CreateWidgetConstructors(PurposeRow, MatchApproach, Columns, Arguments, CallbackWrapper);
}

void UEditorDataStorageUi::RegisterWidgetPurpose(FName Purpose, EPurposeType Type, FText Description)
{
	RegisterWidgetPurpose(UE::Editor::DataStorage::FMapKey(Purpose), FPurposeInfo(Purpose, Type, Description));
}

bool UEditorDataStorageUi::RegisterWidgetFactory(FName Purpose, const UScriptStruct* Constructor)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));

	if (Storage->IsRowAvailable(PurposeRow))
	{
		return RegisterWidgetFactory(PurposeRow, Constructor);
	}
	
	UE_LOGF(LogEditorDataStorageUI, Warning, 
		"Unable to register widget factory '%ls' as purpose '%ls' isn't registered.", *Constructor->GetName(), *Purpose.ToString());

	return false;
}

bool UEditorDataStorageUi::RegisterWidgetFactory(
	FName Purpose, const UScriptStruct* Constructor, UE::Editor::DataStorage::Queries::FConditions Columns)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	
	if (Storage->IsRowAvailable(PurposeRow))
	{
		return RegisterWidgetFactory(PurposeRow, Constructor, Columns);
	}
	
	UE_LOGF(LogEditorDataStorageUI, Warning, 
			"Unable to register widget factory '%ls' as purpose '%ls' isn't registered.", *Constructor->GetName(), *Purpose.ToString());

	return false;
}

bool UEditorDataStorageUi::RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	if (Storage->IsRowAvailable(PurposeRow))
	{
		return RegisterWidgetFactory(PurposeRow, MoveTemp(Constructor));
	}
	
	UE_LOGF(LogEditorDataStorageUI, Warning, 
			"Unable to register widget factory as purpose '%ls' isn't registered.", *Purpose.ToString());
	return false;
}

bool UEditorDataStorageUi::RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor, 
	UE::Editor::DataStorage::Queries::FConditions Columns)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	
	if (Storage->IsRowAvailable(PurposeRow))
	{
		return RegisterWidgetFactory(PurposeRow, MoveTemp(Constructor), Columns);
	}
		
	UE_LOGF(LogEditorDataStorageUI, Warning, "Unable to register widget factory '%ls' as purpose '%ls' isn't registered.", 
			*Constructor->GetTypeInfo()->GetName(), *Purpose.ToString());
	return false;
}

void UEditorDataStorageUi::CreateWidgetConstructors(FName Purpose,
	const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	CreateWidgetConstructors(PurposeRow, Arguments, Callback);
}

void UEditorDataStorageUi::CreateWidgetConstructors(FName Purpose, EMatchApproach MatchApproach, 
	TArray<TWeakObjectPtr<const UScriptStruct>>& Columns, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetConstructorCallback& Callback)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	CreateWidgetConstructors(PurposeRow, MatchApproach, Columns, Arguments, Callback);
}

void UEditorDataStorageUi::ConstructWidgets(FName Purpose, const UE::Editor::DataStorage::FMetaDataView& Arguments,
	const WidgetCreatedCallback& ConstructionCallback)
{
	UE::Editor::DataStorage::RowHandle PurposeRow = Storage->LookupMappedRow(PurposeMappingDomain, UE::Editor::DataStorage::FMapKey(Purpose));
	ConstructWidgets(PurposeRow, Arguments, ConstructionCallback);
}

// Deprecated functions end here
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE // "TypedElementDatabaseUI"
