// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/Map.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/DecoratorWidgetConstructor.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Logging/LogMacros.h"
#include "UObject/ObjectMacros.h"

#include "TypedElementDatabaseUI.generated.h"

#define UE_API TEDSCORE_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class ICompatibilityProvider;
} // namespace UE::Editor::DataStorage


TEDSCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogEditorDataStorageUI, Log, All);

UCLASS(MinimalAPI)
class UEditorDataStorageUi final
	: public UObject
	, public UE::Editor::DataStorage::IUiProvider
{
	GENERATED_BODY()

public:
	~UEditorDataStorageUi() override = default;

	UE_API void Initialize(
		UE::Editor::DataStorage::ICoreProvider* StorageInterface, 
		UE::Editor::DataStorage::ICompatibilityProvider* StorageCompatibilityInterface);
	UE_API void PostInitialize(
		UE::Editor::DataStorage::ICoreProvider* StorageInterface, 
		UE::Editor::DataStorage::ICompatibilityProvider* StorageCompatibilityInterface);
	UE_API void Deinitialize();

	UE_API virtual UE::Editor::DataStorage::RowHandle RegisterWidgetPurpose(const FPurposeID& PurposeID, const FPurposeInfo& InPurposeInfo) override;
	UE_API virtual UE::Editor::DataStorage::RowHandle RegisterWidgetPurpose(const FPurposeInfo& InPurposeInfo) override;

	UE_API virtual bool RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor) override;
	UE_API virtual bool RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor,
		UE::Editor::DataStorage::Queries::FConditions Columns) override;
	UE_API virtual bool RegisterDecoratorWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, const UScriptStruct* Constructor,
		const UScriptStruct* Column) override;
	
	UE_API virtual void CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructionCallback& Callback) override;
	UE_API virtual void CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructionCallback& Callback) override;
	UE_API virtual void ConstructWidgets(UE::Editor::DataStorage::RowHandle PurposeRow, const UE::Editor::DataStorage::FMetaDataView& Arguments,
			const WidgetCreatedCallback& ConstructionCallback) override;

	// Deprecated functions
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API virtual void CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, 
	const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) override;
	UE_API virtual void CreateWidgetConstructors(UE::Editor::DataStorage::RowHandle PurposeRow, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) override;
	UE_API virtual bool RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow,
	TUniquePtr<FTypedElementWidgetConstructor>&& Constructor) override;
	UE_API virtual bool RegisterWidgetFactory(UE::Editor::DataStorage::RowHandle PurposeRow, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor, 
		UE::Editor::DataStorage::Queries::FConditions Columns) override;
	UE_API void RegisterWidgetPurpose(FName Purpose, EPurposeType Type, FText Description) override;
	UE_API bool RegisterWidgetFactory(FName Purpose, const UScriptStruct* Constructor) override;
	UE_API bool RegisterWidgetFactory(FName Purpose, const UScriptStruct* Constructor,
		UE::Editor::DataStorage::Queries::FConditions Columns) override;
	UE_API bool RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor) override;
	UE_API bool RegisterWidgetFactory(FName Purpose, TUniquePtr<FTypedElementWidgetConstructor>&& Constructor,
		UE::Editor::DataStorage::Queries::FConditions Columns) override;
	UE_API void CreateWidgetConstructors(FName Purpose,
		const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) override;
	UE_API void CreateWidgetConstructors(FName Purpose, EMatchApproach MatchApproach, TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		const UE::Editor::DataStorage::FMetaDataView& Arguments, const WidgetConstructorCallback& Callback) override;
	UE_API void ConstructWidgets(FName Purpose, const UE::Editor::DataStorage::FMetaDataView& Arguments,
		const WidgetCreatedCallback& ConstructionCallback) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// ~Deprecated functions
	
	UE_API TSharedPtr<SWidget> ConstructWidget(UE::Editor::DataStorage::RowHandle Row, FTypedElementWidgetConstructor& Constructor,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TSharedPtr<SWidget> ConstructInternalWidget(UE::Editor::DataStorage::RowHandle Row, FTypedElementWidgetConstructor& Constructor,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	
	UE_API void ListWidgetPurposes(const WidgetPurposeCallback& Callback) const override;

	UE_API bool SupportsExtension(FName Extension) const override;
	UE_API void ListExtensions(TFunctionRef<void(FName)> Callback) const override;

	/** Create the container widget that every TEDS UI widget is stored in */
	UE_API virtual TSharedPtr<UE::Editor::DataStorage::ITedsWidget> CreateContainerTedsWidget(UE::Editor::DataStorage::RowHandle UiRowHandle) const override;
	
	/** Get the table where TEDS UI widgets are stored */
	UE_API virtual UE::Editor::DataStorage::TableHandle GetWidgetTable() const override;

	UE_API virtual void GeneratePropertySearchers(TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>>& Results,
		TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const override;

	UE_API virtual void RegisterSearcherGeneratorForProperty(const FFieldClass* PropertyType, 
		PropertySearcherConstructorCallback PropertySearcherConstructor) override;

	UE_API virtual void UnregisterSearcherGeneratorForProperty(const FFieldClass* PropertyType) override;

	UE_API virtual void GeneratePropertySorters(TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>>& Results,
		TArrayView<TWeakObjectPtr<const UScriptStruct>> Columns) const override;

	UE_API virtual void RegisterSorterGeneratorForProperty(const FFieldClass* PropertyType, 
		PropertySorterConstructorCallback PropertySorterConstructor) override;

	UE_API virtual void UnregisterSorterGeneratorForProperty(const FFieldClass* PropertyType) override;
	
	/** Get the ID of the default TEDS UI widget purpose used to register default widgets for different types of data (e.g FText -> STextBlock) */
	UE_API virtual FPurposeID GetDefaultWidgetPurposeID() const;
	
	/** Get the ID of the general TEDS UI purpose used to register general purpose widgets for columns */
	UE_API virtual FPurposeID GetGeneralWidgetPurposeID() const;
	
	UE_API virtual UE::Editor::DataStorage::FHierarchyHandle GetWidgetPurposeHierarchyHandle() const override;
	
	/** Find the row handle for a purpose by looking it up using the purpose ID */
	UE_API virtual UE::Editor::DataStorage::RowHandle FindPurpose(const FPurposeID& PurposeID) const override;

private:
	UE_API void CreateStandardArchetypes();
	UE_API void RegisterQueries();
	UE_API void RegisterHierarchies();
	void PostDataStorageUpdate();

	using FWidgetConstructionMethod = TFunction<bool(UE::Editor::DataStorage::RowHandle, TArray<TWeakObjectPtr<const UScriptStruct>>)>;
	
	UE_API bool CreateSingleWidgetConstructor(
		UE::Editor::DataStorage::RowHandle FactoryRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments,
		TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes,
		const WidgetConstructionCallback& Callback);

	UE_API void CreateWidgetInstance(
		FTypedElementWidgetConstructor& Constructor,
		const UE::Editor::DataStorage::FMetaDataView& Arguments,
		const WidgetCreatedCallback& ConstructionCallback);

	UE_API void CreateWidgetInstance(
		const UScriptStruct* ConstructorType,
		const UE::Editor::DataStorage::FMetaDataView& Arguments,
		const WidgetCreatedCallback& ConstructionCallback);

	UE_API bool CreateWidgetConstructors_LongestMatch(
		const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories, 
		TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		FWidgetConstructionMethod ConstructionFunction);
	
	UE_API void CreateWidgetConstructors_ExactMatch(
		const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories,
		TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		FWidgetConstructionMethod ConstructionFunction);
	
	UE_API void CreateWidgetConstructors_SingleMatch(
		const TArray<UE::Editor::DataStorage::RowHandle>& WidgetFactories,
		TArray<TWeakObjectPtr<const UScriptStruct>>& Columns,
		FWidgetConstructionMethod ConstructionFunction);

	UE_API TSharedPtr<FTedsDecoratorWidgetConstructor> CreateSingleDecoratorWidget(
		UE::Editor::DataStorage::RowHandle FactoryRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments,
		TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes);

	// Register a widget factory for the provided purpose
	UE_API UE::Editor::DataStorage::RowHandle RegisterWidgetFactoryRow(UE::Editor::DataStorage::RowHandle PurposeRowHandle, bool bDecoratorWidgetFactory = false) const;

	// Register a unique factory for the provided purpose, clearing the info if there was any factory previously registered for the purpose
	UE_API UE::Editor::DataStorage::RowHandle RegisterUniqueWidgetFactoryRow(UE::Editor::DataStorage::RowHandle PurposeRowHandle, bool bDecoratorWidgetFactory = false) const;

	// Get all (non-decorator) factories for the provided purpose
	UE_API void GetWidgetFactories(UE::Editor::DataStorage::RowHandle PurposeRowHandle, TArray<UE::Editor::DataStorage::RowHandle>& OutFactories) const;

	// Get all decorator factories for the provided purpose
	UE_API void GetDecoratorWidgetFactories(UE::Editor::DataStorage::RowHandle PurposeRowHandle, TArray<UE::Editor::DataStorage::RowHandle>& OutFactories) const;

	// The query conditions for the provided factory
	UE_API const UE::Editor::DataStorage::Queries::FConditions& GetFactoryConditions(UE::Editor::DataStorage::RowHandle FactoryRow) const;

	// Create decorator widgets for the given TEDS UI widget
	UE_API TSharedPtr<SWidget> CreateDecoratorWidgets(UE::Editor::DataStorage::RowHandle WidgetRowHandle, TSharedPtr<SWidget> InternalWidget, FTypedElementWidgetConstructor& Constructor, const UE::Editor::DataStorage::FMetaDataView& Arguments);

	// Register observers for the given decorator + purpose combo to automatically add/remove the decorator widget when the column is added/removed from widget rows
	UE_API void RegisterDecoratorWidgetObservers(UE::Editor::DataStorage::RowHandle PurposeRowHandle, const UScriptStruct* DecoratorColumn);

	TMap<const FFieldClass*, PropertySorterConstructorCallback> PropertySorterConstructors;
	TMap<const FFieldClass*, PropertySearcherConstructorCallback> PropertySearcherConstructors;

	UE::Editor::DataStorage::TableHandle WidgetTable{ UE::Editor::DataStorage::InvalidTableHandle };
	UE::Editor::DataStorage::TableHandle WidgetPurposeTable{ UE::Editor::DataStorage::InvalidTableHandle };
	UE::Editor::DataStorage::TableHandle WidgetFactoryTable{ UE::Editor::DataStorage::InvalidTableHandle };
	UE::Editor::DataStorage::TableHandle DecoratorWidgetFactoryTable{ UE::Editor::DataStorage::InvalidTableHandle };

	UE::Editor::DataStorage::ICoreProvider* Storage{ nullptr };
	UE::Editor::DataStorage::ICompatibilityProvider* StorageCompatibility{ nullptr };

	TArray<UE::Editor::DataStorage::QueryHandle> DecoratorObservers;

	// Widget rows that need their decorators updated
	TSet<UE::Editor::DataStorage::RowHandle> WidgetsToUpdateDecorators;
	
	UE::Editor::DataStorage::FHierarchyHandle WidgetPurposesHierarchyHandle;
};

#undef UE_API
