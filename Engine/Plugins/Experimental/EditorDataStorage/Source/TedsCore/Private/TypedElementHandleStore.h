// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/Array.h"
#include "Containers/Deque.h"
#include "Templates/Function.h"
#include "Templates/MemoryOps.h"

namespace UE::Editor::DataStorage
{
	template<typename DataType, uint32 ReservationSize = 64>
	class THandleStore
	{
	public:
		struct FGeneration
		{
			uint32 Generation : 31;
			uint32 bIsAlive : 1;
		};

		class Handle
		{
		public:
			Handle() = default;
			explicit Handle(uint64 InPackedData)
				: PackedData(InPackedData)
			{};

			Handle(uint32 Index, uint32 Generation)
			{
				PackedData = (static_cast<uint64_t>(Index) << 32) | Generation;
			}

			uint64 Packed() const
			{
				return PackedData;
			}

			uint32_t Generation() const
			{
				return static_cast<uint32_t>(PackedData); // Unpack
			}

			uint32_t Index() const
			{
				return static_cast<uint32_t>(PackedData >> 32);  // Unpack
			}

			friend auto operator<=>(const Handle&, const Handle&) = default;
		private:
			uint64 PackedData = TNumericLimits<uint64>::Max();
		};

		using ListAliveEntriesConstCallback = TFunctionRef<void(Handle, const DataType&)>;
		using ListAliveEntriesCallback = TFunctionRef<void(Handle, DataType&)>;

		THandleStore() = default;
		~THandleStore();
		THandleStore(const THandleStore&) = delete;
		THandleStore(THandleStore&&) = delete;
		THandleStore& operator=(const THandleStore&) = delete;
		THandleStore& operator=(THandleStore&&) = delete;

		template<typename... Args>
		Handle Emplace(Args... Arguments);

		DataType& Get(Handle Entry);
		DataType& GetMutable(Handle Entry);
		const DataType& Get(Handle Entry) const;

		/** Removes the entry at the given handle if alive. */
		void Remove(Handle Entry);

		bool IsAlive(Handle Entry) const;
		void ListAliveEntries(const ListAliveEntriesConstCallback& Callback) const;
		void ListAliveEntries(const ListAliveEntriesConstCallback& Callback);
		void ListAliveEntries(const ListAliveEntriesCallback& Callback);

	private:
		TArray<DataType> Data;
		TArray<FGeneration> Generations;
		TDeque<uint32> RecycleBin; // Using a deque to avoid the same slot being constantly reused.
	};



	// Implementations
	template<typename DataType, uint32 ReservationSize>
	THandleStore<DataType, ReservationSize>::~THandleStore()
	{
		if constexpr (std::is_destructible_v<DataType> && !std::is_trivially_destructible_v<DataType>)
		{
			int32 Count = Data.Num();
			const DataType* EntryIt = Data.GetData();
			const FGeneration* GenerationIt = Generations.GetData();

			for (int32 Index = 0; Index < Count; ++Index)
			{
				if (GenerationIt->bIsAlive)
				{
					DestructItem(EntryIt);
				}

				++EntryIt;
				++GenerationIt;
			}
		}
		Data.SetNumUnsafeInternal(0); // Make sure array destructor does not run destructor on items again
	}

	template<typename DataType, uint32 ReservationSize>
	template<typename... Args>
	auto THandleStore<DataType, ReservationSize>::Emplace(Args... Arguments) -> Handle
	{
		if (RecycleBin.IsEmpty())
		{
			uint32 NewSize = Data.Num() + ReservationSize;
			Data.Reserve(NewSize);
			Generations.Reserve(NewSize);

			for (uint32 Index = Data.Num(); Index < NewSize; ++Index)
			{
				Data.AddZeroed();
				Generations.AddZeroed();
				RecycleBin.EmplaceLast(Index);
			}
		}

		uint32 Index = RecycleBin.Last();
		RecycleBin.PopLast();

		DataType& Entry = Data[Index];
		new(&Entry) DataType(Forward<Args>(Arguments)...);

		Handle Result(Index, Generations[Index].Generation);
		Generations[Index].bIsAlive = 1;
		return Result;
	}

	template<typename DataType, uint32 ReservationSize>
	DataType& THandleStore<DataType, ReservationSize>::Get(Handle Entry)
	{
		checkf(IsAlive(Entry), TEXT("Attempting to retrieve a dead entry from a Typed Element Handle Store."));
		return Data[Entry.Index()];
	}

	template<typename DataType, uint32 ReservationSize>
	DataType& THandleStore<DataType, ReservationSize>::GetMutable(Handle Entry)
	{
		checkf(IsAlive(Entry), TEXT("Attempting to retrieve a dead entry from a Typed Element Handle Store."));
		return Data[Entry.Index()];
	}

	template<typename DataType, uint32 ReservationSize>
	const DataType& THandleStore<DataType, ReservationSize>::Get(Handle Entry) const
	{
		checkf(IsAlive(Entry), TEXT("Attempting to retrieve a dead entry from a Typed Element Handle Store."));
		return Data[Entry.Index()];
	}

	template<typename DataType, uint32 ReservationSize>
	void THandleStore<DataType, ReservationSize>::Remove(Handle Entry)
	{
		if (IsAlive(Entry))
		{
			FGeneration& Generation = Generations[Entry.Index()];
			++Generation.Generation;
			Generation.bIsAlive = 0;
			if constexpr (std::is_destructible_v<DataType> && !std::is_trivially_destructible_v<DataType>)
			{
				DestructItem(&Data[Entry.Index()]);
			}
			RecycleBin.EmplaceLast(Entry.Index());
		}
	}

	template<typename DataType, uint32 ReservationSize>
	bool THandleStore<DataType, ReservationSize>::IsAlive(Handle Entry) const
	{
		return
			Entry.Index() < static_cast<uint32>(Generations.Num()) &&
			Generations[Entry.Index()].Generation == Entry.Generation();
	}

	template<typename DataType, uint32 ReservationSize>
	void THandleStore<DataType, ReservationSize>::ListAliveEntries(const ListAliveEntriesConstCallback& Callback) const
	{
		int32 Count = Data.Num();
		const DataType* EntryIt = Data.GetData();
		const FGeneration* GenerationIt = Generations.GetData();

		for (int32 Index = 0; Index < Count; ++Index)
		{
			if (GenerationIt->bIsAlive)
			{
				Handle DataHandle(Index, GenerationIt->Generation);
				Callback(DataHandle, *EntryIt);
			}

			++EntryIt;
			++GenerationIt;
		}
	}

	template<typename DataType, uint32 ReservationSize>
	void THandleStore<DataType, ReservationSize>::ListAliveEntries(const ListAliveEntriesConstCallback& Callback)
	{
		const_cast<const THandleStore<DataType, ReservationSize>*>(this)->ListAliveEntries(Callback);
	}

	template<typename DataType, uint32 ReservationSize>
	void THandleStore<DataType, ReservationSize>::ListAliveEntries(const ListAliveEntriesCallback& Callback)
	{
		int32 Count = Data.Num();
		DataType* EntryIt = Data.GetData();
		FGeneration* GenerationIt = Generations.GetData();

		for (int32 Index = 0; Index < Count; ++Index)
		{
			if (GenerationIt->bIsAlive)
			{
				Handle DataHandle(Index, GenerationIt->Generation);
				Callback(DataHandle, *EntryIt);
			}

			++EntryIt;
			++GenerationIt;
		}
	}
} // namespace UE::Editor::DataStorage
