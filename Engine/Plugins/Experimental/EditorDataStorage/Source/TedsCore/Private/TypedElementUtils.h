// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/ContainersFwd.h"
#include "Mass/EntityHandle.h"

namespace UE::Editor::DataStorage
{
	inline FMassEntityHandle RowToMassEntityConversion(RowHandle Row);
	inline RowHandle MassEntityToRowConversion(FMassEntityHandle Entity);

	inline TConstArrayView<FMassEntityHandle> RowsToMassEntitiesConversion(TConstArrayView<RowHandle> Rows);
	inline TArrayView<FMassEntityHandle> RowsToMassEntitiesConversion(TArrayView<RowHandle> Rows);
	inline TConstArrayView<RowHandle> MassEntitiesToRowsConversion(TConstArrayView<FMassEntityHandle> Entities);
	inline TArrayView<RowHandle> MassEntitiesToRowsConversion(TArrayView<FMassEntityHandle> Entities);
} // namespace UE::Editor::DataStorage

// Implementations;
namespace UE::Editor::DataStorage
{
	FMassEntityHandle RowToMassEntityConversion(RowHandle Row)
	{
		return FMassEntityHandle::FromNumber(Row);
	}

	RowHandle MassEntityToRowConversion(FMassEntityHandle Entity)
	{
		return Entity.AsNumber();
	}

	TConstArrayView<FMassEntityHandle> RowsToMassEntitiesConversion(TConstArrayView<RowHandle> Rows)
	{
		return *reinterpret_cast<TConstArrayView<FMassEntityHandle>*>(&Rows);
	}

	TArrayView<FMassEntityHandle> RowsToMassEntitiesConversion(TArrayView<RowHandle> Rows)
	{
		return *reinterpret_cast<TArrayView<FMassEntityHandle>*>(&Rows);
	}

	TConstArrayView<RowHandle> MassEntitiesToRowsConversion(TConstArrayView<FMassEntityHandle> Entities)
	{
		return *reinterpret_cast<TConstArrayView<RowHandle>*>(&Entities);
	}

	TArrayView<RowHandle> MassEntitiesToRowsConversion(TArrayView<FMassEntityHandle> Entities)
	{
		return *reinterpret_cast<TArrayView<RowHandle>*>(&Entities);
	}
} // namespace UE::Editor::DataStorage

// Several of the conversions between TEDS rows and Mass entities depend on the fact that they're both 64 bit integers so a cheap
// type reinterpretation can be done instead of having copy lists. Especially for large lists this can become quite costly.
static_assert(sizeof(FMassEntityHandle) == sizeof(UE::Editor::DataStorage::RowHandle),
	"Size of Mass entity and data storage row have gone out of sync.");
static_assert(alignof(FMassEntityHandle) == alignof(UE::Editor::DataStorage::RowHandle), 
	"Alignment of Mass entity and data storage row have gone out of sync.");