// Copyright Epic Games, Inc. All Rights Reserved.

#include "TypedElementsDataStorage.h"

#include "DataStorage/Features.h"
#include "DataStorage/Debug/Log.h"
#include "TypedElementDatabaseEnvironment.h"
#include "EditorDataStorageSettings.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Features/IModularFeatures.h"
#include "ISettingsModule.h"
#include "MassEntityTypes.h"
#include "Misc/CoreDelegates.h"
#include "Templates/IsPolymorphic.h"
#include "TypedElementDatabase.h"
#include "TypedElementDatabaseCompatibility.h"
#include "TypedElementDatabaseUI.h"
#include "TypedElementDataStorageSharedColumn.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "FEditorDataStorageModule"

namespace Private
{
	static bool GEnableTEDS = true;
	static FAutoConsoleVariableRef CVarEnableTEDS(
		TEXT("TEDS.Enable"),
		GEnableTEDS,
		TEXT("Enable the TEDS Editor Data Storage - Used as the central storage for editor data."),
		ECVF_ReadOnly
	);
} // namespace Private

// MASS uses CDO in a few places, making it a difficult to consistently register Type Element's Columns and Tags
// as they may have not been set up to impersonate MASS' Fragments and Tags yet. There are currently no longer
// any cases where TEDS relies on this but may happen again the future. For the standalone version a static
// can be used to initialize the impersonation before the CDO get a chance to run, but for a cooked editor this will
// not work.
/**
 * Typed Elements provides base classes for columns and tags. These directly map to fragments and tags in MASS.
 * To avoid deep and tight coupling between both systems, columns and tags don't directly inhered from MASS, but
 * are otherwise fully compatible. To allow MASS to do its type safety checks, this class updates the type
 * information so Typed Elements columns and tags present as MASS fragments and tags from MASS's perspective.
 */
void ImpersonateMassTagsAndFragments()
{
	// Have UE::Editor::DataStorage::FColumn impersonate a FMassFragment, which is the actual data storage when using MASS as a backend.
	static_assert(sizeof(UE::Editor::DataStorage::FColumn) == sizeof(FMassFragment),
		"In order for UE::Editor::DataStorage::FColumn to impersonate FMassFragment they need to be identical.");
	static_assert(!TIsPolymorphic<FMassFragment>::Value,
		"In order to be able to impersonate FMassFragment it can't have any virtual functions.");
	static_assert(!TIsPolymorphic<UE::Editor::DataStorage::FColumn>::Value,
		"In order to be able to use UE::Editor::DataStorage::FColumn to impersonate FMassFragment it can't have any virtual functions.");
	UE::Editor::DataStorage::FColumn::StaticStruct()->SetSuperStruct(FMassFragment::StaticStruct());

	// Have UE::Editor::DataStorage::FTag impersonate a FMassTag, which is the tag type when using MASS as a backend.
	static_assert(sizeof(UE::Editor::DataStorage::FTag) == sizeof(FMassTag),
		"In order for UE::Editor::DataStorage::FTag to impersonate FMassTag they need to be identical.");
	static_assert(!TIsPolymorphic<FMassTag>::Value,
		"In order to be able to impersonate FMassTag it can't have any virtual functions.");
	static_assert(!TIsPolymorphic<UE::Editor::DataStorage::FTag>::Value,
		"In order to be able to use UE::Editor::DataStorage::FTag to impersonate FMassTag it can't have any virtual functions.");
	UE::Editor::DataStorage::FTag::StaticStruct()->SetSuperStruct(FMassTag::StaticStruct());

	FTedsSharedColumn::StaticStruct()->SetSuperStruct(FMassConstSharedFragment::StaticStruct());
}

void FEditorDataStorageModule::StartupModule()
{
	// Setup the editor settings;
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		// automation
		SettingsModule->RegisterSettings("Editor", "Advanced", "The Editor Data Storage",
			LOCTEXT("DataStorageSettingsName", "The Editor Data Storage"),
			LOCTEXT("DataStorageSettingsDescription", "Configuration options for the central data storage used by various tools to store their data."),
			GetMutableDefault<UEditorDataStorageSettings>()
		);
	}

	// Load the dependent TypedElementFramework module (holding TypedElementRegistry) here so that it is guaranteed to be available in Shutdown
	// and it is shutdown AFTER FEditorDataStorageModule
	FModuleManager::Get().LoadModule(TEXT("TypedElementFramework"));

	// Make sure this is loaded in case it got missed due to commandlets
	FModuleManager::Get().LoadModule(TEXT("MassEntityEditor"));
	
	ImpersonateMassTagsAndFragments();

	// TEDS relies on ticking Mass for processors to run and to pump command queues
	// Commandlets do not run the same loop as the editor and does not set up tickables to execute each frame.
	if (!Private::GEnableTEDS || IsRunningCommandlet())
	{
		UE_LOGF(LogEditorDataStorage, Log, "TEDS has been disabled.");
		return;
	}

	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda(
		[this]()
		{
			if (!bInitialized)
			{
				using namespace UE::Editor::DataStorage;

				UE_LOGF(LogEditorDataStorage, Log, "Initializing");
				
				DataStorage = NewObject<UEditorDataStorage>();
				DataStorage->Initialize();

				DataStorageCompatibility = NewObject<UEditorDataStorageCompatibility>();
				DataStorageCompatibility->Initialize(DataStorage.Get());

				DataStorageUi = NewObject<UEditorDataStorageUi>();
				DataStorageUi->Initialize(DataStorage.Get(), DataStorageCompatibility.Get());

				ObjectReinstancingManager = NewObject<UTedsObjectReinstancingManager>();
				ObjectReinstancingManager->Initialize(*DataStorage, *DataStorageCompatibility);

				// Register the various DataStorage instances.
				IModularFeatures::Get().RegisterModularFeature(StorageFeatureName, DataStorage.Get());
				IModularFeatures::Get().RegisterModularFeature(CompatibilityFeatureName, DataStorageCompatibility.Get());
				IModularFeatures::Get().RegisterModularFeature(UiFeatureName, DataStorageUi.Get());
				OnEditorDataStorageFeaturesEnabled().Broadcast();

				// Allow any factories to register their content.
				TArray<UClass*> FactoryClasses;
				GetDerivedClasses(UEditorDataStorageFactory::StaticClass(), FactoryClasses);

				DataStorage->SetFactories(FactoryClasses);
				TArray<UEditorDataStorageFactory*> Factories;
				Factories.Reserve(FactoryClasses.Num());

				// First pass to call all registration without dependencies.
				for (UEditorDataStorage::FactoryIterator Iterator = DataStorage->CreateFactoryIterator(); Iterator; ++Iterator)
				{
					UEditorDataStorageFactory* Factory = *Iterator;
					
					Factory->RegisterTables(*DataStorage);
					Factory->RegisterTables(*DataStorage, *DataStorageCompatibility);
					
					Factory->RegisterTickGroups(*DataStorage);

					Factory->RegisterHierarchies(*DataStorage);
					
					Factory->RegisterRegistrationFilters(*DataStorageCompatibility);
					Factory->RegisterDealiaser(*DataStorageCompatibility);
					
					Factory->RegisterWidgetPurposes(*DataStorageUi);
					Factory->RegisterPropertySearchers(*DataStorageUi);
					Factory->RegisterPropertySorters(*DataStorageUi);
				}

				// Second pass to call all registration that would benefit or need the registration in the previous pass.
				for (UEditorDataStorage::FactoryIterator Iterator = DataStorage->CreateFactoryIterator(); Iterator; ++Iterator)
				{
					UEditorDataStorageFactory* Factory = *Iterator;
					
					Factory->RegisterQueries(*DataStorage);
					Factory->RegisterWidgetConstructors(*DataStorage, *DataStorageUi);
				}

				// Third pass to call post register.
				for (UEditorDataStorage::FactoryIterator Iterator = DataStorage->CreateFactoryIterator(); Iterator; ++Iterator)
				{
					UEditorDataStorageFactory* Factory = *Iterator;

					Factory->PostRegister(*DataStorage);
				}

				// Initialize the root scope row for the Editor Scope Tree system.
				DataStorage->GetEnvironment().GetScopeSystem().InitRootScope();

				UE_LOGF(LogEditorDataStorage, Log, "Initialized");

				bInitialized = true;
				
				DataStorage->PostInitialize();
				DataStorageCompatibility->PostInitialize(DataStorage);
				DataStorageUi->PostInitialize(DataStorage, DataStorageCompatibility);
			}
		});
	FCoreDelegates::OnExit.AddRaw(this, &FEditorDataStorageModule::ShutdownModule);
}

void FEditorDataStorageModule::ShutdownModule()
{
	if (bInitialized)
	{
		UE_LOGF(LogEditorDataStorage, Log, "Deinitializing");

		using namespace UE::Editor::DataStorage;
		DataStorage->ResetFactories();

		IModularFeatures::Get().UnregisterModularFeature(UiFeatureName, DataStorageUi.Get());
		IModularFeatures::Get().UnregisterModularFeature(CompatibilityFeatureName, DataStorageCompatibility.Get());
		IModularFeatures::Get().UnregisterModularFeature(StorageFeatureName, DataStorage.Get());

		if (UObjectInitialized())
		{
			ObjectReinstancingManager->Deinitialize();
			DataStorageUi->Deinitialize();
			DataStorageCompatibility->Deinitialize();
			DataStorage->Deinitialize();
		}

		bInitialized = false;
	}
}

void FEditorDataStorageModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (bInitialized)
	{
		Collector.AddReferencedObject(DataStorage);
		Collector.AddReferencedObject(DataStorageCompatibility);
		Collector.AddReferencedObject(DataStorageUi);
		Collector.AddReferencedObject(ObjectReinstancingManager);
	}
}

FString FEditorDataStorageModule::GetReferencerName() const
{
	return TEXT("TEDS: Editor Data Storage Core Module");
}

#undef LOCTEXT_NAMESPACE

// ============================================================================
// ICoreProvider Scope forwarders
// ============================================================================

using namespace UE::Editor::DataStorage;
using namespace UE::Editor::DataStorage::Scope;

RowHandle UEditorDataStorage::AddScopeRow()
{
	return GetEnvironment().GetScopeSystem().AddScopeRow();
}

RowHandle UEditorDataStorage::AddScopeRow(FStringView Label)
{
	return GetEnvironment().GetScopeSystem().AddScopeRow(Label);
}

RowHandle UEditorDataStorage::AddScopeRow(FStringView Label, RowHandle Parent)
{
	return GetEnvironment().GetScopeSystem().AddScopeRow(Label, Parent);
}

void UEditorDataStorage::RemoveScopeRow(RowHandle Row)
{
	GetEnvironment().GetScopeSystem().RemoveScopeRow(Row);
}

void UEditorDataStorage::SetParentScope(RowHandle Child, RowHandle Parent)
{
	GetEnvironment().GetScopeSystem().SetParentScope(Child, Parent);
}

RowHandle UEditorDataStorage::GetParentScope(RowHandle Row)
{
	return GetEnvironment().GetScopeSystem().GetParentScope(Row);
}

const void* UEditorDataStorage::GetScopeDataRaw(RowHandle Row, const UScriptStruct* ColumnType)
{
	return GetEnvironment().GetScopeSystem().GetScopeDataRaw(Row, ColumnType);
}

TArray<const UScriptStruct*> UEditorDataStorage::GetAllVisibleScopeColumns(RowHandle Row)
{
	return GetEnvironment().GetScopeSystem().GetAllVisibleScopeColumns(Row);
}

FScopeDataVersion UEditorDataStorage::SetScopeData(
	RowHandle Row, const UScriptStruct* ColumnType,
	ColumnCreationCallbackRef Initializer, ColumnCopyOrMoveCallback Relocator)
{
	return GetEnvironment().GetScopeSystem().SetScopeData(Row, ColumnType, Initializer, Relocator);
}

bool UEditorDataStorage::RemoveScopeData(RowHandle Row, const UScriptStruct* ColumnType)
{
	return GetEnvironment().GetScopeSystem().RemoveScopeData(Row, ColumnType);
}

FScopeDataVersion UEditorDataStorage::GetScopeDataVersion(
	RowHandle Row, const UScriptStruct* ColumnType)
{
	return GetEnvironment().GetScopeSystem().GetScopeDataVersion(Row, ColumnType);
}

FHierarchyHandle UEditorDataStorage::GetScopeHierarchy()
{
	return GetEnvironment().GetScopeSystem().GetScopeHierarchy();
}

RowHandle UEditorDataStorage::GetRootScope() const
{
	return GetEnvironment().GetScopeSystem().GetRootScope();
}

IMPLEMENT_MODULE(FEditorDataStorageModule, TedsCore)