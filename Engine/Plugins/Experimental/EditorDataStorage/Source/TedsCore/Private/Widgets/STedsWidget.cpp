// Copyright Epic Games, Inc. All Rights Reserved.

#include "STedsWidget.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

namespace UE::Editor::DataStorage::TedsWidget::Private
{
	bool bUseDefaultAttributeBindings = false;

	static FAutoConsoleVariableRef CVarUseDefaultAttributeBindings(
		TEXT("TEDS.UI.UseDefaultAttributeBindings"),
		bUseDefaultAttributeBindings,
		TEXT("If true, widgets created through TEDS UI will provide a default set of attribute bindings. Does not apply to existing widgets retroactively"),
		ECVF_Default
	);
}

namespace UE::Editor::DataStorage::Widgets
{
	void STedsWidget::Construct(const FArguments& InArgs)
	{
		UiRowHandle = InArgs._UiRowHandle;

		// If the Ui row wasn't already registered externally, register it with Teds
		if (UiRowHandle == UE::Editor::DataStorage::InvalidRowHandle)
		{
			RegisterTedsWidget(InArgs._Content.Widget);
		}

		if (UE::Editor::DataStorage::TedsWidget::Private::bUseDefaultAttributeBindings)
		{
			UE::Editor::DataStorage::FAttributeBinder Binder(UiRowHandle, GetStorageIfAvailable());

			SetColorAndOpacity(Binder.BindData(&FSlateColorColumn::Color, [](const FSlateColor& InColor)
			{
				if(InColor.IsColorSpecified())
				{
					return InColor.GetSpecifiedColor();
				}

				return FLinearColor::White;
			}, FSlateColor::UseForeground()));
		}

		SetContent(InArgs._Content.Widget);
	}

	void STedsWidget::RegisterTedsWidget(const TSharedPtr<SWidget>& InContentWidget)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* Storage = GetStorageIfAvailable();
		IUiProvider* StorageUi = GetStorageUiIfAvailable();

		// If TEDS is not enabled, STedsWidget will just behave like a regular widget
		if (!Storage || !StorageUi)
		{
			return;
		}
		
		const TableHandle WidgetTable = StorageUi->GetWidgetTable();
		if (WidgetTable == InvalidTableHandle)
		{
			return;
		}
		
		UiRowHandle = Storage->AddRow(WidgetTable);
		
		if (FTypedElementSlateWidgetReferenceColumn* WidgetReferenceColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(UiRowHandle))
		{
			WidgetReferenceColumn->TedsWidget = SharedThis(this);
			WidgetReferenceColumn->Widget = InContentWidget;
		}
	}

	void STedsWidget::SetContent(const TSharedRef< SWidget >& InContent)
	{
		if (ICoreProvider* Storage = GetStorageIfAvailable())
		{
			if (FTypedElementSlateWidgetReferenceColumn* WidgetReferenceColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(UiRowHandle))
			{
				// First we set the widget reference on the column
				WidgetReferenceColumn->Widget = InContent;

				if (IUiProvider::FWidgetConstructorPtr Constructor = WidgetReferenceColumn->WidgetConstructor.Pin())
				{
					TConstArrayView<const UScriptStruct*> AdditionalColumns = Constructor->GetAdditionalColumnsList();
					
					// When we are setting valid content, we want to add the additional columns to the widget row so it gets picked up by any queries
					// the widget constructor requires.
					if (InContent != SNullWidget::NullWidget)
					{
						Storage->AddColumns(UiRowHandle, AdditionalColumns);
					}
					// If we are setting to null content, i.e removing the widget we also remove the additional columns so the row stops matching any
					// queries designed to operate on the widget
					else
					{
						// RemoveColumns ensures if the columns don't exist on the row
						if (Storage->HasColumns(UiRowHandle, AdditionalColumns))
						{
							Storage->RemoveColumns(UiRowHandle, AdditionalColumns);
						}
					}
				}
			}
		}
		
		ChildSlot
		[
			InContent
		];
	}

	UE::Editor::DataStorage::RowHandle STedsWidget::GetRowHandle() const
	{
		return UiRowHandle;
	}

	TSharedRef<SWidget> STedsWidget::AsWidget()
	{
		return SharedThis(this);
	}

	ICoreProvider* STedsWidget::GetStorageIfAvailable()
	{
		using namespace UE::Editor::DataStorage;
		return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	}

	IUiProvider* STedsWidget::GetStorageUiIfAvailable()
	{
		using namespace UE::Editor::DataStorage;
		return GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
	}
}