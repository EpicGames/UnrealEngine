// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/SCompoundWidget.h"

struct FTypedElementWidgetConstructor;

namespace UE::Editor::DataStorage
{
	class IUiProvider;
	namespace Widgets
	{
		class STedsWidget;
	} // namespace UE::Editor::DataStorage::Widgets
} // namespace UE::Editor::DataStorage


template<>
struct TWidgetTypeTraits<UE::Editor::DataStorage::Widgets::STedsWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


namespace UE::Editor::DataStorage::Widgets
{
	/*
	 * All Teds widgets will be contained inside STedsWidget which acts like a container widget
	 * so we can have guaranteed access to the contents inside to dynamically update them if required.
	 * This widget is created and returned for any Teds widget requested for a row, regardless of if
	 * the actual internal widget exists or not.
	 * 
	 * Currently this is simply an SCompoundWidget
	 */
	class STedsWidget : public SCompoundWidget, public UE::Editor::DataStorage::ITedsWidget
	{
	public:

		SLATE_BEGIN_ARGS(STedsWidget)
			: _UiRowHandle(UE::Editor::DataStorage::InvalidRowHandle)
			, _Content()
		{
		}

		// The UI Row this widget will be assigned to
		SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, UiRowHandle)
	
		/** The actual widget content */
		SLATE_DEFAULT_SLOT(FArguments, Content)
	
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs );
	
		virtual void SetContent(const TSharedRef< SWidget >& InContent) override;

		virtual UE::Editor::DataStorage::RowHandle GetRowHandle() const override;
	
		virtual TSharedRef<SWidget> AsWidget() override;

	private:

		void RegisterTedsWidget(const TSharedPtr<SWidget>& InContentWidget);
	
		static ICoreProvider* GetStorageIfAvailable();
		static IUiProvider* GetStorageUiIfAvailable();

	private:
	
		RowHandle UiRowHandle = InvalidRowHandle;
	};
}
