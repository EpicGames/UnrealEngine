// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TypedElementMementoTranslators.generated.h"

/**
 * A MementoTranslator is used to convert columns of data to mementos and back again
 * Mementos are special types of columns that hold a subset of column data for dehydrating
 * a row.  This can be useful for reinstancing, undo/redo, LODs, etc.
 */
UCLASS(Abstract)
class UTedsMementoTranslatorBase : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * The type information of the column that this translator acts on
	 */
	virtual const UScriptStruct* GetColumnType() const
	{
		LowLevelFatalError(TEXT("Pure virtual not implemented (UTedsMementoTranslatorBase)"));
		return nullptr;
	}

	/**
	 * Type information for the memento
	 */
	virtual const UScriptStruct* GetMementoType() const
	{
		LowLevelFatalError(TEXT("Pure virtual not implemented (UTedsMementoTranslatorBase)"));
		return nullptr;
	}
	
	/**
	 * Implements the logic to convert a column to a memento
	 */
	virtual void TranslateColumnToMemento(const void* TypeErasedColumn, void* TypeErasedMemento) const PURE_VIRTUAL(UTedsMementoTranslatorBase, )
	/**
	 * Implements the logic to convert a memento to a column
	 */
	virtual void TranslateMementoToColumn(const void* TypeErasedMemento, void* TypeErasedColumn) const PURE_VIRTUAL(UTedsMementoTranslatorBase, )
	
	/**
	 * This column (if specified) will be added to the object row after the memento is restored
	 */
	const UScriptStruct* GetColumnToAddOnRestore() const;
	
protected:
	
	// Optional column that will be added to the object row after the memento is restored
	UPROPERTY()
	TObjectPtr<const UScriptStruct> ColumnToAddOnRestore = nullptr;
};

/**
 * Turn-key translator to convert columns to mementos and back
 * Creates a new struct that has the same UPROPERTIES the ColumnType
 * To opt-in a column to be mementoized, override the GetColumnType with the column that should be mementoized
 *
 * As an example, suppose there was a column defined as:
 * ```
 * USTRUCT()
 * struct FPathColumn : public FEditorDataStorageColumn
 * {
 *		GENERATED_BODY()
 *		
 *		UPROPERTY()
 *		FString Path;
 * };
 * ```
 * 
 * To opt into a default memento translator for FPathColumn, define a default translator:
 * ```
 * UCLASS()
 * class UTypedElementPathColumnTranslator : public UTypedElementDefaultMementoTranslator
 * {
 *     virtual const UScriptStruct* GetColumnType() const override { return FPathColumn::StaticStruct; }
 * };
 * ```
 * The system will take care of doing the required operations when rows are deleted that
 * that have both FTypedElementMementoOnDelete and FPathColumn columns
 */
UCLASS(Abstract)
class UTedsDefaultMementoTranslator : public UTedsMementoTranslatorBase
{
	GENERATED_BODY()
public:
	/**
	 * Override this in derived classes to get automatic mementoization of the given column
	 */
	virtual const UScriptStruct* GetColumnType() const override { return nullptr; }
	virtual const UScriptStruct* GetMementoType() const final override;

	virtual void PostInitProperties() final override;
	
	virtual void TranslateColumnToMemento(const void* Column, void* Memento) const final override;
	virtual void TranslateMementoToColumn(const void* TypeErasedMemento, void* TypeErasedColumn) const final override;

private:
	UPROPERTY()
	TObjectPtr<const UScriptStruct> MementoType = nullptr;

	TArray<const FProperty*> MementoizedColumnProperties;
	TArray<const FProperty*> MementoProperties;
};

