// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"

class FReferenceCollector;
class UEditorDataStorage;
class UEditorDataStorageCompatibility;
class UEditorDataStorageUi;
class UTedsObjectReinstancingManager;

class FEditorDataStorageModule : public IModuleInterface, public FGCObject
{
public:
	~FEditorDataStorageModule() override = default;

	//
	// IModuleInterface
	//

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	
	//
	// FGCObject
	//
	
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	FString GetReferencerName() const override;

private:
	TObjectPtr<UEditorDataStorage> DataStorage;
	TObjectPtr<UEditorDataStorageCompatibility> DataStorageCompatibility;
	TObjectPtr<UEditorDataStorageUi> DataStorageUi;
	TObjectPtr<UTedsObjectReinstancingManager> ObjectReinstancingManager;
	bool bInitialized{ false };
};
