// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsCore : ModuleRules
{
	public TedsCore(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"EditorSubsystem",
					"MassCore",
					"MassEntity",
					"MassEntityEditor",
					"TypedElementFramework",
					"SlateCore",
					"Slate",
					"UnrealEd"
				});

			PrivateDependencyModuleNames.AddRange(new string[] {});
			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}

		// This module needs AutoRTFM so disable the auto disable since its in a plugin.
		// The call to `ICompatibilityProvider::AddCompatibleObject` calls into here.
		bDisableAutoRTFMInstrumentation = false;
	}
}
