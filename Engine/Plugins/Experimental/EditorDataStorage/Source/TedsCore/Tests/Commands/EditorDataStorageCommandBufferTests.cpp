// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "TypedElementDatabaseScratchBuffer.h"
#include "Commands/EditorDataStorageCommandBuffer.h"
#include "Containers/ContainersFwd.h"
#include "Tests/TestHarnessAdapter.h"

struct FTestCommandA { int32 Value; };
struct FTestCommandB { float Value; };
struct FTestCommandC { bool Value; };
enum class ETestCommandType
{
	Nop, A, B, C
};

using TestCommandBuffer = UE::Editor::DataStorage::FCommandBuffer<FTestCommandA, FTestCommandB, FTestCommandC>;

constexpr const TCHAR* TypeToString(ETestCommandType Type)
{
	switch (Type)
	{
	case ETestCommandType::Nop:
		return TEXT("Nop");
	case ETestCommandType::A:
		return TEXT("A");
	case ETestCommandType::B:
		return TEXT("B");
	case ETestCommandType::C:
		return TEXT("C");
	default:
		return TEXT("<unknown>");
	}
}

struct FSequenceTestingProcessor
{
	explicit FSequenceTestingProcessor(TArray<ETestCommandType> InCommandTypes)
		: CommandTypes(MoveTemp(InCommandTypes))
	{}

	TArray<ETestCommandType> CommandTypes;
	int Index = 0;

	void operator()(const UE::Editor::DataStorage::FNopCommand&)
	{
		REQUIRE_MESSAGE(TEXT("An additional NOP command was issued when no more commands were expected."), Index < CommandTypes.Num());
		if (CommandTypes[Index] != ETestCommandType::Nop)
		{
			ADD_ERROR(FString::Printf(TEXT("Got nop command, but expected %s."), TypeToString(CommandTypes[Index])))
		}
		Index++;
	}
	void operator()(const FTestCommandA&)
	{
		REQUIRE_MESSAGE(TEXT("An additional A command was issued when no more commands were expected."), Index < CommandTypes.Num());
		if (CommandTypes[Index] != ETestCommandType::A)
		{
			ADD_ERROR(FString::Printf(TEXT("Got A command, but expected %s."), TypeToString(CommandTypes[Index])))
		}
		Index++;
	}
	void operator()(const FTestCommandB&)
	{
		REQUIRE_MESSAGE(TEXT("An additional B command was issued when no more commands were expected."), Index < CommandTypes.Num());
		if (CommandTypes[Index] != ETestCommandType::B)
		{
			ADD_ERROR(FString::Printf(TEXT("Got B command, but expected %s."), TypeToString(CommandTypes[Index])))
		}
		Index++;
	}
	void operator()(const FTestCommandC&)
	{
		REQUIRE_MESSAGE(TEXT("An additional C command was issued when no more commands were expected."), Index < CommandTypes.Num());
		if (CommandTypes[Index] != ETestCommandType::C)
		{
			ADD_ERROR(FString::Printf(TEXT("Got C command, but expected %s."), TypeToString(CommandTypes[Index])))
		}
		Index++;
	}
};

TEST_CASE_NAMED(EditorDataStorage_FCommandBuffer, "Editor::DataStorage::Command Buffer", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	
	FScratchBuffer ScratchBuffer;

	SECTION("Initialize")
	{
		TestCommandBuffer CommandBuffer;
		CommandBuffer.Initialize(ScratchBuffer);
	}

	SECTION("AddCommand")
	{
		TestCommandBuffer CommandBuffer;
		CommandBuffer.Initialize(ScratchBuffer);
		CommandBuffer.AddCommand(FTestCommandA{ .Value = 42 });
		ScratchBuffer.BatchDelete();
	}

	SECTION("Collect")
	{
		TestCommandBuffer CommandBuffer;
		CommandBuffer.Initialize(ScratchBuffer);
		CommandBuffer.AddCommand(FTestCommandA{ .Value = 42 });
		TestCommandBuffer::FCollection PendingCommands;
		SIZE_T CollectedCount = CommandBuffer.Collect(PendingCommands);
		CHECK_EQUALS(TEXT("After adding one command only one should be collected."), CollectedCount, SIZE_T(1));
		ScratchBuffer.BatchDelete();
	}

	SECTION("Collect with large number of commands.")
	{
		static constexpr int32 CommandCount = 10000;

		TestCommandBuffer CommandBuffer;
		CommandBuffer.Initialize(ScratchBuffer);
		for (int32 Counter = 0; Counter < CommandCount; ++Counter)
		{
			CommandBuffer.AddCommand(FTestCommandA{ .Value = Counter });
		}
		TestCommandBuffer::FCollection PendingCommands;
		SIZE_T CollectedCount = CommandBuffer.Collect(PendingCommands);
		CHECK_EQUALS(TEXT("After adding one command only one should be collected."), CollectedCount, SIZE_T(CommandCount));
		ScratchBuffer.BatchDelete();
	}
}

TEST_CASE_NAMED(EditorDataStorage_FCommandBuffer_FCollection, "Editor::DataStorage::Command Buffer::FCollection", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;

	FScratchBuffer ScratchBuffer;
	TestCommandBuffer CommandBuffer;
	CommandBuffer.Initialize(ScratchBuffer);

	SECTION("Add command")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);

		CHECK_EQUALS(TEXT("An empty command collection should not have nops."), Commands.GetCommandCount<FNopCommand>(), 0);
		Commands.AddCommand<FNopCommand>();
		CHECK_EQUALS(TEXT("After adding a nop command there should be a naop."), Commands.GetCommandCount<FNopCommand>(), 1);
	}

	SECTION("Add command with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);

		CHECK_EQUALS(TEXT("An empty command collection should not have nops."), Commands.GetCommandCount<FTestCommandA>(), 0);
		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		CHECK_EQUALS(TEXT("After adding a nop command there should be a naop."), Commands.GetCommandCount<FTestCommandA>(), 1);
	}

	SECTION("Replace command")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);

		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		CHECK_EQUALS(TEXT("The test command wasn't in the collection."), Commands.GetCommandCount<FTestCommandA>(), 1);
		CHECK_EQUALS(TEXT("An empty command collection should not have nops."), Commands.GetCommandCount<FNopCommand>(), 0);
		
		Commands.ReplaceCommand<FNopCommand>(0);
		CHECK_EQUALS(TEXT("After replacing there shouldn't be a test command anymore."), Commands.GetCommandCount<FTestCommandA>(), 0);
		CHECK_EQUALS(TEXT("After replacing with a nop command there should be a nop."), Commands.GetCommandCount<FNopCommand>(), 1);
	}

	SECTION("Replace command with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);

		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		CHECK_EQUALS(TEXT("Test command A wasn't in the collection."), Commands.GetCommandCount<FTestCommandA>(), 1);
		CHECK_EQUALS(TEXT("An empty command collection should not have a test command B."), Commands.GetCommandCount<FTestCommandB>(), 0);
		
		Commands.ReplaceCommand(0, FTestCommandB{ .Value = 3.14f });
		CHECK_EQUALS(TEXT("After replacing there shouldn't be a test command A anymore."), Commands.GetCommandCount<FTestCommandA>(), 0);
		CHECK_EQUALS(TEXT("After replacing with there should be a test command B."), Commands.GetCommandCount<FTestCommandB>(), 1);
	}

	SECTION("InsertCommandBefore with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 10 });
		Commands.AddCommand(FTestCommandA{ .Value = 20 });
		Commands.AddCommand(FTestCommandA{ .Value = 30 });

		Commands.InsertCommandBefore(1, FTestCommandB{ .Value = 42.0f });

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::B, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("InsertCommandBefore with argument before first")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 10 });
		Commands.AddCommand(FTestCommandA{ .Value = 20 });
		Commands.AddCommand(FTestCommandA{ .Value = 30 });

		Commands.InsertCommandBefore(0, FTestCommandB{ .Value = 42.0f });

		FSequenceTestingProcessor Processor({ ETestCommandType::B, ETestCommandType::A, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("Process")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		Commands.AddCommand(FTestCommandB{ .Value = 42.0f });

		struct FProcessor
		{
			int ACount = 0;
			int BCount = 0;

			void operator()(const FNopCommand&) { }
			void operator()(const FTestCommandA&) { ++ACount; }
			void operator()(const FTestCommandB&) { ++BCount; }
			void operator()(const FTestCommandC&) { }
		};
		FProcessor Processor;
		Commands.Process(Processor);

		CHECK_EQUALS(TEXT("Expected test command A to be touched exactly once."), Processor.ACount, int(1));
		CHECK_EQUALS(TEXT("Expected test command B to be touched exactly once."), Processor.BCount, int(1));
	}

	SECTION("InsertCommandBefore")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 10 });
		Commands.AddCommand(FTestCommandA{ .Value = 20 });
		Commands.AddCommand(FTestCommandA{ .Value = 30 });

		Commands.InsertCommandBefore<FNopCommand>(1);

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::Nop, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("InsertCommandBefore before first")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 10 });
		Commands.AddCommand(FTestCommandA{ .Value = 20 });
		Commands.AddCommand(FTestCommandA{ .Value = 30 });

		Commands.InsertCommandBefore<FNopCommand>(0);

		FSequenceTestingProcessor Processor({ ETestCommandType::Nop, ETestCommandType::A, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("ForEach")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		Commands.AddCommand(FTestCommandB{ .Value = 42.0f });

		struct FIterator
		{
			int ACount = 0;
			int BCount = 0;

			void operator()(const FNopCommand&) {}
			void operator()(const FTestCommandA&) { ++ACount; }
			void operator()(const FTestCommandB&) { ++BCount; }
			void operator()(const FTestCommandC&) {}
		};
		FIterator Iterator;
		Commands.ForEach(
			[&Iterator](int32 Index, TestCommandBuffer::TCommandVariant& Command)
			{
				Visit(Iterator, Command);
			});

		CHECK_EQUALS(TEXT("Expected test command A to be touched exactly once."), Iterator.ACount, int(1));
		CHECK_EQUALS(TEXT("Expected test command B to be touched exactly once."), Iterator.BCount, int(1));
	}

	SECTION("ForEach with culling")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 42 });
		Commands.AddCommand(FTestCommandB{ .Value = 42.0f });

		Commands.ForEach(
			[](int32 Index, TestCommandBuffer::TCommandVariant& Command)
			{
				struct FIterator
				{
					bool operator()(const FNopCommand&) { return true; }
					bool operator()(const FTestCommandA& Command) { return false; }
					bool operator()(const FTestCommandB&) { return true; }
					bool operator()(const FTestCommandC&) { return true; }
				};
				if (!Visit(FIterator(), Command))
				{
					Command.Emplace<FNopCommand>();
				}
			});
		struct FProcessor
		{
			int NopCount = 0;
			int ACount = 0;
			int BCount = 0;
			int CCount = 0;

			void operator()(const FNopCommand&) { ++NopCount; }
			void operator()(const FTestCommandA&) { ++ACount; }
			void operator()(const FTestCommandB&) { ++BCount; }
			void operator()(const FTestCommandC&) { ++CCount; }
		};
		FProcessor Processor;
		Commands.Process(Processor);

		CHECK_EQUALS(TEXT("Expected test command Nop to be touched exactly once."), Processor.NopCount, int(1));
		CHECK_EQUALS(TEXT("Expected test command A to not be touched."), Processor.ACount, int(0));
		CHECK_EQUALS(TEXT("Expected test command B to be touched exactly once."), Processor.BCount, int(1));
		CHECK_EQUALS(TEXT("Expected test command C to not be touched."), Processor.CCount, int(0));
	}

	SECTION("Sort")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandB{ .Value = 1.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		Commands.Sort<true /* Use stable sort */>(
			[](const TestCommandBuffer::TCommandVariant& Lhs, const TestCommandBuffer::TCommandVariant& Rhs)
			{
				return Lhs.GetIndex() < Rhs.GetIndex();
			});

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::A, ETestCommandType::B, ETestCommandType::B });
		Commands.Process(Processor);
	}

	SECTION("Reset")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandB{ .Value = 1.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		Commands.Reset();
		uint32 Count = Commands.GetTotalCommandCount();
		CHECK_EQUALS(TEXT("There are still commands in the command buffer after a reset."), Count, 0);

		FSequenceTestingProcessor Processor({});
		Commands.Process(Processor);
	}

	SECTION("GetCommandCount")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandB{ .Value = 1.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		uint32 CountA = Commands.GetCommandCount<FTestCommandA>();
		uint32 CountB = Commands.GetCommandCount<FTestCommandB>();
		CHECK_EQUALS(TEXT("Total A count didn't match the number of added commands."), CountA, 2);
		CHECK_EQUALS(TEXT("Total B count didn't match the number of added commands."), CountB, 2);
	}

	SECTION("GetTotalCommandCount")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandB{ .Value = 1.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		uint32 Count = Commands.GetTotalCommandCount();
		CHECK_EQUALS(TEXT("Total count didn't match the number of added commands."), Count, 4);
	}
}

TEST_CASE_NAMED(EditorDataStorage_FCommandBuffer_FOptimizer, "Editor::DataStorage::Command Buffer::FOptimizer", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;

	FScratchBuffer ScratchBuffer;
	TestCommandBuffer CommandBuffer;
	CommandBuffer.Initialize(ScratchBuffer);

	SECTION("Optimizing empty buffer")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		
		// Expecting no crash.
		TestCommandBuffer::FOptimizer Optimizer(Commands);
	}

	SECTION("Optimizing buffer with single command")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });

		// Expecting no crash.
		TestCommandBuffer::FOptimizer Optimizer(Commands);
	}

	SECTION("Optimizing buffer with nop command")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand<FNopCommand>();

		// Expecting no crash.
		TestCommandBuffer::FOptimizer Optimizer(Commands);
	}

	SECTION("Constructor corrects for nops before left")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		bool LeftMatches = Optimizer.GetLeft().IsType<FTestCommandA>();
		bool RightMatches = Optimizer.GetRight().IsType<FTestCommandB>();
		CHECK_EQUALS(TEXT("Left was not correctly set."), LeftMatches, true);
		CHECK_EQUALS(TEXT("Right was not correctly set."), RightMatches, true);
	}

	SECTION("Constructor corrects for nops before right")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		bool LeftMatches = Optimizer.GetLeft().IsType<FTestCommandA>();
		bool RightMatches = Optimizer.GetRight().IsType<FTestCommandB>();
		CHECK_EQUALS(TEXT("Left was not correctly set."), LeftMatches, true);
		CHECK_EQUALS(TEXT("Right was not correctly set."), RightMatches, true);
	}

	SECTION("GetLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		bool LeftMatches = Optimizer.GetLeft().IsType<FTestCommandA>();
		CHECK_EQUALS(TEXT("Left was not correctly retrieved."), LeftMatches, true);
	}

	SECTION("GetRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandB{ .Value = 2.0f });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		bool RightMatches = Optimizer.GetRight().IsType<FTestCommandB>();
		CHECK_EQUALS(TEXT("Right was not correctly retrieved."), RightMatches, true);
	}

	SECTION("MoveToNextLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 1);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 2);
	}

	SECTION("MoveToNextLeft but leave right as it's further out")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2
		Optimizer.MoveToNextRight(); // 2 -> 3
		Optimizer.MoveToNextLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 1);
		CHECK_EQUALS(TEXT("Right was incorrectly moved."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveToNextLeft and skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right was incorrectly moved."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveToNextRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 0);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 2);
	}

	SECTION("MoveToNextRight and skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand<FNopCommand>(); 
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 0);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 2);
	}

	SECTION("ResetRightNextToLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2
		
		Optimizer.ResetRightNextToLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 0);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 1);
	}

	SECTION("ResetRightNextToLeft and skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.ResetRightNextToLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 0);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 1);
	}

	SECTION("MoveToNextLeftAndRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.MoveToNextLeftAndRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 1);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveToNextLeftAndRight and skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.MoveToNextLeftAndRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 1);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveLeftBeforeRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		
		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.MoveLeftBeforeRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 1);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 2);
	}

	SECTION("MoveLeftBeforeRight skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2
		Optimizer.MoveToNextRight(); // 2 -> 3

		Optimizer.MoveLeftBeforeRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveLeftToRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.MoveLeftToRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("MoveLeftToRight skip nops")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 0 });
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand<FNopCommand>();
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextRight(); // 1 -> 2

		Optimizer.MoveLeftToRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), Optimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right not moved correctly."), Optimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("ReplaceLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		
		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.ReplaceLeft<FNopCommand>();

		FSequenceTestingProcessor Processor({ ETestCommandType::Nop, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("ReplaceLeft with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.ReplaceLeft(FTestCommandB{ .Value = 3.14f });

		FSequenceTestingProcessor Processor({ ETestCommandType::B, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("ReplaceRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.ReplaceRight<FNopCommand>();

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::Nop });
		Commands.Process(Processor);
	}

	SECTION("ReplaceRight with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.ReplaceRight(FTestCommandB{ .Value = 3.14f });

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::B });
		Commands.Process(Processor);
	}

	SECTION("InsertBeforeLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextLeft(); // 1 -> 2
		Optimizer.InsertBeforeLeft<FNopCommand>();

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::Nop, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("InsertBeforeLeft with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextLeft(); // 1 -> 2
		Optimizer.InsertBeforeLeft(FTestCommandB{ .Value = 3.14f });

		FSequenceTestingProcessor Processor({ ETestCommandType::A, ETestCommandType::B, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("InsertBeforeLeft before first")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		
		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.InsertBeforeLeft<FNopCommand>();

		FSequenceTestingProcessor Processor({ ETestCommandType::Nop, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("InsertBeforeLeft before first with argument")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.InsertBeforeLeft(FTestCommandB{ .Value = 3.14f });

		FSequenceTestingProcessor Processor({ ETestCommandType::B, ETestCommandType::A, ETestCommandType::A });
		Commands.Process(Processor);
	}

	SECTION("BranchOnLeft")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		Optimizer.MoveToNextLeftAndRight(); // 1->2, 2->3

		TestCommandBuffer::FOptimizer BranchedOptimizer = Optimizer.BranchOnLeft();
		CHECK_EQUALS(TEXT("Left not moved correctly."), BranchedOptimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right not moved correctly."), BranchedOptimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}

	SECTION("BranchOnRight")
	{
		TestCommandBuffer::FCollection Commands;
		CommandBuffer.Collect(Commands);
		Commands.AddCommand(FTestCommandA{ .Value = 1 });
		Commands.AddCommand(FTestCommandA{ .Value = 2 });
		Commands.AddCommand(FTestCommandA{ .Value = 3 });

		TestCommandBuffer::FOptimizer Optimizer(Commands);
		
		TestCommandBuffer::FOptimizer BranchedOptimizer = Optimizer.BranchOnRight();
		CHECK_EQUALS(TEXT("Left not moved correctly."), BranchedOptimizer.GetLeft().Get<FTestCommandA>().Value, 2);
		CHECK_EQUALS(TEXT("Right not moved correctly."), BranchedOptimizer.GetRight().Get<FTestCommandA>().Value, 3);
	}
}
#endif // WITH_TESTS
