// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Tests
{
	using namespace UE::Editor::DataStorage::Queries;
	
	BEGIN_DEFINE_SPEC(CommandBufferTimingTestFixture, "Editor.DataStorage.CommandBufferTiming", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		
		ICoreProvider* TedsInterface = nullptr;
	
		const FName TestTableName = "TestTable_CommandBufferTest";
		const FName TestEmptyTableName = "TestEmptyTable_CommandBufferTest";
	
		TableHandle TestTable = InvalidTableHandle;
		TableHandle TestEmptyTable = InvalidTableHandle;
	
		RowHandle TestRow  = InvalidRowHandle;
		QueryHandle Query = InvalidQueryHandle;
		QueryHandle Query2 = InvalidQueryHandle;
	
		enum EColumnState : int
		{
			BeforeQuery = 0,
			QueryRun = 1,
			CommandRun = 2
		};
	
		// Command to update the value of TestInt to CommandRun
		struct FTestCommand
		{
			void operator()()
			{
				if (Storage)
				{
					if (FTestColumnInt* TestColumnInt = Storage->GetColumn<FTestColumnInt>(Row))
					{
						TestColumnInt->TestInt = CommandRun;
					}
				}
			}
			
			ICoreProvider* Storage;
			RowHandle Row;
		};

		void RegisterTables()
		{
			TestTable = TedsInterface->FindTable(TestTableName);
			if (TestTable == InvalidTableHandle)
			{
				TestTable = TedsInterface->RegisterTable<FTestColumnInt>(TestTableName);
			}
			
			TestEmptyTable = TedsInterface->FindTable(TestEmptyTableName);
			if (TestEmptyTable == InvalidTableHandle)
			{
				TestEmptyTable = TedsInterface->RegisterTable(TestEmptyTableName);
			}
		}
		
		RowHandle CreateTestRow(TableHandle InTableHandle)
		{
			TestRow = TedsInterface->AddRow(InTableHandle);
			return TestRow;
		}

		QueryHandle RegisterQuery(FQueryDescription&& QueryDesc)
		{
			Query = TedsInterface->RegisterQuery(MoveTemp(QueryDesc));
			return Query;
		}
	
		void Cleanup()
		{
			TedsInterface->RemoveRow(TestRow);
			TedsInterface->UnregisterQuery(Query);
			TedsInterface->UnregisterQuery(Query2);
			TedsInterface = nullptr;
		}
	
	END_DEFINE_SPEC(CommandBufferTimingTestFixture)

	void CommandBufferTimingTestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			checkf(TedsInterface != nullptr, TEXT("Cannot run editor data storage tests without initialiing the TEDS interface"));
		
			RegisterTables();
		});
		
		xDescribe("CommandBuffer", [this]
		{
			It("Direct Query: Pushed Command should run immediately after direct query is run", [this]
			{
				// Create test row and set initial value of command
				{
					CreateTestRow(TestTable);
					
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					if (TestNotNull("Expected IntColumn to be present on row before query", IntColumn))
					{
						IntColumn->TestInt = BeforeQuery;
					}
				}
					
				// Run a direct query with a command to update value of the test column
				{
					FQueryDescription QueryDesc = Select().ReadWrite<FTestColumnInt>().Compile();
					Query = RegisterQuery(MoveTemp(QueryDesc));
						
					TedsInterface->RunQuery(Query, CreateDirectQueryCallbackBinding([this](IDirectQueryContext& Context, RowHandle Row, FTestColumnInt& IntColumn)
					{
						IntColumn.TestInt = QueryRun;
						Context.PushCommand(FTestCommand{.Storage = TedsInterface, .Row = Row});
					}));
				}
					
				// Check value of column post query run (the command is expected to have run at this point)
				{
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					if (TestNotNull("Expected IntColumn to be present on row after query", IntColumn))
					{
						TestTrue("Expected command to be run right after the direct query i.e IntColumn value to be CommandRun", IntColumn->TestInt == CommandRun);
					}
				}
			});
				
			It("Observer: Pushed Command should run immediately after observer is run", [this]
			{
				// Create test row without the column
				{
					CreateTestRow(TestEmptyTable);
					
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					TestNull("Expected IntColumn to NOT be present on row", IntColumn);
				}
					
				// Create an observer to look for addition of IntColumn and change the value in the callback and in a command
				{
					FQueryDescription QueryDesc =
						Select(
							TEXT("On TestIntColumn Added"),
							FObserver::OnAdd<FTestColumnInt>().SetExecutionMode(EExecutionMode::GameThread),
							[this](IQueryContext& Context, RowHandle Row, FTestColumnInt& IntColumn)
							{
								IntColumn.TestInt = QueryRun;
								Context.PushCommand(FTestCommand{.Storage = TedsInterface, .Row = Row});
							})
						.Compile();
					
					RegisterQuery(MoveTemp(QueryDesc));
				}
					
				// Add FTestColumnInt to row and check value
				{
					// This will run the observer which will then push the command, which is expected to run immediately after.
					TedsInterface->AddColumn(TestRow, FTestColumnInt{.TestInt = BeforeQuery});
					
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					if (TestNotNull("Expected IntColumn to be present on row", IntColumn))
					{
						TestTrue("Expected observer and command to be run i.e IntColumn value to be CommandRun", IntColumn->TestInt == CommandRun);
					}
				}
			});
				
			Describe("Processor: Pushed Command should run between the two processors (i.e when a tick phase ends)", [this]()
			{
				// Create the test row and set the initial value
				BeforeEach([this]()
				{
					CreateTestRow(TestTable);
						
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					if (TestNotNull("Expected IntColumn to be present on row before query", IntColumn))
					{
						IntColumn->TestInt = BeforeQuery;
					}
				});
					
				// Create the processors and then wait a frame to allow them to run
				LatentIt("", EAsyncExecution::TaskGraphMainTick, [this](const FDoneDelegate Done)
				{
					// Procesor to update the value of the column and push a command to update it again
					FQueryDescription UpdateQueryDesc = 
						Select(
						TEXT("Update TestInt Value"),
						FProcessor(EQueryTickPhase::PrePhysics, TedsInterface->GetQueryTickGroupName(EQueryTickGroups::Update))
						.SetExecutionMode(EExecutionMode::GameThread),
						[this](IQueryContext& Context, RowHandle Row, FTestColumnInt& IntColumn)
							{
								IntColumn.TestInt = QueryRun;
								Context.PushCommand(FTestCommand{.Storage = TedsInterface, .Row = Row});
							}
						)
					.Compile();
						
					Query = TedsInterface->RegisterQuery(MoveTemp(UpdateQueryDesc));
								
					// Processor to check the value of the column, it is expected that the command pushed by the previous processor is run before this processor
					// since it is in the next tick phase
					FQueryDescription CheckQueryDesc = 
						Select(
						TEXT("Check TestInt Value"),
						FProcessor(EQueryTickPhase::PostPhysics, TedsInterface->GetQueryTickGroupName(EQueryTickGroups::Update))
						.SetExecutionMode(EExecutionMode::GameThread),
						[this](IQueryContext& Context, RowHandle Row, const FTestColumnInt& IntColumn)
							{
								TestTrue("Expected command from processor in previous phase to be run i.e IntColumn value to be CommandRun", IntColumn.TestInt == CommandRun);
							}
						)
					.Compile();
						
					Query2 = TedsInterface->RegisterQuery(MoveTemp(CheckQueryDesc));
						
					// Wait a frame so the processors are run
					FTSTicker::GetCoreTicker().AddTicker(
						FTickerDelegate::CreateLambda([Done](float)
						{
							Done.Execute();
							return false;
						}),
						0.0f
					);
				});

				// Sanity check the values after all the tests are run
				AfterEach([this]()
				{
					FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(TestRow);
					if (TestNotNull("Expected IntColumn to be present on row after processor", IntColumn))
					{
						TestTrue("Expected IntColumn value to still be CommandRun", IntColumn->TestInt == CommandRun);
					}
				});
				
			});
		});
		
		AfterEach([this]()
		{
			Cleanup();
		});
	}
} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS
