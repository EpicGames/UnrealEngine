// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Debug/TypedElementDatabaseDebugTypes.h"
#include "Misc/CoreDelegates.h"
#if WITH_TESTS

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "Misc/AutomationTest.h"

#define TEST_IDENTIFIER_0 "StaticMesh"
#define TEST_IDENTIFIER_1 "Animation"
#define TEST_IDENTIFIER_2 "AudioClip"

namespace UE::Editor::DataStorage::Tests
{
	BEGIN_DEFINE_SPEC(DynamicColumnTestFixture, "Editor.DataStorage.DynamicColumns", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		ICoreProvider* TedsInterface = nullptr;
		const FName TestTableName = TEXT("TestTable_DynamicColumnsTest");

		TableHandle TestTable;
		TArray<RowHandle> Rows;
		TArray<QueryHandle> QueryHandles;

		TArray<FName> Identifiers;

		TableHandle RegisterTestTable() const
		{
			const TableHandle Table = TedsInterface->FindTable(TestTableName);
				
			if (Table != InvalidTableHandle)
			{
				return Table;
			}
				
			return TedsInterface->RegisterTable(
			{
				FTestColumnA::StaticStruct()
			},
			TestTableName);
		}

		RowHandle CreateTestRow(TableHandle InTableHandle)
		{
			RowHandle Row = TedsInterface->AddRow(InTableHandle);
			Rows.Add(Row);
			return Row;
		}

		QueryHandle RegisterQuery(FQueryDescription&& Query)
		{
			QueryHandle QueryHandle = TedsInterface->RegisterQuery(MoveTemp(Query));
			QueryHandles.Add(QueryHandle);
			return QueryHandle;
		}

	
	END_DEFINE_SPEC(DynamicColumnTestFixture)

	void DynamicColumnTestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTable = RegisterTestTable();
			Identifiers = { TEXT( TEST_IDENTIFIER_0 ), TEXT( TEST_IDENTIFIER_1 ), TEXT( TEST_IDENTIFIER_2 ) };
			TestTrue("", TedsInterface != nullptr);
		});
			
		Describe("", [this]
		{
			It(TEXT("Tags"), EAsyncExecution::TaskGraphMainThread, [this]()
			{
				// Add dynamic columns that are actually tags (ie. dataless)
				{
					for (int32 Index = 0; Index <= 2; ++Index)
					{
						CreateTestRow(TestTable);
					}

					TedsInterface->AddColumn<FTestDynamicTag>(Rows[0], Identifiers[0]);
					TedsInterface->AddColumn<FTestDynamicTag>(Rows[0], Identifiers[1]);

					TedsInterface->AddColumn<FTestDynamicTag>(Rows[1], Identifiers[0]);

					TedsInterface->AddColumn<FTestDynamicTag>(Rows[2], Identifiers[1]);

					// Check they were added
					// Note: There is no HasColumn function for syntactic sugar to get a dynamic column type
					TArray<const UScriptStruct*> DynamicTagTypes;
					
					DynamicTagTypes.Emplace(TedsInterface->FindDynamicColumnType<FTestDynamicTag>(Identifiers[0]));
					DynamicTagTypes.Emplace(TedsInterface->FindDynamicColumnType<FTestDynamicTag>(Identifiers[1]));
					
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[0], MakeConstArrayView({DynamicTagTypes[0], DynamicTagTypes[1]})));
					
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[1], MakeConstArrayView({DynamicTagTypes[0]})));
					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[1], MakeConstArrayView({DynamicTagTypes[1]})));

					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[2], MakeConstArrayView({DynamicTagTypes[0]})));
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[2], MakeConstArrayView({DynamicTagTypes[1]})));
				}
				
				// Check if two dynamic columns with the same string length are unique
				{
					using namespace UE::Editor::DataStorage::Queries;
					{
						FName FirstName = TColumn<FTestDynamicTag, "Truck">().Identifier;
						FName SecondName = TColumn<FTestDynamicTag, "Hello">().Identifier;
						TestNotEqual(TEXT("Different id with the same length should produce two different ids."), FirstName, SecondName);
					}
				}

				// Direct Query
				{
					{
						using namespace UE::Editor::DataStorage::Queries;

						TArray<RowHandle> RowsToMatch;
						TBitArray WasMatched;
						auto SetExpectedMatches = [&RowsToMatch, &WasMatched](TConstArrayView<RowHandle> Expectation)
						{
							RowsToMatch.Empty(RowsToMatch.Num());
							RowsToMatch.Append(Expectation);
							WasMatched.Reset();
							WasMatched.SetNum(Expectation.Num(), false);
						};
						auto GetMatchCount = [&WasMatched]()
						{
							return WasMatched.CountSetBits();
						};
						auto Callback = CreateDirectQueryCallbackBinding([this, &RowsToMatch, &WasMatched] (IDirectQueryContext& Context, const RowHandle* CallbackRows)
						{
							TConstArrayView<RowHandle> RowsView = MakeConstArrayView(CallbackRows, Context.GetRowCount());
							for (RowHandle Row : RowsView)
							{
								int32 Index = RowsToMatch.Find(Row);
								TestTrue(TEXT("Returned row in query is within expected match array"), Index != INDEX_NONE);
								if (Index != INDEX_NONE)
								{
									TestFalse("Returned row was not duplicated in the callback", WasMatched[Index]);
									WasMatched[Index] = true;
								}
							}
						});					

						{
							// Should match Rows[0]
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag>(Identifiers[0]) && TColumn<FTestDynamicTag>(Identifiers[1])).
								Compile());
						
							SetExpectedMatches({Rows[0]});
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows[0] (using compile time dynamic columns)
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag, TEST_IDENTIFIER_0>() && TColumn<FTestDynamicTag, TEST_IDENTIFIER_1>()).
								Compile());

							SetExpectedMatches({ Rows[0] });
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0 and 1
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag>(Identifiers[0])).
								Compile());
							SetExpectedMatches({Rows[0], Rows[1]});
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0] and Row[1]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0 and 1 (using compile time dynamic columns)
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag, TEST_IDENTIFIER_0>()).
								Compile());
							SetExpectedMatches({ Rows[0], Rows[1] });
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0] and Row[1]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0 and 2
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag>(Identifiers[1])).
								Compile());
							SetExpectedMatches({Rows[0], Rows[2]});
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0] and Row[2]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0 and 2 (using compile time dynamic columns)
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag, TEST_IDENTIFIER_1>()).
								Compile());
							SetExpectedMatches({ Rows[0], Rows[2] });
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match Row[0] and Row[2]", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0, 1 and 2
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag>(Identifiers[0]) || TColumn<FTestDynamicTag>(Identifiers[1])).
								Compile());
							SetExpectedMatches({Rows[0], Rows[1], Rows[2]});
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match All Rows", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0, 1 and 2 (using compile time dynamic columns)
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag, TEST_IDENTIFIER_0>() || TColumn<FTestDynamicTag, TEST_IDENTIFIER_1>()).
								Compile());
							SetExpectedMatches({ Rows[0], Rows[1], Rows[2] });
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match All Rows", Result.Count, GetMatchCount());
						}
						{
							// Should match Rows 0, 1 and 2
							QueryHandle Query = RegisterQuery(
								Select().
								Where(TColumn<FTestDynamicTag>()).
								Compile());
							SetExpectedMatches({Rows[0], Rows[1], Rows[2]});
							FQueryResult Result = TedsInterface->RunQuery(Query, Callback);
							TestEqual("Match All Rows using template", Result.Count, GetMatchCount());
						}
					}
				}
			});

			It(TEXT("Columns"), EAsyncExecution::TaskGraphMainThread, [this]()
			{
				using namespace UE::Editor::DataStorage::Queries;
				
				// Add dynamic columns that have data
				{
					for (int32 Index = 0; Index <= 2; ++Index)
					{
						CreateTestRow(TestTable);
					}

					TedsInterface->AddColumn(Rows[0], Identifiers[0], FTestDynamicColumn
					{
						.IntArray = {1, 2, 3},
					});
					
					TedsInterface->AddColumn(Rows[0], Identifiers[1], FTestDynamicColumn
					{
						.IntArray = {10, 11, 12, 13},
					});

					TedsInterface->AddColumn(Rows[1], Identifiers[0], FTestDynamicColumn
					{
						.IntArray = {14, 15, 16},
					});

					TedsInterface->AddColumn(Rows[2], Identifiers[1], FTestDynamicColumn
					{
						.IntArray = {11, 22, 33, 43},
					});

					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[0], TColumn<FTestDynamicColumn>(Identifiers[0]), TColumn<FTestDynamicColumn>(Identifiers[1])));
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[0], TColumn<FTestDynamicColumn, TEST_IDENTIFIER_0>(), TColumn<FTestDynamicColumn, TEST_IDENTIFIER_1>()));

					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[1], TColumn<FTestDynamicColumn>(Identifiers[0])));
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[1], TColumn<FTestDynamicColumn, TEST_IDENTIFIER_0>()));
					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[1], TColumn<FTestDynamicColumn>(Identifiers[1])));
					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[1], TColumn<FTestDynamicColumn, TEST_IDENTIFIER_1>()));

					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[2], TColumn<FTestDynamicColumn>(Identifiers[0])));
					TestFalse("Unexpected columns found", TedsInterface->HasColumns(Rows[2], TColumn<FTestDynamicColumn, TEST_IDENTIFIER_0>()));
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[2], TColumn<FTestDynamicColumn>(Identifiers[1])));
					TestTrue("Expected columns not found", TedsInterface->HasColumns(Rows[2], TColumn<FTestDynamicColumn, TEST_IDENTIFIER_1>()));
					
					using namespace UE::Editor::DataStorage::Queries;

					{
						QueryHandle Query = RegisterQuery(
									Select().
									Where(TColumn<FTestDynamicColumn>()).
									Compile());

						FQueryResult Result = TedsInterface->RunQuery(Query);
						TestEqual("Unexpected number of rows queried", Result.Count, 3);
					}
				}
			});
		});
		
		AfterEach([this]()
		{
			Identifiers.Empty(Identifiers.Num());
			for (RowHandle Row : Rows)
			{
				TedsInterface->RemoveRow(Row);
			}
			Rows.Empty(Rows.Num());

			for (QueryHandle QueryHandle : QueryHandles)
			{
				TedsInterface->UnregisterQuery(QueryHandle);
			}
			QueryHandles.Empty(QueryHandles.Num());

			TedsInterface = nullptr;
		});
	}
} // namespace UE::Editor::DataStorage::Tests

#undef TEST_IDENTIFIER_2
#undef TEST_IDENTIFIER_1
#undef TEST_IDENTIFIER_0

#endif // WITH_TESTS
