// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "DataStorage/Features.h"

#if WITH_TESTS

#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Tests
{
	BEGIN_DEFINE_SPEC(TedsHierarchyTestFixture, "Editor.DataStorage.Hierarchy", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		const FName TestTableName = TEXT("TestTable_HierarchyTestTable");
		const FName HierarchyName = TEXT("TedsHierarchyTestFixture_Hierarchy");
		FHierarchyHandle HierarchyHandle;
		ICoreProvider* TedsInterface = nullptr;
		TableHandle TestTable;
		TArray<RowHandle> Rows;

		TableHandle RegisterTestTable() const
		{
			const TableHandle Table = TedsInterface->FindTable(TestTableName);
					
			if (Table != InvalidTableHandle)
			{
				return Table;
			}
					
			return TedsInterface->RegisterTable(
			{
				FTestColumnB::StaticStruct()
			},
			TestTableName);
		}

		FHierarchyHandle RegisterHierarchy()
		{
			HierarchyHandle = TedsInterface->FindHierarchyByName(HierarchyName);
			if (!TedsInterface->IsValidHierarchyHandle(HierarchyHandle))
			{
				UE::Editor::DataStorage::FHierarchyRegistrationParams EntityHierarchyParams{.Name = HierarchyName};
				HierarchyHandle = TedsInterface->RegisterHierarchy(EntityHierarchyParams);
			}
			return HierarchyHandle;
		}

		RowHandle CreateTestRow(TableHandle InTableHandle)
		{
			RowHandle Row = TedsInterface->AddRow(InTableHandle);
			Rows.Add(Row);
			return Row;
		}

		struct FTreeNodes
		{
			RowHandle A = InvalidRowHandle;
			RowHandle B = InvalidRowHandle;
			RowHandle C = InvalidRowHandle;
			RowHandle D = InvalidRowHandle;
			RowHandle E = InvalidRowHandle;
		};

		int32 ComputeTreeSize(RowHandle Root)
		{
			int32 Count = 0;
			TedsInterface->WalkDepthFirst(HierarchyHandle, Root, [&Count](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
				{
					Count++;
				});
			return Count;
		};

		// Build out this topology:
		// A
		// - B
		FTreeNodes BuildSimpleTopology()
		{
			RowHandle A = CreateTestRow(TestTable);
			RowHandle B = CreateTestRow(TestTable);

			TedsInterface->SetParentRow(HierarchyHandle, B, A);

			return FTreeNodes
			{
				.A = A,
				.B = B,
			};
		};

		// Build out this topology:
		// A
		// - B
		// - C
		//   - D
		FTreeNodes BuildNestedTopology()
		{
			RowHandle A = CreateTestRow(TestTable);
			RowHandle B = CreateTestRow(TestTable);
			RowHandle C = CreateTestRow(TestTable);
			RowHandle D = CreateTestRow(TestTable);

			TedsInterface->SetParentRow(HierarchyHandle, B, A);
			TedsInterface->SetParentRow(HierarchyHandle, C, A);
			TedsInterface->SetParentRow(HierarchyHandle, D, C);

			return FTreeNodes
			{
				.A = A,
				.B = B,
				.C = C,
				.D = D
			};
		};
		
	END_DEFINE_SPEC(TedsHierarchyTestFixture)

	void TedsHierarchyTestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTable = RegisterTestTable();
			HierarchyHandle = RegisterHierarchy();
		});	

		It("Test Simple Topology", [this]()
			{
				auto Tree = BuildSimpleTopology();

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);

				TestEqual("", ComputeTreeSize(Tree.A), 2);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
			});
		
		It("Test Nested Topology", [this]()
		{
			auto Tree = BuildNestedTopology();

			TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
			TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
			TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);

			TestEqual("", ComputeTreeSize(Tree.A), 4);
			TestEqual("", ComputeTreeSize(Tree.B), 1);
			TestEqual("", ComputeTreeSize(Tree.C), 2);
			TestEqual("", ComputeTreeSize(Tree.D), 1);
		});
			
		It("Unparent childless leaf from single-child parent", [this]()
		{
				auto Tree = BuildSimpleTopology();
				TedsInterface->SetParentRow(HierarchyHandle, Tree.B, InvalidRowHandle);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);

				TestEqual("", ComputeTreeSize(Tree.A), 1);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
		});

		It("Delete childless leaf from single-child parent", [this]()
		{
				auto Tree = BuildSimpleTopology();
				TedsInterface->RemoveRow(Tree.B);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);

				TestEqual("", ComputeTreeSize(Tree.A), 1);
				TestEqual("", ComputeTreeSize(Tree.B), 0);
		});

		It("Delete single-child parent of childless leaf node ", [this]()
		{
				auto Tree = BuildSimpleTopology();
				TedsInterface->RemoveRow(Tree.A);
				Rows.Remove(Tree.A);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);

				TestEqual("", ComputeTreeSize(Tree.A), 0);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
		});
			
		It("Unparent childless leaf from multi-child parent", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->SetParentRow(HierarchyHandle, Tree.B, InvalidRowHandle);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);

				TestEqual("", ComputeTreeSize(Tree.A), 3);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
				TestEqual("", ComputeTreeSize(Tree.C), 2);
				TestEqual("", ComputeTreeSize(Tree.D), 1);
		});

		It("Unparent middle parent from middle parent", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->SetParentRow(HierarchyHandle, Tree.C, InvalidRowHandle);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);

				TestEqual("", ComputeTreeSize(Tree.A), 2);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
				TestEqual("", ComputeTreeSize(Tree.C), 2);
				TestEqual("", ComputeTreeSize(Tree.D), 1);
		});

		It("Unparent childless leaf from middle parent", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->SetParentRow(HierarchyHandle, Tree.D, InvalidRowHandle);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), InvalidRowHandle);

				TestEqual("", ComputeTreeSize(Tree.A), 3);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
				TestEqual("", ComputeTreeSize(Tree.C), 1);
				TestEqual("", ComputeTreeSize(Tree.D), 1);
		});

		It("Reparent childless leaf to childless leaf", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->SetParentRow(HierarchyHandle, Tree.B, Tree.D);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.D);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);

				TestEqual("", ComputeTreeSize(Tree.A), 4);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
				TestEqual("", ComputeTreeSize(Tree.C), 3);
				TestEqual("", ComputeTreeSize(Tree.D), 2);
		});

		It("Delete childless leaf of multi-child root", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->RemoveRow(Tree.B);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);

				TestEqual("ComputeTreeSize(Tree.A)", ComputeTreeSize(Tree.A), 3);
				TestEqual("ComputeTreeSize(Tree.B)", ComputeTreeSize(Tree.B), 0);
				TestEqual("ComputeTreeSize(Tree.C)", ComputeTreeSize(Tree.C), 2);
				TestEqual("ComputeTreeSize(Tree.D)", ComputeTreeSize(Tree.D), 1);
		});

		It("Delete childless leaf of middle parent", [this]()
		{
				auto Tree = BuildNestedTopology();
				TedsInterface->RemoveRow(Tree.D);

				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
				TestEqual("", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), InvalidRowHandle);

				TestEqual("", ComputeTreeSize(Tree.A), 3);
				TestEqual("", ComputeTreeSize(Tree.B), 1);
				TestEqual("", ComputeTreeSize(Tree.C), 1);
				TestEqual("", ComputeTreeSize(Tree.D), 0);
		});

		It("Destroy parent with multiple children orphans all children (does not cascade)", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->RemoveRow(Tree.A);
			Rows.Remove(Tree.A);

			TestTrue("B is still assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestTrue("C is still assigned", TedsInterface->IsRowAssigned(Tree.C));
			TestTrue("D is still assigned", TedsInterface->IsRowAssigned(Tree.D));
			TestEqual("B has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
			TestEqual("C has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), InvalidRowHandle);
			TestEqual("D still has C as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
		});

		AfterEach([this]()
		{
			for (RowHandle Row : Rows)
			{
				TedsInterface->RemoveRow(Row);
			}
			Rows.Empty(Rows.Num());

			HierarchyHandle = FHierarchyHandle();

			TedsInterface = nullptr;
		});
	}

	//
	// Relations-backed hierarchy tests — same behavioral tests as Legacy but using EBackend::Relations.
	// Verifies behavioral equivalence between the two hierarchy backends.
	//
	BEGIN_DEFINE_SPEC(TedsRelationsHierarchyTestFixture, "Editor.DataStorage.Hierarchy.Relations", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		const FName TestTableName = TEXT("TestTable_RelationsHierarchyTestTable");
		const FName HierarchyName = TEXT("TedsRelationsHierarchyTestFixture_Hierarchy");
		FHierarchyHandle HierarchyHandle;
		ICoreProvider* TedsInterface = nullptr;
		TableHandle TestTable;
		TArray<RowHandle> Rows;

		TableHandle RegisterTestTable() const
		{
			const TableHandle Table = TedsInterface->FindTable(TestTableName);
			if (Table != InvalidTableHandle)
			{
				return Table;
			}
			return TedsInterface->RegisterTable({ FTestColumnB::StaticStruct() }, TestTableName);
		}

		FHierarchyHandle RegisterHierarchy()
		{
			HierarchyHandle = TedsInterface->FindHierarchyByName(HierarchyName);
			if (!TedsInterface->IsValidHierarchyHandle(HierarchyHandle))
			{
				FHierarchyRegistrationParams Params
				{
					.Name = HierarchyName,
					.Backend = FHierarchyRegistrationParams::EBackend::Relations
				};
				HierarchyHandle = TedsInterface->RegisterHierarchy(Params);
			}
			return HierarchyHandle;
		}

		RowHandle CreateTestRow(TableHandle InTableHandle)
		{
			RowHandle Row = TedsInterface->AddRow(InTableHandle);
			Rows.Add(Row);
			return Row;
		}

		struct FTreeNodes
		{
			RowHandle A = InvalidRowHandle;
			RowHandle B = InvalidRowHandle;
			RowHandle C = InvalidRowHandle;
			RowHandle D = InvalidRowHandle;
			RowHandle E = InvalidRowHandle;
		};

		int32 ComputeTreeSize(RowHandle Root)
		{
			int32 Count = 0;
			TedsInterface->WalkDepthFirst(HierarchyHandle, Root, [&Count](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
			{
				Count++;
			});
			return Count;
		}

		FTreeNodes BuildSimpleTopology()
		{
			RowHandle A = CreateTestRow(TestTable);
			RowHandle B = CreateTestRow(TestTable);
			TedsInterface->SetParentRow(HierarchyHandle, B, A);
			return FTreeNodes{ .A = A, .B = B };
		}

		FTreeNodes BuildNestedTopology()
		{
			RowHandle A = CreateTestRow(TestTable);
			RowHandle B = CreateTestRow(TestTable);
			RowHandle C = CreateTestRow(TestTable);
			RowHandle D = CreateTestRow(TestTable);
			TedsInterface->SetParentRow(HierarchyHandle, B, A);
			TedsInterface->SetParentRow(HierarchyHandle, C, A);
			TedsInterface->SetParentRow(HierarchyHandle, D, C);
			return FTreeNodes{ .A = A, .B = B, .C = C, .D = D };
		}
	END_DEFINE_SPEC(TedsRelationsHierarchyTestFixture)

	void TedsRelationsHierarchyTestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTable = RegisterTestTable();
			HierarchyHandle = RegisterHierarchy();
		});

		// --- Behavioral equivalence tests (same as Legacy) ---

		It("Test Simple Topology", [this]()
		{
			auto Tree = BuildSimpleTopology();
			TestEqual("A has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
			TestEqual("B's parent is A", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
			TestEqual("Tree from A has 2 nodes", ComputeTreeSize(Tree.A), 2);
			TestEqual("Tree from B has 1 node", ComputeTreeSize(Tree.B), 1);
		});

		It("Test Nested Topology", [this]()
		{
			auto Tree = BuildNestedTopology();
			TestEqual("A has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.A), InvalidRowHandle);
			TestEqual("B's parent is A", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
			TestEqual("C's parent is A", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestEqual("D's parent is C", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
			TestEqual("Tree from A has 4 nodes", ComputeTreeSize(Tree.A), 4);
			TestEqual("Tree from C has 2 nodes", ComputeTreeSize(Tree.C), 2);
		});

		It("Orphan child from parent", [this]()
		{
			auto Tree = BuildSimpleTopology();
			TedsInterface->SetParentRow(HierarchyHandle, Tree.B, InvalidRowHandle);
			TestEqual("B has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
			TestEqual("A tree shrinks to 1", ComputeTreeSize(Tree.A), 1);
		});

		It("Reparent child to different parent", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->SetParentRow(HierarchyHandle, Tree.B, Tree.D);
			TestEqual("B's new parent is D", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.D);
			TestEqual("D has children", TedsInterface->HasChildren(HierarchyHandle, Tree.D), true);
			TestEqual("Tree from A has 4 nodes", ComputeTreeSize(Tree.A), 4);
			TestEqual("Tree from D has 2 nodes", ComputeTreeSize(Tree.D), 2);
		});

		It("HasChildren returns correct values", [this]()
		{
			auto Tree = BuildNestedTopology();
			TestTrue("A has children", TedsInterface->HasChildren(HierarchyHandle, Tree.A));
			TestTrue("C has children", TedsInterface->HasChildren(HierarchyHandle, Tree.C));
			TestFalse("B has no children", TedsInterface->HasChildren(HierarchyHandle, Tree.B));
			TestFalse("D has no children", TedsInterface->HasChildren(HierarchyHandle, Tree.D));
		});

		It("Walk pre-order visits all nodes", [this]()
		{
			auto Tree = BuildNestedTopology();
			TArray<RowHandle> VisitedTargets;
			TedsInterface->WalkDepthFirst(HierarchyHandle, Tree.A,
				[&VisitedTargets](const ICoreProvider&, RowHandle Owner, RowHandle Target)
				{
					VisitedTargets.Add(Target);
				}, ICoreProvider::ETraversalOrder::PreOrder);
			TestEqual("All 4 nodes visited", VisitedTargets.Num(), 4);
		});

		It("Walk post-order visits all nodes", [this]()
		{
			auto Tree = BuildNestedTopology();
			TArray<RowHandle> VisitedTargets;
			TedsInterface->WalkDepthFirst(HierarchyHandle, Tree.A,
				[&VisitedTargets](const ICoreProvider&, RowHandle Owner, RowHandle Target)
				{
					VisitedTargets.Add(Target);
				}, ICoreProvider::ETraversalOrder::PostOrder);
			TestEqual("All 4 nodes visited", VisitedTargets.Num(), 4);
		});

		It("Unparent leaf from multi-child parent", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->SetParentRow(HierarchyHandle, Tree.B, InvalidRowHandle);
			TestEqual("B has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
			TestEqual("C still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestEqual("D still has C as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
			TestEqual("Tree from A has 3 nodes", ComputeTreeSize(Tree.A), 3);
		});

		It("Unparent middle parent from root", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->SetParentRow(HierarchyHandle, Tree.C, InvalidRowHandle);
			TestEqual("C has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), InvalidRowHandle);
			TestEqual("D still has C as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
			TestEqual("B still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
			TestEqual("A tree has 2 nodes", ComputeTreeSize(Tree.A), 2);
			TestEqual("C tree has 2 nodes", ComputeTreeSize(Tree.C), 2);
		});

		It("Destroy parent orphans children (does not cascade)", [this]()
		{
			// Build A -> B, A -> C -> D
			auto Tree = BuildNestedTopology();
			// Destroy A (the root parent)
			TedsInterface->RemoveRow(Tree.A);
			Rows.Remove(Tree.A);

			// Children B and C must survive (orphaned, not cascaded)
			TestTrue("B is still assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestTrue("C is still assigned", TedsInterface->IsRowAssigned(Tree.C));
			TestTrue("D is still assigned", TedsInterface->IsRowAssigned(Tree.D));

			// B and C are now parentless
			TestEqual("B has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
			TestEqual("C has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), InvalidRowHandle);
			// D's parent is still C (unaffected by A's destruction)
			TestEqual("D still has C as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
		});

		It("Destroy middle node orphans its children", [this]()
		{
			auto Tree = BuildNestedTopology();
			// Destroy C (middle node, parent of D, child of A)
			TedsInterface->RemoveRow(Tree.C);
			Rows.Remove(Tree.C);

			// D must survive (orphaned)
			TestTrue("D is still assigned", TedsInterface->IsRowAssigned(Tree.D));
			TestEqual("D has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), InvalidRowHandle);
			// A and B are unaffected
			TestTrue("A is still assigned", TedsInterface->IsRowAssigned(Tree.A));
			TestTrue("B is still assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestEqual("B still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
		});

		It("Delete childless leaf from single-child parent", [this]()
		{
			auto Tree = BuildSimpleTopology();
			TedsInterface->RemoveRow(Tree.B);
			Rows.Remove(Tree.B);

			TestFalse("B is no longer assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestFalse("A has no children", TedsInterface->HasChildren(HierarchyHandle, Tree.A));
			TestEqual("Tree from A has 1 node", ComputeTreeSize(Tree.A), 1);
		});

		It("Delete single-child parent of childless leaf node", [this]()
		{
			auto Tree = BuildSimpleTopology();
			TedsInterface->RemoveRow(Tree.A);
			Rows.Remove(Tree.A);

			TestFalse("A is no longer assigned", TedsInterface->IsRowAssigned(Tree.A));
			TestTrue("B is still assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestEqual("B has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), InvalidRowHandle);
			TestEqual("Tree from B has 1 node", ComputeTreeSize(Tree.B), 1);
		});

		It("Unparent childless leaf from middle parent", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->SetParentRow(HierarchyHandle, Tree.D, InvalidRowHandle);

			TestEqual("D has no parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), InvalidRowHandle);
			TestEqual("C still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestEqual("B still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.B), Tree.A);
			TestFalse("C has no children", TedsInterface->HasChildren(HierarchyHandle, Tree.C));
			TestEqual("Tree from A has 3 nodes", ComputeTreeSize(Tree.A), 3);
		});

		It("Delete childless leaf of multi-child root", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->RemoveRow(Tree.B);
			Rows.Remove(Tree.B);

			TestFalse("B is no longer assigned", TedsInterface->IsRowAssigned(Tree.B));
			TestEqual("C still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestEqual("D still has C as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.D), Tree.C);
			TestEqual("Tree from A has 3 nodes", ComputeTreeSize(Tree.A), 3);
			TestEqual("Tree from C has 2 nodes", ComputeTreeSize(Tree.C), 2);
		});

		It("Delete childless leaf of middle parent", [this]()
		{
			auto Tree = BuildNestedTopology();
			TedsInterface->RemoveRow(Tree.D);
			Rows.Remove(Tree.D);

			TestFalse("D is no longer assigned", TedsInterface->IsRowAssigned(Tree.D));
			TestEqual("C still has A as parent", TedsInterface->GetParentRow(HierarchyHandle, Tree.C), Tree.A);
			TestFalse("C has no children", TedsInterface->HasChildren(HierarchyHandle, Tree.C));
			TestEqual("Tree from A has 3 nodes", ComputeTreeSize(Tree.A), 3);
		});

		It("IterateChildren visits only direct children and supports early exit", [this]()
		{
			auto Tree = BuildNestedTopology();

			// A has direct children B and C (not D)
			TArray<RowHandle> DirectChildren;
			bool bCompleted = TedsInterface->IterateChildren(HierarchyHandle, Tree.A,
				[&DirectChildren](const ICoreProvider&, RowHandle Child)
				{
					DirectChildren.Add(Child);
					return true;
				});
			TestTrue("Iteration completed", bCompleted);
			TestEqual("A has 2 direct children", DirectChildren.Num(), 2);
			TestTrue("Direct children contain B", DirectChildren.Contains(Tree.B));
			TestTrue("Direct children contain C", DirectChildren.Contains(Tree.C));
			TestFalse("D is not a direct child of A", DirectChildren.Contains(Tree.D));

			// C has only D as direct child
			DirectChildren.Reset();
			TedsInterface->IterateChildren(HierarchyHandle, Tree.C,
				[&DirectChildren](const ICoreProvider&, RowHandle Child)
				{
					DirectChildren.Add(Child);
					return true;
				});
			TestEqual("C has 1 direct child", DirectChildren.Num(), 1);
			TestEqual("C's only child is D", DirectChildren[0], Tree.D);

			// B is a leaf — no children
			DirectChildren.Reset();
			TedsInterface->IterateChildren(HierarchyHandle, Tree.B,
				[&DirectChildren](const ICoreProvider&, RowHandle Child)
				{
					DirectChildren.Add(Child);
					return true;
				});
			TestEqual("B has no direct children", DirectChildren.Num(), 0);

			// Early exit: stop after first child of A
			int32 VisitCount = 0;
			bCompleted = TedsInterface->IterateChildren(HierarchyHandle, Tree.A,
				[&VisitCount](const ICoreProvider&, RowHandle)
				{
					++VisitCount;
					return false; // stop immediately
				});
			TestFalse("Early exit returns false", bCompleted);
			TestEqual("Only one child visited before early exit", VisitCount, 1);
		});

		// ===================================================================
		// ParentChangedColumn -- Legacy (adjacency-list) backend
		// ===================================================================

		Describe("ParentChangedColumn (Legacy backend)", [this]()
		{
			It("GetParentChangedColumnType returns nullptr when not enabled", [this]()
			{
				TestNull("No column when not enabled",
					TedsInterface->GetParentChangedColumnType(HierarchyHandle));
			});

			It("GetParentChangedColumnType returns non-null when bEnableParentChangedColumn=true", [this]()
			{
				FHierarchyRegistrationParams Params;
				Params.Name = TEXT("TestHier_LegacyParentChanged");
				Params.bEnableParentChangedColumn = true;
				// Default Backend = Legacy
				FHierarchyHandle Handle = TedsInterface->RegisterHierarchy(Params);

				const UScriptStruct* Col = TedsInterface->GetParentChangedColumnType(Handle);
				TestNotNull("ParentChangedColumn exists", Col);
			});

			It("ParentChangedColumn is stamped on child after SetParentRow", [this]()
			{
				FHierarchyRegistrationParams Params;
				Params.Name = TEXT("TestHier_LegacyParentChangedStamp");
				Params.bEnableParentChangedColumn = true;
				FHierarchyHandle Handle = TedsInterface->RegisterHierarchy(Params);

				const UScriptStruct* Col = TedsInterface->GetParentChangedColumnType(Handle);
				if (!TestNotNull("ParentChangedColumn must exist", Col))
				{
					return;
				}

				RowHandle Parent = CreateTestRow(TestTable);
				RowHandle Child  = CreateTestRow(TestTable);

				TestFalse("Column absent before SetParentRow",
					TedsInterface->HasColumns(Child, TConstArrayView<const UScriptStruct*>({Col})));
				TedsInterface->SetParentRow(Handle, Child, Parent);
				TestTrue("Column stamped after SetParentRow",
					TedsInterface->HasColumns(Child, TConstArrayView<const UScriptStruct*>({Col})));
			});

			It("ParentChangedColumn is stamped on child after reparenting", [this]()
			{
				FHierarchyRegistrationParams Params;
				Params.Name = TEXT("TestHier_LegacyParentChangedReparent");
				Params.bEnableParentChangedColumn = true;
				FHierarchyHandle Handle = TedsInterface->RegisterHierarchy(Params);

				const UScriptStruct* Col = TedsInterface->GetParentChangedColumnType(Handle);
				if (!TestNotNull("ParentChangedColumn must exist", Col))
				{
					return;
				}

				RowHandle Parent1 = CreateTestRow(TestTable);
				RowHandle Parent2 = CreateTestRow(TestTable);
				RowHandle Child   = CreateTestRow(TestTable);

				TedsInterface->SetParentRow(Handle, Child, Parent1);
				// Simulate frame rollover by removing the column
				TedsInterface->RemoveColumns(Child, {Col});

				TedsInterface->SetParentRow(Handle, Child, Parent2);
				TestTrue("Column restamped after reparenting",
					TedsInterface->HasColumns(Child, TConstArrayView<const UScriptStruct*>({Col})));
			});
		});

		AfterEach([this]()
		{
			for (RowHandle Row : Rows)
			{
				TedsInterface->RemoveRow(Row);
			}
			Rows.Empty(Rows.Num());
			HierarchyHandle = FHierarchyHandle();
			TedsInterface = nullptr;
		});
	}

} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS
