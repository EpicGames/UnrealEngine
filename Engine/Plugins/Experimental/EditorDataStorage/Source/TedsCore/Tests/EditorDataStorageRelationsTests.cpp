// Copyright Epic Games, Inc. All Rights Reserved.

#include "Relations/EditorDataStorageRelationColumns.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "DataStorage/Features.h"

#if WITH_TESTS

#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Tests
{
	//
	// Comprehensive Relations API tests — tests the non-hierarchy relation operations,
	// exclusivity, batch creation, destruction policies, and hierarchical queries.
	//
	BEGIN_DEFINE_SPEC(TedsRelationsAPITestFixture, "Editor.DataStorage.Relations", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		const FName TestTableName = TEXT("TestTable_RelationsAPITestTable");
		ICoreProvider* TedsInterface = nullptr;
		TableHandle TestTable;
		TArray<RowHandle> Rows;
		TArray<QueryHandle> QueryHandles;
		TArray<RelationTypeHandle> RegisteredRelationTypes;

		TableHandle RegisterTestTable() const
		{
			const TableHandle Table = TedsInterface->FindTable(TestTableName);
			if (Table != InvalidTableHandle)
			{
				return Table;
			}
			return TedsInterface->RegisterTable({ FTestColumnB::StaticStruct() }, TestTableName);
		}

		RowHandle CreateTestRow()
		{
			RowHandle Row = TedsInterface->AddRow(TestTable);
			Rows.Add(Row);
			return Row;
		}

		RelationTypeHandle RegisterSimpleRelation(const FName& Name, bool bExclusiveObject = false, EHierarchyMode HierarchyMode = EHierarchyMode::Disabled)
		{
			FRelationRegistrationParams Params;
			Params.Name = Name;
			Params.Traits.HierarchyMode = HierarchyMode;
			Params.Traits.Object.bExclusive = bExclusiveObject;
			RelationTypeHandle Handle = TedsInterface->RegisterRelationType(Params);
			RegisteredRelationTypes.Add(Handle);
			return Handle;
		}

		QueryHandle RegisterTestQuery(FQueryDescription&& Query)
		{
			QueryHandle Handle = TedsInterface->RegisterQuery(MoveTemp(Query));
			QueryHandles.Add(Handle);
			return Handle;
		}

		TArray<RowHandle> RunQueryCollectRows(QueryHandle Query)
		{
			TArray<RowHandle> CollectedRows;
			TedsInterface->RunQuery(Query, Queries::CreateDirectQueryCallbackBinding(
				[&CollectedRows](IDirectQueryContext& Context, const RowHandle*)
				{
					CollectedRows.Append(Context.GetRowHandles());
				}));
			return CollectedRows;
		}
	END_DEFINE_SPEC(TedsRelationsAPITestFixture)

	void TedsRelationsAPITestFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTable = RegisterTestTable();
		});

		// ===== Relation Type Registration =====

		Describe("Type Registration", [this]()
		{
			It("RegisterRelationType returns valid handle", [this]()
			{
				RelationTypeHandle Handle = RegisterSimpleRelation(TEXT("TestRel_Register"));
				TestTrue("Handle is valid", TedsInterface->IsValidRelationType(Handle));
			});

			It("FindRelationType returns registered type", [this]()
			{
				const FName RelName = TEXT("TestRel_Find");
				RegisterSimpleRelation(RelName);
				RelationTypeHandle Found = TedsInterface->FindRelationType(RelName);
				TestTrue("Found is valid", TedsInterface->IsValidRelationType(Found));
			});

			It("FindRelationType returns invalid for unknown name", [this]()
			{
				RelationTypeHandle Found = TedsInterface->FindRelationType(TEXT("NonExistentRelation_12345"));
				TestEqual("Not found", Found, InvalidRelationTypeHandle);
			});

			It("GetRelationTypeTraits returns correct traits", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_Traits");
				Params.Traits.HierarchyMode = EHierarchyMode::IntervalEncoded;
				Params.Traits.Object.bExclusive = true;
				Params.Traits.Object.DestructionPolicy = FTedsRelationRoleTraits::EDestructionPolicy::CleanUp;
				RelationTypeHandle Handle = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Handle);

				const FTedsRelationTraits* Traits = TedsInterface->GetRelationTypeTraits(Handle);
				TestNotNull("Traits exist", Traits);
				if (Traits)
				{
					TestEqual("HierarchyMode is Versioned", Traits->HierarchyMode, EHierarchyMode::IntervalEncoded);
					TestTrue("Object is exclusive", Traits->Object.bExclusive);
				}
			});

			It("ListRelationTypes includes registered types", [this]()
			{
				const FName RelName = TEXT("TestRel_List");
				RelationTypeHandle Handle = RegisterSimpleRelation(RelName);

				bool bFound = false;
				TedsInterface->ListRelationTypes([&](RelationTypeHandle ListedHandle, const FName& ListedName)
				{
					if (ListedName == RelName)
					{
						bFound = true;
						TestEqual("Handle matches", ListedHandle, Handle);
					}
				});
				TestTrue("Type found in list", bFound);
			});

			It("Duplicate registration returns same handle", [this]()
			{
				const FName RelName = TEXT("TestRel_Duplicate");
				RelationTypeHandle First = RegisterSimpleRelation(RelName);
				RelationTypeHandle Second = RegisterSimpleRelation(RelName);
				TestEqual("Same handle returned", First, Second);
			});
		});

		// ===== Basic Relation CRUD =====

		Describe("Relation CRUD", [this]()
		{
			It("CreateRelation returns valid row", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Create"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				RowHandle RelRow = TedsInterface->CreateRelation(Type, A, B);
				TestNotEqual("Relation row is valid", RelRow, InvalidRowHandle);
			});

			It("HasRelation returns true after creation", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Has"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				TedsInterface->CreateRelation(Type, A, B);
				TestTrue("Relation exists", TedsInterface->HasRelation(Type, A, B));
			});

			It("HasRelation returns false for non-existent relation", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasNot"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				TestFalse("No relation yet", TedsInterface->HasRelation(Type, A, B));
			});

			It("HasRelation is directional", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Directional"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				TedsInterface->CreateRelation(Type, A, B);
				TestTrue("A->B exists", TedsInterface->HasRelation(Type, A, B));
				TestFalse("B->A does not exist", TedsInterface->HasRelation(Type, B, A));
			});

			It("DestroyRelation removes the relation", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Destroy"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				TedsInterface->CreateRelation(Type, A, B);
				bool bDestroyed = TedsInterface->DestroyRelation(Type, A, B);
				TestTrue("Destroy returned true", bDestroyed);
				TestFalse("Relation no longer exists", TedsInterface->HasRelation(Type, A, B));
			});

			It("DestroyRelation returns false for non-existent", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DestroyNone"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				bool bDestroyed = TedsInterface->DestroyRelation(Type, A, B);
				TestFalse("Nothing to destroy", bDestroyed);
			});
		});

		// ===== Subject/Object Queries =====

		Describe("Subject and Object Queries", [this]()
		{
			It("GetRelationObjects returns objects for a subject", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetObj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object1);
				TedsInterface->CreateRelation(Type, Subject, Object2);

				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, Subject, Objects);
				TestEqual("Two objects", Objects.Num(), 2);
				TestTrue("Contains Object1", Objects.Contains(Object1));
				TestTrue("Contains Object2", Objects.Contains(Object2));
			});

			It("GetRelationSubjects returns subjects for an object", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetSubj"));
				RowHandle Subject1 = CreateTestRow();
				RowHandle Subject2 = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject1, Object);
				TedsInterface->CreateRelation(Type, Subject2, Object);

				TArray<RowHandle> Subjects;
				TedsInterface->GetRelationSubjects(Type, Object, Subjects);
				TestEqual("Two subjects", Subjects.Num(), 2);
				TestTrue("Contains Subject1", Subjects.Contains(Subject1));
				TestTrue("Contains Subject2", Subjects.Contains(Subject2));
			});

			It("GetRelationObjects returns empty for row with no relations", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_NoObj"));
				RowHandle Row = CreateTestRow();

				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, Row, Objects);
				TestEqual("No objects", Objects.Num(), 0);
			});

			It("Destroying a relation updates Subject/Object queries", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DestroyUpdate"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object1);
				TedsInterface->CreateRelation(Type, Subject, Object2);
				TedsInterface->DestroyRelation(Type, Subject, Object1);

				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, Subject, Objects);
				TestEqual("One object remains", Objects.Num(), 1);
				TestTrue("Object2 remains", Objects.Contains(Object2));
			});
		});

		// ===== Exclusivity =====

		Describe("Exclusivity", [this]()
		{
			It("Exclusive Object rejects second relation with same subject", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Exclusive"), /*bExclusiveObject=*/true);
				RowHandle Child = CreateTestRow();
				RowHandle Parent1 = CreateTestRow();
				RowHandle Parent2 = CreateTestRow();

				RowHandle Rel1 = TedsInterface->CreateRelation(Type, Child, Parent1);
				TestNotEqual("First relation created", Rel1, InvalidRowHandle);

				// Second relation with same subject should be rejected (exclusive Object means one parent per child).
				// Object exclusivity is enforced by Mass internally.
				RowHandle Rel2 = TedsInterface->CreateRelation(Type, Child, Parent2);
				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, Child, Objects);
				TestTrue("Child has at most one parent", Objects.Num() == 1);
			});
		});

		// ===== Batch Creation =====

		Describe("Batch Operations", [this]()
		{
			It("BatchCreateRelations creates multiple relations", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Batch"));
				RowHandle S1 = CreateTestRow();
				RowHandle S2 = CreateTestRow();
				RowHandle S3 = CreateTestRow();
				RowHandle O1 = CreateTestRow();
				RowHandle O2 = CreateTestRow();
				RowHandle O3 = CreateTestRow();

				TArray<RowHandle> Subjects = { S1, S2, S3 };
				TArray<RowHandle> Objects = { O1, O2, O3 };
				TArray<RowHandle> OutRelationRows;

				TedsInterface->BatchCreateRelations(Type, Subjects, Objects, &OutRelationRows);

				TestEqual("Three relation rows created", OutRelationRows.Num(), 3);
				TestTrue("S1->O1 exists", TedsInterface->HasRelation(Type, S1, O1));
				TestTrue("S2->O2 exists", TedsInterface->HasRelation(Type, S2, O2));
				TestTrue("S3->O3 exists", TedsInterface->HasRelation(Type, S3, O3));
			});

		});

		// ===== Multiple Relation Types =====

		Describe("Multiple Relation Types", [this]()
		{
			It("Same rows can participate in different relation types", [this]()
			{
				RelationTypeHandle TypeA = RegisterSimpleRelation(TEXT("TestRel_MultiA"));
				RelationTypeHandle TypeB = RegisterSimpleRelation(TEXT("TestRel_MultiB"));
				RowHandle Row1 = CreateTestRow();
				RowHandle Row2 = CreateTestRow();

				TedsInterface->CreateRelation(TypeA, Row1, Row2);
				TedsInterface->CreateRelation(TypeB, Row2, Row1);

				TestTrue("TypeA: Row1->Row2", TedsInterface->HasRelation(TypeA, Row1, Row2));
				TestFalse("TypeA: Row2->Row1 not present", TedsInterface->HasRelation(TypeA, Row2, Row1));
				TestTrue("TypeB: Row2->Row1", TedsInterface->HasRelation(TypeB, Row2, Row1));
				TestFalse("TypeB: Row1->Row2 not present", TedsInterface->HasRelation(TypeB, Row1, Row2));
			});

			It("Destroying one type doesn't affect other types", [this]()
			{
				RelationTypeHandle TypeA = RegisterSimpleRelation(TEXT("TestRel_IsoA"));
				RelationTypeHandle TypeB = RegisterSimpleRelation(TEXT("TestRel_IsoB"));
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();

				TedsInterface->CreateRelation(TypeA, A, B);
				TedsInterface->CreateRelation(TypeB, A, B);

				TedsInterface->DestroyRelation(TypeA, A, B);
				TestFalse("TypeA destroyed", TedsInterface->HasRelation(TypeA, A, B));
				TestTrue("TypeB still exists", TedsInterface->HasRelation(TypeB, A, B));
			});
		});

		// ===== Destruction Policies =====

		Describe("Destruction Policies", [this]()
		{
			It("CleanUp: destroying Subject leaves Object alive", [this]()
			{
				// CleanUp is the default — only the relation row is removed
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_CleanupSubj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->RemoveRow(Subject);
				Rows.Remove(Subject);

				TestTrue("Object survives", TedsInterface->IsRowAssigned(Object));
				TestFalse("Relation is gone", TedsInterface->HasRelation(Type, Subject, Object));
			});

			It("CleanUp: destroying Object leaves Subject alive", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_CleanupObj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->RemoveRow(Object);
				Rows.Remove(Object);

				TestTrue("Subject survives", TedsInterface->IsRowAssigned(Subject));
				TestFalse("Relation is gone", TedsInterface->HasRelation(Type, Subject, Object));
			});

			It("Cascade on Object: relation is removed when Object is destroyed", [this]()
			{
				// Cascade destruction of the OTHER participant is deferred (processed next frame).
				// In this synchronous test we verify the relation is cleaned up and the cascade
				// policy is correctly configured.
				// TODO: verify cascaded participant (Subject) is destroyed after frame tick when test harness supports it.
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_CascadeObj");
				Params.Traits.Object.DestructionPolicy = FTedsRelationRoleTraits::EDestructionPolicy::Cascade;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Relation exists before destroy", TedsInterface->HasRelation(Type, Subject, Object));

				TedsInterface->RemoveRow(Object);
				Rows.Remove(Object);

				// Relation is removed immediately
				TestFalse("Relation is gone", TedsInterface->HasRelation(Type, Subject, Object));

				// Verify the cascade policy was configured correctly
				const FTedsRelationTraits* Traits = TedsInterface->GetRelationTypeTraits(Type);
				TestNotNull("Traits exist", Traits);
				if (Traits)
				{
					TestEqual("Object policy is Cascade",
						static_cast<uint8>(Traits->Object.DestructionPolicy),
						static_cast<uint8>(FTedsRelationRoleTraits::EDestructionPolicy::Cascade));
				}
			});

			It("Cascade on Subject: relation is removed when Subject is destroyed", [this]()
			{
				// TODO: verify cascaded participant (Object) is destroyed after frame tick when test harness supports it.
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_CascadeSubj");
				Params.Traits.Subject.DestructionPolicy = FTedsRelationRoleTraits::EDestructionPolicy::Cascade;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->RemoveRow(Subject);
				Rows.Remove(Subject);

				TestFalse("Relation is gone", TedsInterface->HasRelation(Type, Subject, Object));

				const FTedsRelationTraits* Traits = TedsInterface->GetRelationTypeTraits(Type);
				TestNotNull("Traits exist", Traits);
				if (Traits)
				{
					TestEqual("Subject policy is Cascade",
						static_cast<uint8>(Traits->Subject.DestructionPolicy),
						static_cast<uint8>(FTedsRelationRoleTraits::EDestructionPolicy::Cascade));
				}
			});

			It("Row in multiple relation types: each policy acts independently", [this]()
			{
				// Row participates in two relation types with different policies.
				// Destroying the Object of a CleanUp relation should NOT affect the
				// other relation.
				RelationTypeHandle CleanUpType = RegisterSimpleRelation(TEXT("TestRel_MultiPolicy_CU"));
				RelationTypeHandle OtherType = RegisterSimpleRelation(TEXT("TestRel_MultiPolicy_Other"));

				RowHandle Row = CreateTestRow();
				RowHandle CleanUpTarget = CreateTestRow();
				RowHandle OtherTarget = CreateTestRow();

				TedsInterface->CreateRelation(CleanUpType, Row, CleanUpTarget);
				TedsInterface->CreateRelation(OtherType, Row, OtherTarget);

				// Destroy CleanUpTarget — Row should survive, OtherType relation unaffected
				TedsInterface->RemoveRow(CleanUpTarget);
				Rows.Remove(CleanUpTarget);

				TestTrue("Row survives", TedsInterface->IsRowAssigned(Row));
				TestFalse("CleanUp relation gone", TedsInterface->HasRelation(CleanUpType, Row, CleanUpTarget));
				TestTrue("Other relation still exists", TedsInterface->HasRelation(OtherType, Row, OtherTarget));
			});

			It("Row in multiple relations of same type: destroying one Object leaves others", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_MultiSame"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();
				RowHandle Object3 = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object1);
				TedsInterface->CreateRelation(Type, Subject, Object2);
				TedsInterface->CreateRelation(Type, Subject, Object3);

				// Destroy one object — Subject survives, other relations unaffected
				TedsInterface->RemoveRow(Object2);
				Rows.Remove(Object2);

				TestTrue("Subject survives", TedsInterface->IsRowAssigned(Subject));
				TestFalse("Relation to Object2 is gone", TedsInterface->HasRelation(Type, Subject, Object2));
				TestTrue("Relation to Object1 still exists", TedsInterface->HasRelation(Type, Subject, Object1));
				TestTrue("Relation to Object3 still exists", TedsInterface->HasRelation(Type, Subject, Object3));
			});

			It("Cascade does not affect unrelated rows", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_CascadeIso");
				Params.Traits.Object.DestructionPolicy = FTedsRelationRoleTraits::EDestructionPolicy::Cascade;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();
				RowHandle Bystander = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->RemoveRow(Object);
				Rows.Remove(Object);

				TestTrue("Bystander survives", TedsInterface->IsRowAssigned(Bystander));
			});
		});

		// ===== Hierarchical Relations =====

		Describe("Hierarchical Relations", [this]()
		{
			It("GetRelationObject returns the object of a hierarchical relation", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HierParent"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle Child = CreateTestRow();
				RowHandle Parent = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Parent);
				RowHandle Result = TedsInterface->GetRelationObject(Type, Child);
				TestEqual("Parent matches", Result, Parent);
			});

			It("GetRelationSubjects returns direct children of a hierarchical relation", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HierChildren"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle Parent = CreateTestRow();
				RowHandle Child1 = CreateTestRow();
				RowHandle Child2 = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child1, Parent);
				TedsInterface->CreateRelation(Type, Child2, Parent);

				TArray<RowHandle> Children;
				TedsInterface->GetRelationSubjects(Type, Parent, Children);
				TestEqual("Two children", Children.Num(), 2);
				TestTrue("Contains Child1", Children.Contains(Child1));
				TestTrue("Contains Child2", Children.Contains(Child2));
			});

			It("IsDescendantOf returns true for direct child", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDesc1"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Root);
				TestTrue("Child is descendant of Root", TedsInterface->IsDescendantOf(Type, Child, Root));
				TestFalse("Root is not descendant of Child", TedsInterface->IsDescendantOf(Type, Root, Child));
			});

			It("IsDescendantOf returns true for indirect descendant", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDesc2"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();

				TedsInterface->CreateRelation(Type, B, A); // B is child of A
				TedsInterface->CreateRelation(Type, C, B); // C is child of B

				TestTrue("C is descendant of A", TedsInterface->IsDescendantOf(Type, C, A));
				TestTrue("C is descendant of B", TedsInterface->IsDescendantOf(Type, C, B));
				TestTrue("B is descendant of A", TedsInterface->IsDescendantOf(Type, B, A));
				TestFalse("A is not descendant of C", TedsInterface->IsDescendantOf(Type, A, C));
			});

			It("IsDescendantOf returns false for self (default)", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDescSelf"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				TestFalse("Not descendant of self by default", TedsInterface->IsDescendantOf(Type, A, A));
				TestFalse("Not descendant of self explicit false", TedsInterface->IsDescendantOf(Type, A, A, /*bIncludeSelf=*/false));
			});

			It("IsDescendantOf bIncludeSelf=true: row is treated as its own descendant", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDescSelfTrue"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();
				TedsInterface->CreateRelation(Type, Child, Root);

				// Self-test: each row is its own descendant with bIncludeSelf
				TestTrue("Root is descendant of itself with bIncludeSelf", TedsInterface->IsDescendantOf(Type, Root, Root, /*bIncludeSelf=*/true));
				TestTrue("Child is descendant of itself with bIncludeSelf", TedsInterface->IsDescendantOf(Type, Child, Child, /*bIncludeSelf=*/true));

				// Normal descendant still works
				TestTrue("Child is still descendant of Root", TedsInterface->IsDescendantOf(Type, Child, Root, /*bIncludeSelf=*/true));

				// bIncludeSelf does not make unrelated rows pass
				RowHandle Unrelated = CreateTestRow();
				TestFalse("Unrelated not descendant of Root even with bIncludeSelf", TedsInterface->IsDescendantOf(Type, Unrelated, Root, /*bIncludeSelf=*/true));
			});

			It("IsAncestorOf bIncludeSelf=true: row is treated as its own ancestor", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsAncSelfTrue"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();
				TedsInterface->CreateRelation(Type, Child, Root);

				TestTrue("Root is ancestor of itself with bIncludeSelf", TedsInterface->IsAncestorOf(Type, Root, Root, /*bIncludeSelf=*/true));
				TestFalse("Root is not ancestor of itself without bIncludeSelf", TedsInterface->IsAncestorOf(Type, Root, Root));
				TestTrue("Root is still ancestor of Child", TedsInterface->IsAncestorOf(Type, Root, Child, /*bIncludeSelf=*/true));
			});

			It("IsDescendantOf returns false for unrelated rows", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDescUnrel"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();

				TedsInterface->CreateRelation(Type, B, A); // B is child of A, C is unrelated
				TestFalse("C is not descendant of A", TedsInterface->IsDescendantOf(Type, C, A));
			});

			It("GetHierarchyRoot returns root of tree", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Root"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Mid = CreateTestRow();
				RowHandle Leaf = CreateTestRow();

				TedsInterface->CreateRelation(Type, Mid, Root);
				TedsInterface->CreateRelation(Type, Leaf, Mid);

				TestEqual("Root of Root is Root", TedsInterface->GetHierarchyRoot(Type, Root), Root);
				TestEqual("Root of Mid is Root", TedsInterface->GetHierarchyRoot(Type, Mid), Root);
				TestEqual("Root of Leaf is Root", TedsInterface->GetHierarchyRoot(Type, Leaf), Root);
			});

			It("GetHierarchyDepth returns correct depths", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Depth"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Mid = CreateTestRow();
				RowHandle Leaf = CreateTestRow();

				TedsInterface->CreateRelation(Type, Mid, Root);
				TedsInterface->CreateRelation(Type, Leaf, Mid);

				TestEqual("Root depth is 0", TedsInterface->GetHierarchyDepth(Type, Root), 0);
				TestEqual("Mid depth is 1", TedsInterface->GetHierarchyDepth(Type, Mid), 1);
				TestEqual("Leaf depth is 2", TedsInterface->GetHierarchyDepth(Type, Leaf), 2);
			});

			It("GetDescendants returns all descendants", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Desc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();
				RowHandle D = CreateTestRow();

				TedsInterface->CreateRelation(Type, B, A);
				TedsInterface->CreateRelation(Type, C, A);
				TedsInterface->CreateRelation(Type, D, C);

				TArray<RowHandle> Descendants;
				TedsInterface->GetDescendants(Type, A, Descendants);
				TestEqual("Three descendants", Descendants.Num(), 3);
				TestTrue("Contains B", Descendants.Contains(B));
				TestTrue("Contains C", Descendants.Contains(C));
				TestTrue("Contains D", Descendants.Contains(D));

				TArray<RowHandle> CDescendants;
				TedsInterface->GetDescendants(Type, C, CDescendants);
				TestEqual("One descendant of C", CDescendants.Num(), 1);
				TestTrue("Contains D", CDescendants.Contains(D));
			});

			It("GetAncestors returns path to root", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Anc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();

				TedsInterface->CreateRelation(Type, B, A);
				TedsInterface->CreateRelation(Type, C, B);

				TArray<RowHandle> Ancestors;
				TedsInterface->GetAncestors(Type, C, Ancestors);
				TestEqual("Two ancestors", Ancestors.Num(), 2);
				TestEqual("First ancestor is B (parent)", Ancestors[0], B);
				TestEqual("Second ancestor is A (root)", Ancestors[1], A);
			});

			It("TraverseDescendants visits all nodes in pre-order", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Traverse"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child1 = CreateTestRow();
				RowHandle Child2 = CreateTestRow();
				RowHandle GrandChild = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child1, Root);
				TedsInterface->CreateRelation(Type, Child2, Root);
				TedsInterface->CreateRelation(Type, GrandChild, Child1);

				TArray<RowHandle> Visited;
				TedsInterface->TraverseDescendants(Type, Root,
					[&Visited](RowHandle Current, RowHandle Parent, int32 Depth)
					{
						Visited.Add(Current);
					}, ICoreProvider::ETraversalOrder::PreOrder);

				TestEqual("Three descendants visited", Visited.Num(), 3);
				TestTrue("Contains Child1", Visited.Contains(Child1));
				TestTrue("Contains Child2", Visited.Contains(Child2));
				TestTrue("Contains GrandChild", Visited.Contains(GrandChild));
			});

			It("TraverseDescendants respects MaxDepth", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_MaxDepth"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();
				RowHandle GrandChild = CreateTestRow();
				RowHandle GreatGrandChild = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Root);
				TedsInterface->CreateRelation(Type, GrandChild, Child);
				TedsInterface->CreateRelation(Type, GreatGrandChild, GrandChild);

				TArray<RowHandle> Visited;
				TedsInterface->TraverseDescendants(Type, Root,
					[&Visited](RowHandle Current, RowHandle Parent, int32 Depth)
					{
						Visited.Add(Current);
					}, ICoreProvider::ETraversalOrder::PreOrder, /*MaxDepth=*/1);

				TestEqual("Only direct children visited", Visited.Num(), 1);
				TestTrue("Contains Child", Visited.Contains(Child));
				TestFalse("Does not contain GrandChild", Visited.Contains(GrandChild));
			});
		});

		// ===== Query Condition Filtering (Phase 4) =====

		Describe("Query Conditions", [this]()
		{
			It("IsSubjectOf filters to only rows that are subjects", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsSubj"));
				RowHandle Subject1 = CreateTestRow();
				RowHandle Subject2 = CreateTestRow();
				RowHandle Object = CreateTestRow();
				RowHandle Unrelated = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject1, Object);
				TedsInterface->CreateRelation(Type, Subject2, Object);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsSubjectOf(Type)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains Subject1", Results.Contains(Subject1));
				TestTrue("Contains Subject2", Results.Contains(Subject2));
				TestFalse("Does not contain Object", Results.Contains(Object));
				TestFalse("Does not contain Unrelated", Results.Contains(Unrelated));
			});

			It("IsObjectOf filters to only rows that are objects", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsObj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();
				RowHandle Unrelated = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object1);
				TedsInterface->CreateRelation(Type, Subject, Object2);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsObjectOf(Type)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains Object1", Results.Contains(Object1));
				TestTrue("Contains Object2", Results.Contains(Object2));
				TestFalse("Does not contain Subject", Results.Contains(Subject));
				TestFalse("Does not contain Unrelated", Results.Contains(Unrelated));
			});

			It("IsSubjectOf with specific target filters precisely", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsSubjTarget"));
				RowHandle S1 = CreateTestRow();
				RowHandle S2 = CreateTestRow();
				RowHandle O1 = CreateTestRow();
				RowHandle O2 = CreateTestRow();

				TedsInterface->CreateRelation(Type, S1, O1);
				TedsInterface->CreateRelation(Type, S2, O2);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsSubjectOf(Type, O1)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains S1 (subject of O1)", Results.Contains(S1));
				TestFalse("Does not contain S2 (subject of O2, not O1)", Results.Contains(S2));
			});

			It("IsDescendantOf filters to descendants only", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsDesc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child1 = CreateTestRow();
				RowHandle Child2 = CreateTestRow();
				RowHandle GrandChild = CreateTestRow();
				RowHandle Unrelated = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child1, Root);
				TedsInterface->CreateRelation(Type, Child2, Root);
				TedsInterface->CreateRelation(Type, GrandChild, Child1);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsDescendantOf(Type, Root)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains Child1", Results.Contains(Child1));
				TestTrue("Contains Child2", Results.Contains(Child2));
				TestTrue("Contains GrandChild", Results.Contains(GrandChild));
				TestFalse("Does not contain Root", Results.Contains(Root));
				TestFalse("Does not contain Unrelated", Results.Contains(Unrelated));
			});

			It("IsDescendantOf with bIncludeSelf includes the reference row", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsDescSelf"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Root);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsDescendantOf(Type, Root, /*bIncludeSelf=*/true)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains Child", Results.Contains(Child));
				TestTrue("Contains Root (bIncludeSelf)", Results.Contains(Root));
			});

			It("IsAncestorOf filters to ancestors only", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_QIsAnc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Mid = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				RowHandle Unrelated = CreateTestRow();

				TedsInterface->CreateRelation(Type, Mid, Root);
				TedsInterface->CreateRelation(Type, Leaf, Mid);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
						.IsAncestorOf(Type, Leaf)
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Contains Mid (parent of Leaf)", Results.Contains(Mid));
				TestTrue("Contains Root (grandparent of Leaf)", Results.Contains(Root));
				TestFalse("Does not contain Leaf itself", Results.Contains(Leaf));
				TestFalse("Does not contain Unrelated", Results.Contains(Unrelated));
			});

			It("Query with no conditions returns all matching rows", [this]()
			{
				// Baseline: verify that a query without relation conditions
				// returns all rows (i.e., no accidental filtering)
				RowHandle R1 = CreateTestRow();
				RowHandle R2 = CreateTestRow();
				RowHandle R3 = CreateTestRow();

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				// All test rows have FTestColumnB since they're in TestTable
				TestTrue("Contains R1", Results.Contains(R1));
				TestTrue("Contains R2", Results.Contains(R2));
				TestTrue("Contains R3", Results.Contains(R3));
			});
		});

		// ===== T1: HasRelationObject / HasRelationSubject =====

		Describe("HasRelation helpers", [this]()
		{
			It("HasRelationObject returns false before creation, true after", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasObj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TestFalse("No object before creation", TedsInterface->HasRelationObject(Type, Subject));
				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Has object after creation", TedsInterface->HasRelationObject(Type, Subject));
			});

			It("HasRelationSubject returns false before creation, true after", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TestFalse("No subject before creation", TedsInterface->HasRelationSubject(Type, Object));
				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Has subject after creation", TedsInterface->HasRelationSubject(Type, Object));
			});
		});

		// ===== T2: Post-order traversal =====

		Describe("Hierarchical Relations post-order", [this]()
		{
			It("TraverseDescendants visits all nodes in post-order", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_PostOrder"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();
				RowHandle GrandChild = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Root);
				TedsInterface->CreateRelation(Type, GrandChild, Child);

				TArray<RowHandle> Visited;
				TedsInterface->TraverseDescendants(Type, Root,
					[&Visited](RowHandle Current, RowHandle Parent, int32 Depth)
					{
						Visited.Add(Current);
					}, ICoreProvider::ETraversalOrder::PostOrder);

				TestEqual("Two descendants visited", Visited.Num(), 2);
				// Post-order: GrandChild before Child
				TestEqual("GrandChild visited first (deepest)", Visited[0], GrandChild);
				TestEqual("Child visited second", Visited[1], Child);
			});
		});

		// ===== T3: GetRelationObject for parentless row =====

		Describe("Hierarchical Relations parentless row", [this]()
		{
			It("GetRelationObject returns InvalidRowHandle for row with no parent", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_NoParent"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Orphan = CreateTestRow();

				RowHandle Result = TedsInterface->GetRelationObject(Type, Orphan);
				TestEqual("No parent returns InvalidRowHandle", Result, InvalidRowHandle);
			});
		});

		// ===== T4: HasRelation after DestroyRelation =====

		Describe("HasRelation after destroy", [this]()
		{
			It("HasRelation returns false after relation is destroyed", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_PostDestroy"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Relation exists after creation", TedsInterface->HasRelation(Type, Subject, Object));

				TedsInterface->DestroyRelation(Type, Subject, Object);
				TestFalse("Relation gone after destroy", TedsInterface->HasRelation(Type, Subject, Object));
			});
		});

		// ===== T5: GetAncestors for root node =====

		Describe("GetAncestors edge cases", [this]()
		{
			It("GetAncestors returns empty for root node", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_AncRoot"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();

				TArray<RowHandle> Ancestors;
				TedsInterface->GetAncestors(Type, Root, Ancestors);
				TestEqual("Root has no ancestors", Ancestors.Num(), 0);
			});
		});

		// ===== T6: TraverseDescendants MaxDepth=0 =====

		Describe("TraverseDescendants MaxDepth edge cases", [this]()
		{
			It("TraverseDescendants with MaxDepth=0 visits no nodes", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_MD0"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Child = CreateTestRow();

				TedsInterface->CreateRelation(Type, Child, Root);

				TArray<RowHandle> Visited;
				TedsInterface->TraverseDescendants(Type, Root,
					[&Visited](RowHandle Current, RowHandle Parent, int32 Depth)
					{
						Visited.Add(Current);
					}, ICoreProvider::ETraversalOrder::PreOrder, /*MaxDepth=*/0);

				TestEqual("No descendants visited at MaxDepth=0", Visited.Num(), 0);
			});
		});

		// ===== T7: Interval encoding containment property =====

		Describe("Interval encoding", [this]()
		{
			It("Parent interval contains child intervals after SetParentRow", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Interval"), true, EHierarchyMode::IntervalEncoded);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();

				// Build A -> B -> C
				RowHandle RelRowAB = TedsInterface->CreateRelation(Type, B, A);
				RowHandle RelRowBC = TedsInterface->CreateRelation(Type, C, B);

				TestTrue("Relation row AB is valid", TedsInterface->IsRowAvailable(RelRowAB));
				TestTrue("Relation row BC is valid", TedsInterface->IsRowAvailable(RelRowBC));

				const FIntervalEncodedHierarchyMetadata* MetaAB = static_cast<const FIntervalEncodedHierarchyMetadata*>(
					TedsInterface->GetColumnData(RelRowAB, FIntervalEncodedHierarchyMetadata::StaticStruct()));
				const FIntervalEncodedHierarchyMetadata* MetaBC = static_cast<const FIntervalEncodedHierarchyMetadata*>(
					TedsInterface->GetColumnData(RelRowBC, FIntervalEncodedHierarchyMetadata::StaticStruct()));

				if (TestNotNull("MetaAB exists", MetaAB) && TestNotNull("MetaBC exists", MetaBC))
				{
					TestTrue("Intervals are assigned (non-zero)", MetaAB->IntervalLeft != 0 || MetaAB->IntervalRight != 0);
					// Containment: parent AB interval contains child BC interval
					TestTrue("BC.Left > AB.Left", MetaBC->IntervalLeft > MetaAB->IntervalLeft);
					TestTrue("BC.Right < AB.Right", MetaBC->IntervalRight < MetaAB->IntervalRight);
				}
			});
		});

		// ===== SetRelation deferred API =====

		Describe("SetRelation", [this]()
		{
			It("Basic set parent from processor: creates the relation after the query", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_SRP_Basic"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle RowA = CreateTestRow();
				RowHandle RowB = CreateTestRow();

				// Initially no relation
				TestFalse("No relation before query", TedsInterface->HasRelation(Type, RowA, RowB));

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
					.Compile());

				TedsInterface->RunQuery(Query, CreateDirectQueryCallbackBinding(
					[&](IDirectQueryContext& Context, const RowHandle* RowHandles)
					{
						for (uint32 i = 0; i < Context.GetRowCount(); ++i)
						{
							if (RowHandles[i] == RowA)
							{
								Context.SetRelation(Type, RowA, RowB);
							}
						}
					}));

				TestTrue("Relation created after query", TedsInterface->HasRelation(Type, RowA, RowB));
			});

			It("Reparent: existing parent is removed and new one is created", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_SRP_Reparent"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle RowA = CreateTestRow();
				RowHandle RowB = CreateTestRow();
				RowHandle RowC = CreateTestRow();

				// A's initial parent is B
				TedsInterface->CreateRelation(Type, RowA, RowB);
				TestTrue("Initial A->B relation", TedsInterface->HasRelation(Type, RowA, RowB));

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
					.Compile());

				TedsInterface->RunQuery(Query, CreateDirectQueryCallbackBinding(
					[&](IDirectQueryContext& Context, const RowHandle* RowHandles)
					{
						for (uint32 i = 0; i < Context.GetRowCount(); ++i)
						{
							if (RowHandles[i] == RowA)
							{
								Context.SetRelation(Type, RowA, RowC);
							}
						}
					}));

				TestTrue("A->C relation created", TedsInterface->HasRelation(Type, RowA, RowC));
				TestFalse("A->B relation removed", TedsInterface->HasRelation(Type, RowA, RowB));
			});

			It("Unparent: passing InvalidRowHandle removes the existing relation without creating a new one", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_SRP_Unparent"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle RowA = CreateTestRow();
				RowHandle RowB = CreateTestRow();

				TedsInterface->CreateRelation(Type, RowA, RowB);
				TestTrue("Initial A->B relation", TedsInterface->HasRelation(Type, RowA, RowB));

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
					.Compile());

				TedsInterface->RunQuery(Query, CreateDirectQueryCallbackBinding(
					[&](IDirectQueryContext& Context, const RowHandle* RowHandles)
					{
						for (uint32 i = 0; i < Context.GetRowCount(); ++i)
						{
							if (RowHandles[i] == RowA)
							{
								Context.SetRelation(Type, RowA, InvalidRowHandle);
							}
						}
					}));

				TestFalse("A->B relation removed", TedsInterface->HasRelation(Type, RowA, RowB));
				TestEqual("A has no parent", TedsInterface->GetRelationObject(Type, RowA), InvalidRowHandle);
			});

			It("Invalid target is a no-op: no crash and no relation created", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_SRP_InvalidTarget"), /*bExclusiveObject=*/true, EHierarchyMode::IntervalEncoded);
				RowHandle SomeRow = CreateTestRow();

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where()
						.All<FTestColumnB>()
					.Compile());

				// Should not crash
				TedsInterface->RunQuery(Query, CreateDirectQueryCallbackBinding(
					[&](IDirectQueryContext& Context, const RowHandle*)
					{
						Context.SetRelation(Type, InvalidRowHandle, SomeRow);
					}));

				// No relation was created with SomeRow as the object
				TArray<RowHandle> Subjects;
				TedsInterface->GetRelationSubjects(Type, SomeRow, Subjects);
				TestEqual("No relation created for invalid target", Subjects.Num(), 0);
			});
		});

		// ===== EHierarchyMode =====

		Describe("HierarchyMode", [this]()
		{
			using namespace UE::Editor::DataStorage;

			// Helper: register a hierarchy type with the given mode
			auto MakeHierarchyType = [this](EHierarchyMode Mode, const FName& Name) -> RelationTypeHandle
			{
				FRelationRegistrationParams Params;
				Params.Name = Name;
				Params.Traits.HierarchyMode = Mode;
				Params.Traits.Object.bExclusive = true;
				RelationTypeHandle Handle = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Handle);
				return Handle;
			};

			// ----- WalkOnly -----

			Describe("WalkOnly", [this, MakeHierarchyType]()
			{
				It("IsDescendantOf true for direct child", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_Direct"));
					RowHandle Parent = CreateTestRow();
					RowHandle Child  = CreateTestRow();

					TedsInterface->CreateRelation(Type, Child, Parent);
					TestTrue("direct child is descendant", TedsInterface->IsDescendantOf(Type, Child, Parent));
				});

				It("IsDescendantOf true at depth 3", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_Depth3"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();
					RowHandle C    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);
					TedsInterface->CreateRelation(Type, C, B);

					TestTrue("depth-3 descendant", TedsInterface->IsDescendantOf(Type, C, Root));
					TestFalse("root is not descendant of itself", TedsInterface->IsDescendantOf(Type, Root, Root));
				});

				It("IsDescendantOf false for sibling", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_Sibling"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, Root);

					TestFalse("sibling is not descendant", TedsInterface->IsDescendantOf(Type, A, B));
				});

				It("Non-leaf reparent is correct immediately (WalkOnly always walks)", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_Reparent"));
					RowHandle Root    = CreateTestRow();
					RowHandle NewRoot = CreateTestRow();
					RowHandle A       = CreateTestRow();
					RowHandle B       = CreateTestRow();
					RowHandle C       = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);
					TedsInterface->CreateRelation(Type, C, B);

					// Reparent A under NewRoot (A already has children B, C).
					// Destroy the old parent relation first — CreateRelation does not auto-remove it.
					TedsInterface->DestroyRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, A, NewRoot);

					// WalkOnly always walks the chain — result is immediately correct
					TestTrue("depth-3 after reparent, same frame", TedsInterface->IsDescendantOf(Type, C, NewRoot));
				});

				It("GetHierarchyDepth returns correct depth", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_HDepth"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);

					TestEqual("depth of direct child", TedsInterface->GetHierarchyDepth(Type, A), 1);
					TestEqual("depth of grandchild",   TedsInterface->GetHierarchyDepth(Type, B), 2);
				});

				It("GetHierarchyRoot returns root of tree", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::WalkOnly, TEXT("TestRel_WO_HRoot"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);

					TestEqual("root of grandchild", TedsInterface->GetHierarchyRoot(Type, B), Root);
				});
			});

			// ----- Versioned -----

			Describe("IntervalEncoded", [this, MakeHierarchyType]()
			{
				It("IsDescendantOf true for direct child", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_Direct"));
					RowHandle Parent = CreateTestRow();
					RowHandle Child  = CreateTestRow();

					TedsInterface->CreateRelation(Type, Child, Parent);
					TestTrue("direct child is descendant", TedsInterface->IsDescendantOf(Type, Child, Parent));
				});

				It("IsDescendantOf true at depth 3", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_Depth3"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();
					RowHandle C    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);
					TedsInterface->CreateRelation(Type, C, B);

					TestTrue("depth-3 descendant", TedsInterface->IsDescendantOf(Type, C, Root));
					TestFalse("root is not descendant of itself", TedsInterface->IsDescendantOf(Type, Root, Root));
				});

				It("IsDescendantOf false for sibling", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_Sibling"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, Root);

					TestFalse("sibling is not descendant", TedsInterface->IsDescendantOf(Type, A, B));
				});

				It("Fresh leaf insertion: IntervalLeft and IntervalVersion are non-zero", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_Fresh"));
					RowHandle Root  = CreateTestRow();
					RowHandle Child = CreateTestRow();

					RowHandle RelRow = TedsInterface->CreateRelation(Type, Child, Root);
					TestNotEqual("Relation row is valid", RelRow, InvalidRowHandle);

					const FIntervalEncodedHierarchyMetadata* Meta = static_cast<const FIntervalEncodedHierarchyMetadata*>(
						TedsInterface->GetColumnData(RelRow, FIntervalEncodedHierarchyMetadata::StaticStruct()));

					if (TestNotNull("Metadata exists", Meta))
					{
						TestTrue("IntervalLeft is non-zero",    Meta->IntervalLeft    != 0);
						TestTrue("IntervalVersion is non-zero", Meta->IntervalVersion != 0);
					}
				});

				It("Non-leaf reparent: correct result same frame (version fallback)", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_Reparent"));
					RowHandle Root    = CreateTestRow();
					RowHandle NewRoot = CreateTestRow();
					RowHandle A       = CreateTestRow();
					RowHandle B       = CreateTestRow();
					RowHandle C       = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);
					TedsInterface->CreateRelation(Type, C, B);

					// Reparent A under NewRoot (non-leaf move — A has children).
					// Destroy the old parent relation first — CreateRelation does not auto-remove it.
					TedsInterface->DestroyRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, A, NewRoot);

					// Versioned mode falls back to parent-chain walk when version stamp is stale
					TestTrue("non-leaf reparent correct same frame",          TedsInterface->IsDescendantOf(Type, C, NewRoot));
					TestFalse("C not descendant of old Root after reparent", TedsInterface->IsDescendantOf(Type, C, Root));
				});

				It("Version isolation: TypeA reparent does not affect TypeB", [this, MakeHierarchyType]()
				{
					RelationTypeHandle TypeA = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_IsoA"));
					RelationTypeHandle TypeB = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_IsoB"));

					RowHandle RootA    = CreateTestRow();
					RowHandle NewRootA = CreateTestRow();
					RowHandle A1       = CreateTestRow();
					RowHandle A2       = CreateTestRow();

					RowHandle ParentB  = CreateTestRow();
					RowHandle ChildB   = CreateTestRow();

					// Build simple hierarchy in TypeA
					TedsInterface->CreateRelation(TypeA, A1, RootA);
					TedsInterface->CreateRelation(TypeA, A2, A1);

					// Build simple hierarchy in TypeB
					TedsInterface->CreateRelation(TypeB, ChildB, ParentB);

					// Non-leaf reparent in TypeA: destroy old relation first (CreateRelation does not auto-remove it)
					TedsInterface->DestroyRelation(TypeA, A1, RootA);
					TedsInterface->CreateRelation(TypeA, A1, NewRootA);

					// TypeB query must be unaffected
					TestTrue("TypeB query still works after TypeA mutation", TedsInterface->IsDescendantOf(TypeB, ChildB, ParentB));
				});

				It("GetHierarchyDepth and GetHierarchyRoot work in Versioned mode", [this, MakeHierarchyType]()
				{
					RelationTypeHandle Type = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_VER_DepthRoot"));
					RowHandle Root = CreateTestRow();
					RowHandle A    = CreateTestRow();
					RowHandle B    = CreateTestRow();

					TedsInterface->CreateRelation(Type, A, Root);
					TedsInterface->CreateRelation(Type, B, A);

					TestEqual("depth of direct child", TedsInterface->GetHierarchyDepth(Type, A), 1);
					TestEqual("depth of grandchild",   TedsInterface->GetHierarchyDepth(Type, B), 2);
					TestEqual("root of grandchild",    TedsInterface->GetHierarchyRoot(Type, B), Root);
				});
			});

			// ----- CrossMode -----

			Describe("CrossMode", [this, MakeHierarchyType]()
			{
				It("Two types with different modes operate independently", [this, MakeHierarchyType]()
				{
					RelationTypeHandle TypeWalk = MakeHierarchyType(EHierarchyMode::WalkOnly,  TEXT("TestRel_CM_Walk"));
					RelationTypeHandle TypeVers = MakeHierarchyType(EHierarchyMode::IntervalEncoded, TEXT("TestRel_CM_Vers"));

					RowHandle Root1 = CreateTestRow();
					RowHandle A1    = CreateTestRow();
					RowHandle Root2 = CreateTestRow();
					RowHandle A2    = CreateTestRow();

					TedsInterface->CreateRelation(TypeWalk, A1, Root1);
					TedsInterface->CreateRelation(TypeVers, A2, Root2);

					TestTrue("WalkOnly type works",                        TedsInterface->IsDescendantOf(TypeWalk, A1, Root1));
					TestTrue("Versioned type works",                       TedsInterface->IsDescendantOf(TypeVers, A2, Root2));
					TestFalse("cross-type: A1 not descendant in Versioned type", TedsInterface->IsDescendantOf(TypeVers, A1, Root2));
				});

				It("Disabled mode: IsDescendantOf returns false", [this, MakeHierarchyType]()
				{
					RelationTypeHandle TypeDisabled = MakeHierarchyType(EHierarchyMode::Disabled, TEXT("TestRel_CM_Disabled"));
					RowHandle Parent = CreateTestRow();
					RowHandle Child  = CreateTestRow();

					TedsInterface->CreateRelation(TypeDisabled, Child, Parent);

					// Disabled means no hierarchy manager data — IsDescendantOf always returns false
					TestFalse("Disabled mode: IsDescendantOf returns false",
						TedsInterface->IsDescendantOf(TypeDisabled, Child, Parent));
				});
			});
		});

		// ===== FQueryExpression: combined column + relation conditions =====

		Describe("FQueryExpression", [this]()
		{
			It("Column && relation predicate: only rows with column AND relation pass", [this]()
			{
				// A (root, no FTestColumnB), B (child of A, has FTestColumnB),
				// C (grandchild, has FTestColumnB), D (unrelated, has FTestColumnB)
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Expr_Basic"), true, EHierarchyMode::WalkOnly);
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();
				RowHandle D = CreateTestRow();

				// Remove FTestColumnB from A so it fails the column condition
				TedsInterface->RemoveColumns<FTestColumnB>(A);

				TedsInterface->CreateRelation(Type, B, A);
				TedsInterface->CreateRelation(Type, C, B);
				// D has no relation to A at all

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where(TColumn<FTestColumnB>() && IsDescendantOf(Type, A))
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("B returned (has column + is descendant)", Results.Contains(B));
				TestTrue("C returned (has column + transitive descendant)", Results.Contains(C));
				TestFalse("A excluded (no column)", Results.Contains(A));
				TestFalse("D excluded (has column, not a descendant of A)", Results.Contains(D));
			});

			It("Column && IsSubjectOf: only subjects with the column pass", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_Expr_Subj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object   = CreateTestRow();
				RowHandle Unrelated = CreateTestRow();

				// Remove column from Object so it fails column condition
				TedsInterface->RemoveColumns<FTestColumnB>(Object);

				TedsInterface->CreateRelation(Type, Subject, Object);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where(TColumn<FTestColumnB>() && IsSubjectOf(Type))
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestTrue("Subject returned (has column + is subject)", Results.Contains(Subject));
				TestFalse("Object excluded (no column)", Results.Contains(Object));
				TestFalse("Unrelated excluded (has column, not a subject)", Results.Contains(Unrelated));
			});



			It("Two relation predicates &&-ed: both conditions must hold", [this]()
			{
				RelationTypeHandle TypeA = RegisterSimpleRelation(TEXT("TestRel_Expr_AndA"));
				RelationTypeHandle TypeB = RegisterSimpleRelation(TEXT("TestRel_Expr_AndB"));
				RowHandle ObjA    = CreateTestRow();
				RowHandle ObjB    = CreateTestRow();
				RowHandle SubjA   = CreateTestRow();   // subject of A only
				RowHandle SubjB   = CreateTestRow();   // subject of B only
				RowHandle SubjAB  = CreateTestRow();   // subject of both
				RowHandle Neither = CreateTestRow();

				TedsInterface->CreateRelation(TypeA, SubjA,  ObjA);
				TedsInterface->CreateRelation(TypeB, SubjB,  ObjB);
				TedsInterface->CreateRelation(TypeA, SubjAB, ObjA);
				TedsInterface->CreateRelation(TypeB, SubjAB, ObjB);

				using namespace Queries;
				QueryHandle Query = RegisterTestQuery(
					Select()
					.Where(TColumn<FTestColumnB>() && IsSubjectOf(TypeA) && IsSubjectOf(TypeB))
					.Compile());

				TArray<RowHandle> Results = RunQueryCollectRows(Query);
				TestFalse("SubjA excluded (only subject of A)", Results.Contains(SubjA));
				TestFalse("SubjB excluded (only subject of B)", Results.Contains(SubjB));
				TestTrue("SubjAB returned (subject of both)", Results.Contains(SubjAB));
				TestFalse("Neither excluded", Results.Contains(Neither));
			});
		});

		// ===================================================================
		// Relation Changed Columns (new feature)
		// ===================================================================

		Describe("Relation Changed Columns", [this]()
		{
			It("GetRelationSubjectChangedColumn returns nullptr when not enabled", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_NoSubjChanged"));
				TestNull("No column when disabled", TedsInterface->GetRelationSubjectChangedColumn(Type));
			});

			It("GetRelationObjectChangedColumn returns nullptr when not enabled", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_NoObjChanged"));
				TestNull("No column when disabled", TedsInterface->GetRelationObjectChangedColumn(Type));
			});

			It("GetRelationSubjectChangedColumn returns non-null when enabled", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_SubjChangedEnabled");
				Params.bEnableSubjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationSubjectChangedColumn(Type);
				TestNotNull("Column exists", Col);
			});

			It("GetRelationObjectChangedColumn returns non-null when enabled", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_ObjChangedEnabled");
				Params.bEnableObjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationObjectChangedColumn(Type);
				TestNotNull("Column exists", Col);
			});

			It("GetRelationSubjectChangedColumn returns nullptr for InvalidRelationTypeHandle", [this]()
			{
				TestNull("Null for invalid handle",
					TedsInterface->GetRelationSubjectChangedColumn(InvalidRelationTypeHandle));
			});

			It("GetRelationObjectChangedColumn returns nullptr for InvalidRelationTypeHandle", [this]()
			{
				TestNull("Null for invalid handle",
					TedsInterface->GetRelationObjectChangedColumn(InvalidRelationTypeHandle));
			});

			It("SubjectChangedColumn is stamped on Subject after CreateRelation", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_StampSubjCreate");
				Params.bEnableSubjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationSubjectChangedColumn(Type);
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TestFalse("Column absent before", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({Col})));
				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Column stamped on Subject", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({Col})));
				TestFalse("Column NOT stamped on Object", TedsInterface->HasColumns(Object, TConstArrayView<const UScriptStruct*>({Col})));
			});

			It("ObjectChangedColumn is stamped on Object after CreateRelation", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_StampObjCreate");
				Params.bEnableObjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationObjectChangedColumn(Type);
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Column stamped on Object", TedsInterface->HasColumns(Object, TConstArrayView<const UScriptStruct*>({Col})));
				TestFalse("Column NOT stamped on Subject", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({Col})));
			});

			It("SubjectChangedColumn is stamped on Subject after DestroyRelation", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_StampSubjDestroy");
				Params.bEnableSubjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationSubjectChangedColumn(Type);
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				// Remove the column manually to simulate frame rollover
				TedsInterface->RemoveColumns(Subject, {Col});

				TestFalse("Column absent before destroy", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({Col})));
				TedsInterface->DestroyRelation(Type, Subject, Object);
				TestTrue("Column restamped after destroy", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({Col})));
			});

			It("ObjectChangedColumn is stamped on Object after DestroyRelation", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_StampObjDestroy");
				Params.bEnableObjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* Col = TedsInterface->GetRelationObjectChangedColumn(Type);
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->RemoveColumns(Object, {Col});

				TedsInterface->DestroyRelation(Type, Subject, Object);
				TestTrue("Column restamped on Object after destroy", TedsInterface->HasColumns(Object, TConstArrayView<const UScriptStruct*>({Col})));
			});

			It("No column stamped when bEnableSubjectChangedColumn is false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_NoStampSubj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);

				// No changed column registered so nothing to check -- verify no crashes
				TestNull("No subject changed column", TedsInterface->GetRelationSubjectChangedColumn(Type));
			});

			It("Both Subject and Object columns can be enabled simultaneously", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_BothChanged");
				Params.bEnableSubjectChangedColumn = true;
				Params.bEnableObjectChangedColumn = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				const UScriptStruct* SubjCol = TedsInterface->GetRelationSubjectChangedColumn(Type);
				const UScriptStruct* ObjCol = TedsInterface->GetRelationObjectChangedColumn(Type);
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Subject column stamped", TedsInterface->HasColumns(Subject, TConstArrayView<const UScriptStruct*>({SubjCol})));
				TestTrue("Object column stamped", TedsInterface->HasColumns(Object, TConstArrayView<const UScriptStruct*>({ObjCol})));
				TestNotEqual("Subject and Object columns are distinct", SubjCol, ObjCol);
			});

			It("Hierarchy backend GetParentChangedColumnType returns non-null when bEnableParentChangedColumn=true", [this]()
			{
				FHierarchyRegistrationParams HierarchyParams;
				HierarchyParams.Name = TEXT("TestHier_ParentChangedRelations");
				HierarchyParams.bEnableParentChangedColumn = true;
				HierarchyParams.Backend = FHierarchyRegistrationParams::EBackend::Relations;
				FHierarchyHandle Handle = TedsInterface->RegisterHierarchy(HierarchyParams);

				const UScriptStruct* ParentChangedCol = TedsInterface->GetParentChangedColumnType(Handle);
				TestNotNull("ParentChangedColumn exists for Relations backend", ParentChangedCol);
			});

			It("Hierarchy backend ParentChangedColumn == SubjectChangedColumn on underlying relation type", [this]()
			{
				FHierarchyRegistrationParams HierarchyParams;
				HierarchyParams.Name = TEXT("TestHier_ParentChangedStamp");
				HierarchyParams.bEnableParentChangedColumn = true;
				HierarchyParams.Backend = FHierarchyRegistrationParams::EBackend::Relations;
				FHierarchyHandle Handle = TedsInterface->RegisterHierarchy(HierarchyParams);

				const UScriptStruct* ParentChangedCol = TedsInterface->GetParentChangedColumnType(Handle);
				if (!TestNotNull("ParentChangedColumn must exist", ParentChangedCol))
				{
					return;
				}

				// The underlying relation type shares the hierarchy's name.
				// The ParentChangedColumn for the Relations backend must equal
				// the SubjectChangedColumn on that relation type.
				const RelationTypeHandle HierarchyRelationType = TedsInterface->FindRelationType(HierarchyParams.Name);
				if (!TestNotEqual("Hierarchy relation type found",
					HierarchyRelationType, InvalidRelationTypeHandle))
				{
					return;
				}

				const UScriptStruct* SubjectChangedCol =
					TedsInterface->GetRelationSubjectChangedColumn(HierarchyRelationType);

				TestEqual("ParentChangedColumn == SubjectChangedColumn", ParentChangedCol, SubjectChangedCol);

				// CreateRelation(HierarchyRelationType, Child, Parent) stamps SubjectChangedColumn on Child.
				RowHandle Parent = CreateTestRow();
				RowHandle Child = CreateTestRow();
				TedsInterface->CreateRelation(HierarchyRelationType, Child, Parent);

				TestTrue("ParentChanged column stamped on child after relation creation",
					TedsInterface->HasColumns(Child, TConstArrayView<const UScriptStruct*>({ParentChangedCol})));
			});
		});

		Describe("GetRelationSubject (singular)", [this]()
		{
			It("returns the single subject for an exclusive-subject relation", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_GetSubjSingular");
				Params.Traits.Subject.bExclusive = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();

				TedsInterface->CreateRelation(Type, Subject, Object);
				RowHandle Result = TedsInterface->GetRelationSubject(Type, Object);
				TestEqual("Subject matches", Result, Subject);
			});

			It("returns InvalidRowHandle when object has no subject", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_GetSubjSingularNone");
				Params.Traits.Subject.bExclusive = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Orphan = CreateTestRow();
				RowHandle Result = TedsInterface->GetRelationSubject(Type, Orphan);
				TestEqual("No subject returns InvalidRowHandle", Result, InvalidRowHandle);
			});
		});

		Describe("HasRelationSubjects (plural)", [this]()
		{
			It("returns false when object has no subjects", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubjsNone"));
				RowHandle Row = CreateTestRow();
				TestFalse("No subjects", TedsInterface->HasRelationSubjects(Type, Row));
			});

			It("returns true when object has at least one subject", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubjsYes"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();
				TedsInterface->CreateRelation(Type, Subject, Object);
				TestTrue("Has subjects", TedsInterface->HasRelationSubjects(Type, Object));
			});

			It("returns false after all subjects are destroyed", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubjsAfterDestroy"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object = CreateTestRow();
				TedsInterface->CreateRelation(Type, Subject, Object);
				TedsInterface->DestroyRelation(Type, Subject, Object);
				TestFalse("No subjects after destroy", TedsInterface->HasRelationSubjects(Type, Object));
			});
		});

		Describe("ForEachRelationObject", [this]()
		{
			It("visits each object for a subject", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_ForEachObj"));
				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();
				TedsInterface->CreateRelation(Type, Subject, Object1);
				TedsInterface->CreateRelation(Type, Subject, Object2);

				TArray<RowHandle> Visited;
				TedsInterface->ForEachRelationObject(Type, Subject,
					[&Visited](RowHandle Object) { Visited.Add(Object); });

				TestEqual("Two objects visited", Visited.Num(), 2);
				TestTrue("Contains Object1", Visited.Contains(Object1));
				TestTrue("Contains Object2", Visited.Contains(Object2));
			});

			It("visits nothing when subject has no relations", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_ForEachObjEmpty"));
				RowHandle Subject = CreateTestRow();
				int32 Count = 0;
				TedsInterface->ForEachRelationObject(Type, Subject,
					[&Count](RowHandle) { Count++; });
				TestEqual("No objects visited", Count, 0);
			});
		});

		Describe("ForEachRelationSubject", [this]()
		{
			It("visits each subject for an object", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_ForEachSubj"));
				RowHandle Subject1 = CreateTestRow();
				RowHandle Subject2 = CreateTestRow();
				RowHandle Object = CreateTestRow();
				TedsInterface->CreateRelation(Type, Subject1, Object);
				TedsInterface->CreateRelation(Type, Subject2, Object);

				TArray<RowHandle> Visited;
				TedsInterface->ForEachRelationSubject(Type, Object,
					[&Visited](RowHandle Subject) { Visited.Add(Subject); });

				TestEqual("Two subjects visited", Visited.Num(), 2);
				TestTrue("Contains Subject1", Visited.Contains(Subject1));
				TestTrue("Contains Subject2", Visited.Contains(Subject2));
			});

			It("visits nothing when object has no relations", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_ForEachSubjEmpty"));
				RowHandle Object = CreateTestRow();
				int32 Count = 0;
				TedsInterface->ForEachRelationSubject(Type, Object,
					[&Count](RowHandle) { Count++; });
				TestEqual("No subjects visited", Count, 0);
			});
		});

		Describe("TraverseAncestors", [this]()
		{
			It("visits ancestors from parent to root", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravAnc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Mid = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				TedsInterface->CreateRelation(Type, Mid, Root);
				TedsInterface->CreateRelation(Type, Leaf, Mid);

				TArray<RowHandle> Visited;
				TArray<int32> Depths;
				TedsInterface->TraverseAncestors(Type, Leaf,
					[&Visited, &Depths](RowHandle Ancestor, int32 Depth) -> bool
					{
						Visited.Add(Ancestor);
						Depths.Add(Depth);
						return true;
					});

				TestEqual("Two ancestors visited", Visited.Num(), 2);
				TestEqual("First ancestor is Mid", Visited[0], Mid);
				TestEqual("Second ancestor is Root", Visited[1], Root);
			});

			It("visits nothing for a root node", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravAncRoot"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				int32 Count = 0;
				TedsInterface->TraverseAncestors(Type, Root,
					[&Count](RowHandle, int32) -> bool { Count++; return true; });
				TestEqual("No ancestors for root", Count, 0);
			});

			It("supports early termination", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravAncEarly"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Mid = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				TedsInterface->CreateRelation(Type, Mid, Root);
				TedsInterface->CreateRelation(Type, Leaf, Mid);

				TArray<RowHandle> Visited;
				TedsInterface->TraverseAncestors(Type, Leaf,
					[&Visited](RowHandle Ancestor, int32) -> bool
					{
						Visited.Add(Ancestor);
						return false; // stop after first
					});

				TestEqual("Only one ancestor visited (early termination)", Visited.Num(), 1);
				TestEqual("First ancestor is Mid", Visited[0], Mid);
			});
		});

		Describe("ComputeDescendantCount", [this]()
		{
			It("returns correct count for a tree", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DescCount"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				RowHandle C = CreateTestRow();
				TedsInterface->CreateRelation(Type, A, Root);
				TedsInterface->CreateRelation(Type, B, Root);
				TedsInterface->CreateRelation(Type, C, A);

				TestEqual("Root has 3 descendants", TedsInterface->ComputeDescendantCount(Type, Root), 3);
				TestEqual("A has 1 descendant", TedsInterface->ComputeDescendantCount(Type, A), 1);
				TestEqual("B has 0 descendants", TedsInterface->ComputeDescendantCount(Type, B), 0);
			});

			It("returns 0 for a leaf node", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DescCountLeaf"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				TedsInterface->CreateRelation(Type, Leaf, Root);
				TestEqual("Leaf has 0 descendants", TedsInterface->ComputeDescendantCount(Type, Leaf), 0);
			});

			It("returns 0 for an isolated node", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DescCountIsolated"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Isolated = CreateTestRow();
				TestEqual("Isolated node has 0 descendants", TedsInterface->ComputeDescendantCount(Type, Isolated), 0);
			});
		});

		// ===================================================================
		// Invalid RelationTypeHandle robustness
		// ===================================================================

		Describe("Invalid RelationTypeHandle robustness", [this]()
		{
			It("IsValidRelationType returns false for InvalidRelationTypeHandle", [this]()
			{
				TestFalse("Invalid handle is not valid", TedsInterface->IsValidRelationType(InvalidRelationTypeHandle));
			});

			It("GetRelationTypeTraits returns nullptr for InvalidRelationTypeHandle", [this]()
			{
				TestNull("Null traits for invalid handle",
					TedsInterface->GetRelationTypeTraits(InvalidRelationTypeHandle));
			});

			It("HasRelation does not crash with InvalidRelationTypeHandle", [this]()
			{
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				TestFalse("HasRelation returns false", TedsInterface->HasRelation(InvalidRelationTypeHandle, A, B));
			});

			It("CreateRelation returns InvalidRowHandle for InvalidRelationTypeHandle", [this]()
			{
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->CreateRelation(InvalidRelationTypeHandle, A, B), InvalidRowHandle);
			});

			It("DestroyRelation returns false for InvalidRelationTypeHandle", [this]()
			{
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				TestFalse("Returns false", TedsInterface->DestroyRelation(InvalidRelationTypeHandle, A, B));
			});

			It("GetRelationObjects returns empty for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(InvalidRelationTypeHandle, Row, Objects);
				TestEqual("Empty objects", Objects.Num(), 0);
			});

			It("GetRelationSubjects returns empty for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TArray<RowHandle> Subjects;
				TedsInterface->GetRelationSubjects(InvalidRelationTypeHandle, Row, Subjects);
				TestEqual("Empty subjects", Subjects.Num(), 0);
			});

			It("GetRelationObject returns InvalidRowHandle for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->GetRelationObject(InvalidRelationTypeHandle, Row), InvalidRowHandle);
			});

			It("GetRelationSubject returns InvalidRowHandle for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->GetRelationSubject(InvalidRelationTypeHandle, Row), InvalidRowHandle);
			});

			It("HasRelationObject returns false for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestFalse("Returns false", TedsInterface->HasRelationObject(InvalidRelationTypeHandle, Row));
			});

			It("HasRelationSubject returns false for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestFalse("Returns false", TedsInterface->HasRelationSubject(InvalidRelationTypeHandle, Row));
			});

			It("HasRelationSubjects returns false for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestFalse("Returns false", TedsInterface->HasRelationSubjects(InvalidRelationTypeHandle, Row));
			});

			It("ForEachRelationObject visits nothing for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				int32 Count = 0;
				TedsInterface->ForEachRelationObject(InvalidRelationTypeHandle, Row,
					[&Count](RowHandle) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("ForEachRelationSubject visits nothing for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				int32 Count = 0;
				TedsInterface->ForEachRelationSubject(InvalidRelationTypeHandle, Row,
					[&Count](RowHandle) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("IsDescendantOf returns false for InvalidRelationTypeHandle", [this]()
			{
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				TestFalse("Returns false", TedsInterface->IsDescendantOf(InvalidRelationTypeHandle, A, B));
			});

			It("GetDescendants returns empty for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TArray<RowHandle> Descendants;
				TedsInterface->GetDescendants(InvalidRelationTypeHandle, Row, Descendants);
				TestEqual("Empty descendants", Descendants.Num(), 0);
			});

			It("GetAncestors returns empty for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TArray<RowHandle> Ancestors;
				TedsInterface->GetAncestors(InvalidRelationTypeHandle, Row, Ancestors);
				TestEqual("Empty ancestors", Ancestors.Num(), 0);
			});

			It("TraverseDescendants visits nothing for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				int32 Count = 0;
				TedsInterface->TraverseDescendants(InvalidRelationTypeHandle, Row,
					[&Count](RowHandle, RowHandle, int32) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("TraverseAncestors visits nothing for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				int32 Count = 0;
				TedsInterface->TraverseAncestors(InvalidRelationTypeHandle, Row,
					[&Count](RowHandle, int32) -> bool { Count++; return true; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("ComputeDescendantCount returns 0 for InvalidRelationTypeHandle", [this]()
			{
				RowHandle Row = CreateTestRow();
				TestEqual("0 descendants", TedsInterface->ComputeDescendantCount(InvalidRelationTypeHandle, Row), 0);
			});

			It("BatchCreateRelations is a no-op for InvalidRelationTypeHandle", [this]()
			{
				RowHandle A = CreateTestRow();
				RowHandle B = CreateTestRow();
				TArray<RowHandle> Subjects = {A};
				TArray<RowHandle> Objects = {B};
				TArray<RowHandle> OutRows;
				TedsInterface->BatchCreateRelations(InvalidRelationTypeHandle, Subjects, Objects, &OutRows);
				TestEqual("No rows created", OutRows.Num(), 0);
			});
		});

		// ===================================================================
		// InvalidRowHandle robustness
		// ===================================================================

		Describe("InvalidRowHandle robustness", [this]()
		{
			It("CreateRelation with InvalidRowHandle Subject returns InvalidRowHandle", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_InvSubj"));
				RowHandle Object = CreateTestRow();
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->CreateRelation(Type, InvalidRowHandle, Object), InvalidRowHandle);
			});

			It("CreateRelation with InvalidRowHandle Object returns InvalidRowHandle", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_InvObj"));
				RowHandle Subject = CreateTestRow();
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->CreateRelation(Type, Subject, InvalidRowHandle), InvalidRowHandle);
			});

			It("HasRelation with InvalidRowHandle Subject returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasInvSubj"));
				RowHandle Object = CreateTestRow();
				TestFalse("Returns false", TedsInterface->HasRelation(Type, InvalidRowHandle, Object));
			});

			It("HasRelation with InvalidRowHandle Object returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasInvObj"));
				RowHandle Subject = CreateTestRow();
				TestFalse("Returns false", TedsInterface->HasRelation(Type, Subject, InvalidRowHandle));
			});

			It("DestroyRelation with InvalidRowHandle returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DestroyInv"));
				TestFalse("Returns false",
					TedsInterface->DestroyRelation(Type, InvalidRowHandle, InvalidRowHandle));
			});

			It("GetRelationObjects with InvalidRowHandle Subject returns empty", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetObjInv"));
				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, InvalidRowHandle, Objects);
				TestEqual("No objects", Objects.Num(), 0);
			});

			It("GetRelationSubjects with InvalidRowHandle Object returns empty", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetSubjInv"));
				TArray<RowHandle> Subjects;
				TedsInterface->GetRelationSubjects(Type, InvalidRowHandle, Subjects);
				TestEqual("No subjects", Subjects.Num(), 0);
			});

			It("GetRelationObject with InvalidRowHandle returns InvalidRowHandle", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetObjSInv"), true, EHierarchyMode::IntervalEncoded);
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->GetRelationObject(Type, InvalidRowHandle), InvalidRowHandle);
			});

			It("GetRelationSubject with InvalidRowHandle returns InvalidRowHandle", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_GetSubjSInv");
				Params.Traits.Subject.bExclusive = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);
				TestEqual("Returns InvalidRowHandle",
					TedsInterface->GetRelationSubject(Type, InvalidRowHandle), InvalidRowHandle);
			});

			It("HasRelationObject with InvalidRowHandle returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasObjInv"));
				TestFalse("Returns false", TedsInterface->HasRelationObject(Type, InvalidRowHandle));
			});

			It("HasRelationSubject with InvalidRowHandle returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubjInv"));
				TestFalse("Returns false", TedsInterface->HasRelationSubject(Type, InvalidRowHandle));
			});

			It("HasRelationSubjects with InvalidRowHandle returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_HasSubjsInv"));
				TestFalse("Returns false", TedsInterface->HasRelationSubjects(Type, InvalidRowHandle));
			});

			It("ForEachRelationObject with InvalidRowHandle visits nothing", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_FEObjInv"));
				int32 Count = 0;
				TedsInterface->ForEachRelationObject(Type, InvalidRowHandle,
					[&Count](RowHandle) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("ForEachRelationSubject with InvalidRowHandle visits nothing", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_FESubjInv"));
				int32 Count = 0;
				TedsInterface->ForEachRelationSubject(Type, InvalidRowHandle,
					[&Count](RowHandle) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("IsDescendantOf with InvalidRowHandle Descendant returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDescInvDesc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Valid = CreateTestRow();
				TestFalse("Returns false", TedsInterface->IsDescendantOf(Type, InvalidRowHandle, Valid));
			});

			It("IsDescendantOf with InvalidRowHandle Ancestor returns false", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_IsDescInvAnc"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Valid = CreateTestRow();
				TestFalse("Returns false", TedsInterface->IsDescendantOf(Type, Valid, InvalidRowHandle));
			});

			It("GetDescendants with InvalidRowHandle returns empty", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DescInv"), true, EHierarchyMode::IntervalEncoded);
				TArray<RowHandle> Descendants;
				TedsInterface->GetDescendants(Type, InvalidRowHandle, Descendants);
				TestEqual("Empty", Descendants.Num(), 0);
			});

			It("GetAncestors with InvalidRowHandle returns empty", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_AncInv"), true, EHierarchyMode::IntervalEncoded);
				TArray<RowHandle> Ancestors;
				TedsInterface->GetAncestors(Type, InvalidRowHandle, Ancestors);
				TestEqual("Empty", Ancestors.Num(), 0);
			});

			It("TraverseDescendants with InvalidRowHandle visits nothing", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravDescInv"), true, EHierarchyMode::IntervalEncoded);
				int32 Count = 0;
				TedsInterface->TraverseDescendants(Type, InvalidRowHandle,
					[&Count](RowHandle, RowHandle, int32) { Count++; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("TraverseAncestors with InvalidRowHandle visits nothing", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravAncInv"), true, EHierarchyMode::IntervalEncoded);
				int32 Count = 0;
				TedsInterface->TraverseAncestors(Type, InvalidRowHandle,
					[&Count](RowHandle, int32) -> bool { Count++; return true; });
				TestEqual("Nothing visited", Count, 0);
			});

			It("ComputeDescendantCount with InvalidRowHandle returns 0", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_CountInv"), true, EHierarchyMode::IntervalEncoded);
				TestEqual("Returns 0", TedsInterface->ComputeDescendantCount(Type, InvalidRowHandle), 0);
			});
		});

		// ===================================================================
		// Boundary and edge cases
		// ===================================================================

		Describe("BatchCreateRelations edge cases", [this]()
		{
			It("BatchCreateRelations with empty arrays is a no-op", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_BatchEmpty"));
				TArray<RowHandle> Subjects;
				TArray<RowHandle> Objects;
				TArray<RowHandle> OutRows;
				TedsInterface->BatchCreateRelations(Type, Subjects, Objects, &OutRows);
				TestEqual("No rows created", OutRows.Num(), 0);
			});

			It("BatchCreateRelations creates one relation per Subject-Object pair", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_BatchPairs"));
				RowHandle S1 = CreateTestRow();
				RowHandle S2 = CreateTestRow();
				RowHandle O1 = CreateTestRow();
				RowHandle O2 = CreateTestRow();

				TArray<RowHandle> Subjects = {S1, S2};
				TArray<RowHandle> Objects  = {O1, O2};
				TArray<RowHandle> OutRows;
				TedsInterface->BatchCreateRelations(Type, Subjects, Objects, &OutRows);

				TestEqual("Two relation rows created", OutRows.Num(), 2);
				TestTrue("S1->O1 exists", TedsInterface->HasRelation(Type, S1, O1));
				TestTrue("S2->O2 exists", TedsInterface->HasRelation(Type, S2, O2));
			});

			It("BatchCreateRelations with nullptr OutRelationRows does not crash", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_BatchNullOut"));
				RowHandle S = CreateTestRow();
				RowHandle O = CreateTestRow();
				TArray<RowHandle> Subjects = {S};
				TArray<RowHandle> Objects = {O};
				TedsInterface->BatchCreateRelations(Type, Subjects, Objects, nullptr);
				TestTrue("Relation created", TedsInterface->HasRelation(Type, S, O));
			});
		});

		Describe("Exclusivity -- bExclusiveSubject", [this]()
		{
			It("Exclusive Subject rejects second Object for same Subject", [this]()
			{
				FRelationRegistrationParams Params;
				Params.Name = TEXT("TestRel_ExclSubj");
				Params.Traits.Subject.bExclusive = true;
				RelationTypeHandle Type = TedsInterface->RegisterRelationType(Params);
				RegisteredRelationTypes.Add(Type);

				RowHandle Subject = CreateTestRow();
				RowHandle Object1 = CreateTestRow();
				RowHandle Object2 = CreateTestRow();

				RowHandle Rel1 = TedsInterface->CreateRelation(Type, Subject, Object1);
				TestNotEqual("First relation created", Rel1, InvalidRowHandle);

				// Second relation with same Subject should be rejected
				RowHandle Rel2 = TedsInterface->CreateRelation(Type, Subject, Object2);
				TArray<RowHandle> Objects;
				TedsInterface->GetRelationObjects(Type, Subject, Objects);
				TestEqual("Subject has at most one object", Objects.Num(), 1);
			});
		});

		Describe("GetRelationSubjects empty case", [this]()
		{
			It("GetRelationSubjects returns empty for row with no subjects", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_GetSubjEmpty"));
				RowHandle Row = CreateTestRow();
				TArray<RowHandle> Subjects;
				TedsInterface->GetRelationSubjects(Type, Row, Subjects);
				TestEqual("No subjects", Subjects.Num(), 0);
			});
		});

		Describe("Hierarchy leaf and isolated node edge cases", [this]()
		{
			It("GetDescendants returns empty for a leaf node", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DescLeaf"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				TedsInterface->CreateRelation(Type, Leaf, Root);
				TArray<RowHandle> Descendants;
				TedsInterface->GetDescendants(Type, Leaf, Descendants);
				TestEqual("Leaf has no descendants", Descendants.Num(), 0);
			});

			It("GetHierarchyRoot on an isolated node returns itself", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_RootIsolated"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Isolated = CreateTestRow();
				TestEqual("Isolated node is its own root",
					TedsInterface->GetHierarchyRoot(Type, Isolated), Isolated);
			});

			It("GetHierarchyDepth on an isolated node returns 0", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_DepthIsolated"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Isolated = CreateTestRow();
				TestEqual("Isolated node depth is 0",
					TedsInterface->GetHierarchyDepth(Type, Isolated), 0);
			});

			It("TraverseDescendants on a leaf visits nothing", [this]()
			{
				RelationTypeHandle Type = RegisterSimpleRelation(TEXT("TestRel_TravLeaf"), true, EHierarchyMode::IntervalEncoded);
				RowHandle Root = CreateTestRow();
				RowHandle Leaf = CreateTestRow();
				TedsInterface->CreateRelation(Type, Leaf, Root);
				int32 Count = 0;
				TedsInterface->TraverseDescendants(Type, Leaf,
					[&Count](RowHandle, RowHandle, int32) { Count++; });
				TestEqual("Leaf has no descendants to traverse", Count, 0);
			});
		});

		AfterEach([this]()
		{
			for (RowHandle Row : Rows)
			{
				TedsInterface->RemoveRow(Row);
			}
			Rows.Empty(Rows.Num());

			for (QueryHandle QH : QueryHandles)
			{
				TedsInterface->UnregisterQuery(QH);
			}
			QueryHandles.Empty(QueryHandles.Num());

			RegisteredRelationTypes.Empty();
			TedsInterface = nullptr;
		});
	}
}

#endif // WITH_TESTS
