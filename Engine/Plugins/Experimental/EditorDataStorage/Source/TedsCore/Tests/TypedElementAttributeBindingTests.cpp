// Copyright Epic Games, Inc. All Rights Reserved.


#include "Elements/Framework/TypedElementAttributeBinding.h"
#if WITH_TESTS
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/AutomationTest.h"

namespace UE::Editor::DataStorage::Tests
{
	BEGIN_DEFINE_SPEC(TedsAttributeBindingTestsFixture, "Editor.DataStorage.AttributeBinding", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	ICoreProvider* TedsInterface = nullptr;
	const FName TestTableName = TEXT("TestTable_AttributeBinding");
	TableHandle TestTableHandle = InvalidTableHandle;
	RowHandle TestRowHandle = InvalidRowHandle;

	TableHandle RegisterTestTable() const
	{
		const TableHandle Table = TedsInterface->FindTable(TestTableName);
		
		if (Table != InvalidTableHandle)
		{
			return Table;
		}
		
		return TedsInterface->RegisterTable(
		{
			FTestColumnInt::StaticStruct(),
			FTestColumnString::StaticStruct()
		},
		TestTableName);
	}
	
	RowHandle CreateTestRow(TableHandle InTableHandle) const
	{
		const RowHandle RowHandle = TedsInterface->AddRow(InTableHandle);
		return RowHandle;
	}
	
	void CleanupTestRow(RowHandle InRowHandle) const
	{
		TedsInterface->RemoveRow(InRowHandle);
	}
	
	END_DEFINE_SPEC(TedsAttributeBindingTestsFixture)

	void TedsAttributeBindingTestsFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTrue("", TedsInterface != nullptr);
			
			TestTableHandle = RegisterTestTable();
			TestNotEqual("Expecting valid table handle", TestTableHandle, InvalidTableHandle);

			TestRowHandle = CreateTestRow(TestTableHandle);
			TestFalse("Expect valid row handle", TestRowHandle == InvalidRowHandle);
		});

		Describe("", [this]()
		{
			Describe("Integer Attribute", [this]()
			{
				Describe("Direct integer attribute", [this]()
				{
					It("Direct attribute should update on updating column value", [this]()
					{
						constexpr int InitialValue = 10;
						constexpr int UpdatedValue = 20;
						
						// Add the test int column to the test row
						TedsInterface->AddColumn(TestRowHandle, FTestColumnInt{.TestInt =  InitialValue});
						FTestColumnInt* TestColumnInt = TedsInterface->GetColumn<FTestColumnInt>(TestRowHandle);

						TestNotNull("Expecting Valid Column", TestColumnInt);

						// Create an int attribute and bind it
						FAttributeBinder Binder(TestRowHandle);
						const TAttribute TestAttribute(Binder.BindData(&FTestColumnInt::TestInt));

						TestEqual("Expecting attribute value to match column value before modification", TestAttribute.Get(), TestColumnInt->TestInt);

						// Modify the value in the column
						TestColumnInt->TestInt = UpdatedValue;

						TestEqual("Expecting attribute value to update after modification", TestAttribute.Get(), UpdatedValue);
						TestEqual("Expecting attribute value to match column value after modification", TestAttribute.Get(), TestColumnInt->TestInt);
					});
				});

				Describe("Float attribute bound to integer column data", [this]()
				{
					It("Converted attribute should update on updating column value", [this]()
					{
						constexpr int InitialValue = 10;
						constexpr int UpdatedValue = 20;

						// Add the test int column to the test row
						TedsInterface->AddColumn(TestRowHandle, FTestColumnInt{.TestInt =  InitialValue});
						FTestColumnInt* TestColumnInt = TedsInterface->GetColumn<FTestColumnInt>(TestRowHandle);

						TestNotNull("Expecting valid column", TestColumnInt);

						// Create a float attribute and bind it by providing a conversion function
						FAttributeBinder Binder(TestRowHandle);
						const TAttribute<float> TestAttribute(Binder.BindData(&FTestColumnInt::TestInt,
							[](const int& Data)
							{
								return static_cast<float>(Data);
							}));

						TestEqual("Expecting attribute value to match column value before modification", static_cast<int>(TestAttribute.Get()), TestColumnInt->TestInt);

						// Modify the value in the column
						TestColumnInt->TestInt = UpdatedValue;

						TestEqual("Expecting attribute value to update after modification", static_cast<int>(TestAttribute.Get()), UpdatedValue);
						TestEqual("Expecting attribute value to match column value after modification", static_cast<int>(TestAttribute.Get()), TestColumnInt->TestInt);
					});
				});
			});

			Describe("String Attribute", [this]()
			{
				Describe("Direct string attribute", [this]()
				{
					It("Direct attribute should update on updating column value", [this]()
					{
						const FString InitialValue(TEXT("Test String"));
						const FString UpdatedValue(TEXT("Test string after modification"));
						
						// Add the test int column to the test row
						TedsInterface->AddColumn(TestRowHandle, FTestColumnString{.TestString = InitialValue});
						FTestColumnString* TestColumnString = TedsInterface->GetColumn<FTestColumnString>(TestRowHandle);

						TestNotNull("Expecting valid column", TestColumnString);

						// Create an int attribute and bind it
						FAttributeBinder Binder(TestRowHandle);
						const TAttribute TestAttribute(Binder.BindData(&FTestColumnString::TestString));

						TestEqual("Expecting attribute value to match column value before modification", TestAttribute.Get(), TestColumnString->TestString);

						// Modify the value in the column
						TestColumnString->TestString = UpdatedValue;
						
						TestEqual("Expecting attribute value to update after modification", TestAttribute.Get(), UpdatedValue);
						TestEqual("Expecting attribute value to match column value after modification", TestAttribute.Get(), TestColumnString->TestString);
					});
				});

				Describe("Text attribute bound to string column data", [this]()
				{
					It("Converted attribute should update on updating column value", [this]()
					{
						const FString InitialValue(TEXT("Test String"));
						const FString UpdatedValue(TEXT("Test string after modification"));

						// Add the test int column to the test row
						TedsInterface->AddColumn(TestRowHandle, FTestColumnString{.TestString = InitialValue});
						FTestColumnString* TestColumnString = TedsInterface->GetColumn<FTestColumnString>(TestRowHandle);

						TestNotNull("Expecting valid column", TestColumnString);

						// Create an int attribute and bind it
						FAttributeBinder Binder(TestRowHandle);
						const TAttribute<FText> TestAttribute(Binder.BindData(&FTestColumnString::TestString,
							[](const FString& Data)
							{
								return FText::FromString(Data);
							}));

						TestEqual("Expecting attribute value to match column value before modification", TestAttribute.Get().ToString(), TestColumnString->TestString);

						// Modify the value in the column
						TestColumnString->TestString = UpdatedValue;

						TestEqual("Expecting attribute value to update after modification", TestAttribute.Get().ToString(), UpdatedValue);
						TestEqual("Expecting attribute value to match column value after modification", TestAttribute.Get().ToString(), TestColumnString->TestString);
					});
				});
			});

			Describe("Default Value", [this]()
			{
				It("Default value should be used when column isn't present", [this]()
				{
					constexpr int DefaultValue = 10;
					
					FAttributeBinder Binder(TestRowHandle);

					// Create an int attribute and directly bind it
					const TAttribute TestIntAttribute(Binder.BindData(&FTestColumnInt::TestInt, DefaultValue));

					// Create a float attribute and bind it by providing a conversion function
					const TAttribute<float> TestFloatAttribute(Binder.BindData(&FTestColumnInt::TestInt,
						[](const int& Data)
						{
							return static_cast<float>(Data);
						}, DefaultValue));


					// Remove FTestColumnInt from TestRowHandle so the default value is used
					TedsInterface->RemoveColumn(TestRowHandle, FTestColumnInt::StaticStruct());

					TestEqual("Expecting int attribute value to match default value", TestIntAttribute.Get(), DefaultValue);
					TestEqual("Expecting float attribute value to match default value", static_cast<int>(TestFloatAttribute.Get()), DefaultValue);

				});
			});

			Describe("Bind Column", [this]()
			{
				It("Attribute can be bound to a TEDS column", [this]()
				{
					constexpr int InitialValue = 10;
					constexpr int UpdatedValue = 20;
					
					FAttributeBinder Binder(TestRowHandle);
					
					// Add the test int column to the test row
					TedsInterface->AddColumn(TestRowHandle, FTestColumnInt{.TestInt =  InitialValue});
					FTestColumnInt* TestColumnInt = TedsInterface->GetColumn<FTestColumnInt>(TestRowHandle);

					TestNotNull("Expecting valid column", TestColumnInt);

					// Create an int attribute and bind it to the whole column
					const TAttribute TestIntAttribute(Binder.BindColumn<FTestColumnInt>([](const FTestColumnInt& Column)
					{
						return Column.TestInt;
					}));

					// Update the value in the column
					TestColumnInt->TestInt = UpdatedValue;
					
					TestEqual("Expecting int attribute value to match column value", TestIntAttribute.Get(), TestColumnInt->TestInt);
				});
			});
			
			Describe("Bind Column Data", [this]()
			{
				It("Attribute can be bound to a TEDS column data", [this]()
				{
					constexpr int InitialValue = 10;
					constexpr int UpdatedValue = 20;
					
					FAttributeBinder Binder(TestRowHandle);
					
					// Add the test int column to the test row
					TedsInterface->AddColumn(TestRowHandle, FTestColumnInt{.TestInt =  InitialValue});
					FTestColumnInt* TestColumnInt = TedsInterface->GetColumn<FTestColumnInt>(TestRowHandle);

					TestNotNull("Expecting valid column", TestColumnInt);

					// Create an int attribute and bind it to a whole column 
					const TAttribute TestIntAttribute(Binder.BindColumnData(FTestColumnInt::StaticStruct(),
						[](const TWeakObjectPtr<const UScriptStruct>& ColumnType, const void* Data)
					{
						if (const FTestColumnInt* TestColumn = static_cast<const FTestColumnInt*>(Data))
						{
							return TestColumn->TestInt;
						}

						return 0;
					}));

					// Update the value in the column
					TestColumnInt->TestInt = UpdatedValue;
					
					TestEqual("Expecting int attribute value to match column value", TestIntAttribute.Get(), TestColumnInt->TestInt);
				});
			});
		});

		AfterEach([this]()
		{
			CleanupTestRow(TestRowHandle);
			TestRowHandle = InvalidRowHandle;
			TestTableHandle = InvalidTableHandle;
			TedsInterface = nullptr;
		});
	}
} // namespace UE::Editor::DataStorage::Tests

#endif // WITH_TESTS