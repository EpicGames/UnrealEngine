// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_TESTS
#include "Containers/ArrayView.h"
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementTestColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "TickableEditorObject.h"
#endif

namespace UE::Editor::DataStorage::Tests
{
	TableHandle RegisterTestTableA(ICoreProvider* TedsInterface)
	{
		// RegisterTable is not idempotent, so this utility function makes it so
	
		const FName TestTableAName = TEXT("TestTableA");
		TableHandle Table = TedsInterface->FindTable(TestTableAName);
		if (Table != InvalidTableHandle)
		{
			return Table;
		}
	
		return TedsInterface->RegisterTable(
		{
			FTestColumnA::StaticStruct(),
			FTestColumnB::StaticStruct(),
			FTestTagColumnA::StaticStruct(),
			FTestTagColumnB::StaticStruct()
		},
		TEXT("TestTableA"));
	}

	bool IsEmptyQueryDescription(const FQueryDescription& QueryDescription)
	{
		return
			QueryDescription.Callback.MonitoredType == nullptr &&
			QueryDescription.SelectionTypes.IsEmpty() &&
			QueryDescription.SelectionAccessTypes.IsEmpty() &&
			QueryDescription.SelectionMetaData.IsEmpty() &&
			QueryDescription.ConditionTypes.IsEmpty() &&
			QueryDescription.ConditionOperators.IsEmpty() &&
			QueryDescription.DependencyTypes.IsEmpty() &&
			QueryDescription.DependencyFlags.IsEmpty() &&
			QueryDescription.CachedDependencies.IsEmpty() &&
			QueryDescription.Subqueries.IsEmpty();
	}

	void IncrementThing(int64* QueryCallCountPtr)
	{
		++(*QueryCallCountPtr);
	}

	// Temporary wrapper to only register a single observer
	// There is no way to unregister an observer at this time
	template<typename ColumnType>
	void RegisterTestObserver(
		FName ObserverName,
		ICoreProvider* TedsInterface, Queries::FObserver::EEvent EventType, int64* QueryCallCountPtr, QueryHandle* QueryHandleInOut)
	{
		using namespace UE::Editor::DataStorage::Queries;
		FQueryDescription QueryDescription = TedsInterface->GetQueryDescription(*QueryHandleInOut);

		// Check if it is empty... If not then we need to register
		// This is for an idempotent API to the test harness				
		if (IsEmptyQueryDescription(QueryDescription))
		{
			*QueryHandleInOut = TedsInterface->RegisterQuery(
				Select(
					ObserverName,
					FObserver(EventType, ColumnType::StaticStruct()),
					[QueryCallCountPtr](IQueryContext& Context, RowHandle Row)
					{
						++(*QueryCallCountPtr);
					})
				.Compile());
		}
	}

	BEGIN_DEFINE_SPEC(EditorDataStorageTestsFixture, "Editor.DataStorage.Core", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
		ICoreProvider* TedsInterface = nullptr;
		TableHandle TestTableHandleA = InvalidTableHandle;

		TArray<RowHandle> CreatedRows;

		QueryHandle DataColumnA_AddObserverHandle = InvalidQueryHandle;
		QueryHandle DataColumnC_AddObserverHandle = InvalidQueryHandle;
		QueryHandle DataColumnB_RemoveObserverHandle = InvalidQueryHandle;
		QueryHandle TagColumnA_AddObserverHandle = InvalidQueryHandle;
		QueryHandle TagColumnC_AddObserverHandle = InvalidQueryHandle;
		QueryHandle TagColumnB_RemoveObserverHandle = InvalidQueryHandle;

		// Non-zero to catch problems in test where the difference is not being used
		// Avoids need to reset this to 0 for each test
		int64 DataColumnA_AddObserverCallCount = 10;
		int64 DataColumnC_AddObserverCallCount = 10;
		int64 DataColumnB_RemoveObserverCallCount = 10;
		int64 TagColumnA_AddObserverCallCount = 10;
		int64 TagColumnC_AddObserverCallCount = 10;
		int64 TagColumnB_RemoveObserverCallCount = 10;

		void CreateTestRows(int32 Count)
		{
			CreatedRows.Reserve(Count);
			for (int32 Index = 0; Index < Count; ++Index)
			{
				RowHandle RowHandle = TedsInterface->AddRow(TestTableHandleA);
				TestFalse("Expect valid row handle", RowHandle == InvalidRowHandle);
				CreatedRows.Add(RowHandle);
			}
		}

		void CleanupTestRows()
		{
			for (RowHandle RowHandle : CreatedRows)
			{
				TestTrue("Expected row to have been available", TedsInterface->IsRowAvailable(RowHandle));
				TestTrue("Expected row to have been assigned to a table", TedsInterface->IsRowAssigned(RowHandle));
				TedsInterface->RemoveRow(RowHandle);
				TestFalse("Expected row to have been unassigned a table", TedsInterface->IsRowAssigned(RowHandle));
				TestFalse("Expected row to be not available", TedsInterface->IsRowAvailable(RowHandle));
			}
			CreatedRows.Empty();
		}

#if WITH_EDITOR
		void TestDeferredModifications(bool bIsTestingBatchModifications, const int32 NumberOfTestRows)
		{
			const RowHandle AddedRowHandle = TedsInterface->AddRow(TestTableHandleA);
			TArray<RowHandle> AddedRowHandles;
			AddedRowHandles.Reserve(NumberOfTestRows);
			TedsInterface->BatchAddRow(TestTableHandleA, NumberOfTestRows, [&AddedRowHandles](RowHandle Row)
				{
					AddedRowHandles.Add(Row);
				});


			TArray<RowHandle> RowHandlesAddedByQuery;
			RowHandlesAddedByQuery.Reserve(NumberOfTestRows);
			RowHandle RowHandleAddedByQuery;

			FName TestIdentifier(TEXT("TestIdentifier"));

			// We need to uncomment the part of the test that create and check the dynamic column description once the race condition has been fixed

			using namespace Queries;
			QueryHandle Query = TedsInterface->RegisterQuery(
				Select(
					bIsTestingBatchModifications ? TEXT("Test deferred modifications with batch modifications query") : TEXT("Test deferred modifications query"),
					FProcessor(EQueryTickPhase::DuringPhysics, TedsInterface->GetQueryTickGroupName(EQueryTickGroups::Update))
						.BatchModifications(bIsTestingBatchModifications),
					[this, NumberOfTestRows, AddedRowHandle, &AddedRowHandles, &RowHandlesAddedByQuery, &RowHandleAddedByQuery, TestIdentifier](IQueryContext& Context, const RowHandle* Rows, const FTestColumnA* ColumnA, const FTestColumnB* ColumnB)
					{
						const int32 NumberOfRows = Context.GetRowCount();

						TArrayView<const RowHandle> InRowsArrayView(Rows, NumberOfRows);

						FDynamicColumnDescription DynamicColumnDescription;
						DynamicColumnDescription.Identifier = TestIdentifier;
						DynamicColumnDescription.TemplateType = FTestColumnDynamic::StaticStruct();
						//Context.AddColumns(InRowsArrayView, TArrayView<FDynamicColumnDescription>(&DynamicColumnDescription, 1));
						Context.RemoveColumns<FTestColumnB, FTestTagColumnB>(InRowsArrayView);

						for (int32 Count = 0; Count < NumberOfTestRows; ++Count)
						{
							RowHandlesAddedByQuery.Add(Context.AddRow(TestTableHandleA));
						}

						Context.AddColumns<FTestColumnC, FTestTagColumnC>(RowHandlesAddedByQuery);

						RowHandleAddedByQuery = Context.AddRow(TestTableHandleA);
						Context.AddColumns<FTestColumnC, FTestTagColumnC>(RowHandleAddedByQuery);
						FTestColumnInt IntColumn;
						FTestColumnInt& IntColumnRef = Context.AddColumn<FTestColumnInt>(RowHandleAddedByQuery, MoveTemp(IntColumn));
						IntColumnRef.TestInt = NumberOfTestRows;
						Context.RemoveColumns<FTestTagColumnB>(RowHandleAddedByQuery);
						//Context.AddColumn<FTestColumnDynamic>(RowHandleAddedByQuery, TestIdentifier);

						Context.RemoveRow(AddedRowHandle);
						Context.RemoveRows(AddedRowHandles);
					})
				.Where(TColumn<FTestTagColumnA>() && TColumn<FTestTagColumnB>())
				.Compile());

			// Tick Mass
			FTickableEditorObject::TickObjects(-1.f);


			for (const RowHandle& Row : CreatedRows)
			{
				//TestNotNull(TEXT("CreatedRows have the dynamic column"), TedsInterface->GetColumn<FTestColumnDynamic>(Row, TestIdentifier));
				TestFalse(TEXT("CreatedRows don't have a FTestColumnB and FTestTagColumnB"), TedsInterface->HasColumns<FTestColumnB, FTestTagColumnB>(Row));
			}

			TestFalse(TEXT("The AddedRowHandle should be removed"), TedsInterface->IsRowAssigned(AddedRowHandle));
			for (const RowHandle& Row : AddedRowHandles)
			{
				TestFalse(TEXT("The AddedRowHandles should be removed"), TedsInterface->IsRowAssigned(AddedRowHandle));
			}

			TestTrue(TEXT("The RowHandleAddedByQuery should be assigned"), TedsInterface->IsRowAssigned(RowHandleAddedByQuery));
			TestTrue(TEXT("The RowHandleAddedByQuery should have a column FTestColumnC and a Tag ColumnC"), TedsInterface->HasColumns<FTestColumnC, FTestTagColumnC>(RowHandleAddedByQuery));
			TestFalse(TEXT("The RowHandleAddedByQuery shouldn't have the tag FTestTagColumnB"), TedsInterface->HasColumns<FTestTagColumnB>(RowHandleAddedByQuery));
			//TestNotNull(TEXT("The RowHandleAddedByQuery should have the dynamic column"), TedsInterface->GetColumn<FTestColumnDynamic>(RowHandleAddedByQuery, TestIdentifier));
			{
				const FTestColumnInt* IntColumn = TedsInterface->GetColumn<FTestColumnInt>(RowHandleAddedByQuery);
				if (TestNotNull(TEXT("The RowHandleAddedByQuery should have a FTestColumnInt"), IntColumn))
				{
					TestEqual(TEXT("The RowHandleAddedByQuery FTestColumnInt should have the expected value"), IntColumn->TestInt, NumberOfTestRows);
				}
			}

			for (const RowHandle& Row : RowHandlesAddedByQuery)
			{
				if (TestTrue(TEXT("Check that the RowHandlesAddedByQuery are assigned"), TedsInterface->IsRowAssigned(Row)))
				{
					TestTrue(TEXT("Check that the RowHandlesAddedByQuery have a FTestColumnC and FTestTagColumnC"), TedsInterface->HasColumns<FTestColumnC, FTestTagColumnC>(Row));
				}
			}

			TedsInterface->RemoveRow(RowHandleAddedByQuery);
			TedsInterface->BatchRemoveRows(RowHandlesAddedByQuery);
			TedsInterface->UnregisterQuery(Query);
		}
#endif // WITH_EDITOR

	END_DEFINE_SPEC(EditorDataStorageTestsFixture)

	void EditorDataStorageTestsFixture::Define()
	{
		BeforeEach([this]()
		{
			TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			TestTrue("", TedsInterface != nullptr);
		});

		Describe("", [this]()
		{
			BeforeEach([this]()
			{
				TestTrue("Test requires valid TEDS Interface", TedsInterface != nullptr);
			});
		
			Describe("", [this]()
			{
				Describe("RegisterTable(TConstArrayView<const UScriptStruct*>, const FName)", [this]()
				{
					It("should register a table and provide a valid handle", [this]()
					{
						// Note, this test doesn't really do anything a second time.
						// The test fixture depends on global state which currently cannot be cleaned
						// up.
						TableHandle Handle = RegisterTestTableA(TedsInterface);
						TestNotEqual("Expecting valid table handle", Handle, InvalidTableHandle);
					});
				});
			});

			Describe("", [this]()
			{
				BeforeEach([this]()
				{
					TestTableHandleA = RegisterTestTableA(TedsInterface);
				});
			
				AfterEach([this]()
				{
					CleanupTestRows();
				});
			
				Describe("AddRow(TableHandle)", [this]()
				{
					for (int32 RowCount = 1; RowCount <= 2; ++RowCount)
					{
						It(FString::Printf(TEXT("should should be able to create %d rows when called %d times"), RowCount, RowCount), [this, RowCount]()
						{
							for (int32 CreatedRowIndex = 0; CreatedRowIndex < RowCount; ++CreatedRowIndex)
							{
								RowHandle RowHandle = TedsInterface->AddRow(TestTableHandleA);
								CreatedRows.Add(RowHandle);
								TestNotEqual("Expecting valid row", RowHandle, InvalidRowHandle);
							}
						});
					}
				});

				Describe("BatchAddRow(TableHandle, int32, EditorDataStorageCreationCallbackRef)", [this]()
				{
					for (int32 RowCount = 1; RowCount <= 2; ++RowCount)
					{
						It(FString::Printf(TEXT("should should be able to create %d rows when called once"), RowCount), [this, RowCount]()
						{
							auto RowInitializeCallback = [this](RowHandle RowHandle)
							{
								CreatedRows.Add(RowHandle);
								TestNotEqual("Expecting valid row", RowHandle, InvalidRowHandle);
							};
										
							RowHandle RowHandle = TedsInterface->BatchAddRow(TestTableHandleA, RowCount, RowInitializeCallback);
										
							TestEqual("Unexpected number of created rows", CreatedRows.Num(), RowCount);
						});
					}
				});

		
				Describe("", [this]()
				{
					/**
					 * Prerequisites for this batch of tests are:
					 * - A test table is registered
					 * - An add and remove observer is registered for FTestInt64Column
					 */
					BeforeEach([this]()
					{
						using namespace Queries;

						TestTableHandleA = RegisterTestTableA(TedsInterface);
						TestNotEqual("Expecting valid table handle", TestTableHandleA, InvalidTableHandle);

						RegisterTestObserver<FTestColumnA>(
							TEXT("Increment CallCount when FTestColumnA added"),
							TedsInterface,
							FObserver::EEvent::Add,
							&DataColumnA_AddObserverCallCount,
							&DataColumnA_AddObserverHandle);
						TestTrue("Expect valid query observer handle", DataColumnA_AddObserverHandle != InvalidQueryHandle);
					
						RegisterTestObserver<FTestColumnC>(
							TEXT("Increment CallCount when FTestColumnC added"),
							TedsInterface,
							FObserver::EEvent::Add,
							&DataColumnC_AddObserverCallCount,
							&DataColumnC_AddObserverHandle);
						TestTrue("Expect valid query observer handle", DataColumnC_AddObserverHandle != InvalidQueryHandle);

						RegisterTestObserver<FTestColumnB>(
							TEXT("Increment CallCount when FTestColumnB removed"),
							TedsInterface,
							FObserver::EEvent::Remove,
							&DataColumnB_RemoveObserverCallCount,
							&DataColumnB_RemoveObserverHandle);
						TestTrue("Expect valid query observer handle", DataColumnB_RemoveObserverHandle != InvalidQueryHandle);

						RegisterTestObserver<FTestTagColumnA>(
							TEXT("Increment CallCount when FTestColumnA added"),
							TedsInterface,
							FObserver::EEvent::Add,
							&TagColumnA_AddObserverCallCount,
							&TagColumnA_AddObserverHandle);
						TestTrue("Expect valid query observer handle", TagColumnA_AddObserverHandle != InvalidQueryHandle);
											
						RegisterTestObserver<FTestTagColumnC>(
							TEXT("Increment CallCount when FTestTagColumnC added"),
							TedsInterface,
							FObserver::EEvent::Add,
							&TagColumnC_AddObserverCallCount, 
							&TagColumnC_AddObserverHandle);
						TestTrue("Expect valid query observer handle", TagColumnC_AddObserverHandle != InvalidQueryHandle);

						RegisterTestObserver<FTestTagColumnB>(
							TEXT("Increment CallCount when FTestTagColumnB removed"),
							TedsInterface, FObserver::EEvent::Remove,
							&TagColumnB_RemoveObserverCallCount,
							&TagColumnB_RemoveObserverHandle);
						TestTrue("Expect valid query observer handle", TagColumnB_RemoveObserverHandle != InvalidQueryHandle);					
					});
				
					AfterEach([this]()
					{
						CleanupTestRows();
					});

					Describe("AddRow(TableHandle)", [this]()
					{
						for (int32 RowCount = 1; RowCount <= 2; ++RowCount)
						{
							It(FString::Printf(TEXT("should invoke Add observer %d times when called with table containing data column %d times"), RowCount, RowCount), [this, RowCount]()
							{
								int64 PreviousQueryCallCount = DataColumnA_AddObserverCallCount;
							
								for (int32 CreatedRowIndex = 0; CreatedRowIndex < RowCount; ++CreatedRowIndex)
								{
									RowHandle RowHandle = TedsInterface->AddRow(TestTableHandleA);
									CreatedRows.Add(RowHandle);
								}

								TestEqual(TEXT("Expect observer to be called correct number of times"), DataColumnA_AddObserverCallCount, PreviousQueryCallCount + RowCount);
							});
						}
					});

					Describe("BatchAddRow(TableHandle, int32, EditorDataStorageCreationCallbackRef)", [this]()
					{
						for (int32 RowCount = 1; RowCount <= 2; ++RowCount)
						{
							It(FString::Printf(TEXT("should invoke Add observer %d times when called with table containing data column"), RowCount), [this, RowCount]()
							{
								int64 PreviousQueryCallCount = DataColumnA_AddObserverCallCount;
							
								auto RowInitializeCallback = [this](RowHandle RowHandle)
								{
									CreatedRows.Add(RowHandle);
								};
															
								RowHandle RowHandle = TedsInterface->BatchAddRow(TestTableHandleA, RowCount, RowInitializeCallback);
							
								TestEqual(TEXT("Expect observer to be called correct number of times"), DataColumnA_AddObserverCallCount, PreviousQueryCallCount + RowCount);
							});
						}
					});

					Describe("RegisterQuery", [this]()
					{
						It("QueryCondition", [this]()
						{
							using namespace Queries;
							auto Query = TedsInterface->RegisterQuery(
								Select()
								.ReadOnly<FTestColumnD>()
								.Where((TColumn<FTestColumnA>() && TColumn<FTestColumnB>()) || TColumn<FTestColumnC>())
								.Compile());

							if(!TestNotEqual(TEXT("Expect RegisterQuery to provide a valid query handle"), Query, InvalidQueryHandle))
							{
								return;
							}

							// Should not match
							{
								// Expected to have FTestColumnA && FTestColumnB but not FTestColumnD
								RowHandle Handle = TedsInterface->AddRow(TestTableHandleA);
								CreatedRows.Add(Handle);
							}

							// Should match
							{
								// Expected to have FTestColumnA && FTestColumnB
								RowHandle Handle = TedsInterface->AddRow(TestTableHandleA);
								CreatedRows.Add(Handle);
								TedsInterface->AddColumn<FTestColumnD>(Handle, FTestColumnD{});
							}

							// Should not match
							{
								// Expected to have FTestColumnA only
								RowHandle Handle = TedsInterface->AddRow(TestTableHandleA);
								CreatedRows.Add(Handle);
								TedsInterface->RemoveColumn<FTestColumnB>(Handle);
								TedsInterface->AddColumn<FTestColumnD>(Handle, FTestColumnD{});
							}

							// Should match
							{
								// Expected to have FTestColumnC only
								RowHandle Handle = TedsInterface->AddRow(TestTableHandleA);
								CreatedRows.Add(Handle);
								TedsInterface->RemoveColumn<FTestColumnA>(Handle);
								TedsInterface->RemoveColumn<FTestColumnB>(Handle);
								TedsInterface->AddColumn<FTestColumnC>(Handle, FTestColumnC{});
								TedsInterface->AddColumn<FTestColumnD>(Handle, FTestColumnD{});
							}

							FQueryResult Result = TedsInterface->RunQuery(
								Query,
								CreateDirectQueryCallbackBinding([](IDirectQueryContext& Context, const RowHandle* RowHandles, const FTestColumnD* DColumns)
							{
									
							}));

							TestEqual(TEXT("Expect number of rows matched equal to created rows"), Result.Count, 2);

							TedsInterface->UnregisterQuery(Query);
						});
					});

					Describe("", [this]()
					{
						BeforeEach([this]()
						{
							CreateTestRows(1);
						});

						Describe("RemoveRow(TableHandle)", [this]()
						{
							It("should remove row when called", [this]()
							{
								RowHandle Row = CreatedRows[0];
								TestTrue("Row is available", TedsInterface->IsRowAvailable(Row));

								TedsInterface->RemoveRow(Row);
							

								TestFalse("Row is not available", TedsInterface->IsRowAvailable(Row));

								CreatedRows.RemoveAt(0);
							});
						
							It("should call observer of removed data column when called on row that has the column", [this]()
							{
								int64 PreviousQueryCallCount = DataColumnB_RemoveObserverCallCount;
							
								RowHandle Row = CreatedRows[0];
								TedsInterface->RemoveRow(Row);

								TestEqual("Expected observer called", DataColumnB_RemoveObserverCallCount, PreviousQueryCallCount + 1);

								CreatedRows.RemoveAt(0);
							});
						
							It("should call observer of removed tag column when called on row that has the column", [this]()
							{
								int64 PreviousQueryCallCount = TagColumnB_RemoveObserverCallCount;
							
								RowHandle Row = CreatedRows[0];
								TedsInterface->RemoveRow(Row);
							
								TestEqual("Expected observer called", TagColumnB_RemoveObserverCallCount, PreviousQueryCallCount + 1);

								CreatedRows.RemoveAt(0);
							});
						});
					
						Describe("AddColumn(RowHandle, const UScriptStruct*)", [this]()
						{					
							It("should add data column when called", [this]()
							{
								RowHandle Row = CreatedRows[0];
								TestFalse("Expect row to not have column about to be added",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestColumnC::StaticStruct()})));

								TedsInterface->AddColumn(Row, FTestColumnC::StaticStruct());

								TestTrue("Expect row to have column added",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestColumnC::StaticStruct()})));

							});

							It("should invoke registered observer when called with data column", [this]()
							{
								int64 PreviousQueryCallCount = DataColumnC_AddObserverCallCount;
								RowHandle Row = CreatedRows[0];
								TedsInterface->AddColumn(Row, FTestColumnC::StaticStruct());

								TestEqual("Expect observer to have been called one time", DataColumnC_AddObserverCallCount, PreviousQueryCallCount + 1);
							});

							It("should add tag column when called", [this]()
							{
								RowHandle Row = CreatedRows[0];
								TestFalse("Expect row to not have column about to be added",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestTagColumnC::StaticStruct()})));

								TedsInterface->AddColumn(Row, FTestTagColumnC::StaticStruct());

								TestTrue("Expect row to have column added",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestTagColumnC::StaticStruct()})));

							});

							It("should invoke registered observer when called with tag column", [this]()
							{
								int64 PreviousQueryCallCount = TagColumnC_AddObserverCallCount;
								RowHandle Row = CreatedRows[0];
								TedsInterface->AddColumn(Row, FTestTagColumnC::StaticStruct());

								TestEqual("Expect observer to have been called one time", TagColumnC_AddObserverCallCount, PreviousQueryCallCount + 1);
							});
						});
					
						Describe("RemoveColumn(RowHandle, TConstArrayView<const UScriptStruct*>)", [this]()
						{
							It("should remove a single column when called", [this]()
							{
								RowHandle Row = CreatedRows[0];
								TestTrue("Expected to have column about to be removed",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestColumnB::StaticStruct()})));
							
								TedsInterface->RemoveColumn(Row, FTestColumnB::StaticStruct());

								TestFalse("Expected to no longer have removed column",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestColumnB::StaticStruct()})));
							});

							It("should invoke registered data column observer when called", [this]()
							{
								int64 PreviousQueryCallCount = DataColumnB_RemoveObserverCallCount;
								RowHandle Row = CreatedRows[0];
							
								TedsInterface->RemoveColumn(Row, FTestColumnB::StaticStruct());
							
								TestEqual("Expect observer to have been called one time", DataColumnB_RemoveObserverCallCount, PreviousQueryCallCount + 1);
							});

							It("should remove a single tag column when called", [this]()
							{
								RowHandle Row = CreatedRows[0];
								TestTrue("Expected to have column about to be removed",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestTagColumnB::StaticStruct()})));
													
								TedsInterface->RemoveColumn(Row, FTestTagColumnB::StaticStruct());

								TestFalse("Expected to no longer have removed column",
									TedsInterface->HasColumns(Row, MakeArrayView({FTestTagColumnB::StaticStruct()})));
							});

							It("should invoke registered tag column observer when called", [this]()
							{
								int64 PreviousQueryCallCount = TagColumnB_RemoveObserverCallCount;
								RowHandle Row = CreatedRows[0];
							
								TedsInterface->RemoveColumn(Row, FTestTagColumnB::StaticStruct());
							
								TestEqual("Expect observer to have been called one time", TagColumnB_RemoveObserverCallCount, PreviousQueryCallCount + 1);
							});
						});

						Describe("AddColumns(RowHandle, TConstArrayView<const UScriptStruct*>)", [this]()
						{
							It("should add a single column when called with single data column", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnC::StaticStruct(), FTestColumnD::StaticStruct()};
							
								RowHandle Row = CreatedRows[0];
								TestFalse("Expected to not have column about to be added",
									TedsInterface->HasColumns(Row, Columns));

								TedsInterface->AddColumns(Row, Columns);

								TestTrue("Expected to have column about to be added", 
								TedsInterface->HasColumns(Row, Columns));
							});

							It("should add multiple column when called with multiple data columns", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnC::StaticStruct(), FTestColumnD::StaticStruct()};
							
								RowHandle Row = CreatedRows[0];
								TestFalse("Expected to not have column about to be added", 
								TedsInterface->HasColumns(Row, Columns));

								TedsInterface->AddColumns(Row, Columns);

								TestTrue("Expected to have column about to be added",
									TedsInterface->HasColumns(Row, Columns));
							});

							It("should invoke registered data column observer when called", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnC::StaticStruct()};
							
								int64 PreviousQueryCallCount = DataColumnC_AddObserverCallCount;
								RowHandle Row = CreatedRows[0];
							
								TedsInterface->AddColumns(Row, Columns);

								TestEqual("Expect observer to have been called one time", DataColumnC_AddObserverCallCount, PreviousQueryCallCount + 1);
							});

							It("should add a single column when called with single tag column", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestTagColumnC::StaticStruct(), FTestTagColumnD::StaticStruct()};
													
								RowHandle Row = CreatedRows[0];
								TestFalse("Expected to not have column about to be added",
									TedsInterface->HasColumns(Row, Columns));

								TedsInterface->AddColumns(Row, Columns);

								TestTrue("Expected to have column about to be added", 
								TedsInterface->HasColumns(Row, Columns));
							});

							It("should add multiple column when called with multiple tag columns", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestTagColumnC::StaticStruct(), FTestTagColumnD::StaticStruct()};
													
								RowHandle Row = CreatedRows[0];
								TestFalse("Expected to not have column about to be added", 
								TedsInterface->HasColumns(Row, Columns));

								TedsInterface->AddColumns(Row, Columns);

								TestTrue("Expected to have column about to be added",
									TedsInterface->HasColumns(Row, Columns));
							});

							It("should invoke registered tag column added observer when called", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestTagColumnC::StaticStruct()};
													
								int64 PreviousQueryCallCount = TagColumnC_AddObserverCallCount;
								RowHandle Row = CreatedRows[0];
								TedsInterface->AddColumns(Row, Columns);

								TestEqual("Expect observer to have been called one time", TagColumnC_AddObserverCallCount, PreviousQueryCallCount + 1);
							});
						});

						Describe("RemoveColumns(RowHandle, TConstArrayView<const UScriptStruct*>)", [this]()
						{
							It("should remove a single column when RemoveColumns called with single column", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnB::StaticStruct()};
							
								RowHandle Row = CreatedRows[0];
								TestTrue("Expected to not column about to be removed",
									TedsInterface->HasColumns(Row, Columns));

								TedsInterface->RemoveColumns(Row, Columns);

								TestFalse("Expected to not have column that was removed", 
								TedsInterface->HasColumns(Row, MakeArrayView({FTestColumnB::StaticStruct()})));
							});

							It("should remove all columns when RemoveColumns called with multiple columns", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnA::StaticStruct(), FTestColumnB::StaticStruct()};
							
								RowHandle Row = CreatedRows[0];
								TestTrue("Expected to have columns about to be removed", 
								TedsInterface->HasColumns(Row, Columns));

								TedsInterface->RemoveColumns(Row, Columns);

								TestFalse("Expected to have column about to be added",
									TedsInterface->HasColumns(Row, Columns));
							});

							It("should invoke registered observer when RemoveColumns called", [this]()
							{
								TArray<const UScriptStruct*> Columns = {FTestColumnB::StaticStruct()};
							
								int64 PreviousQueryCallCount = DataColumnB_RemoveObserverCallCount;
								RowHandle Row = CreatedRows[0];
								TedsInterface->RemoveColumns(Row, Columns);

								TestEqual("Expect observer to have been called one time", DataColumnB_RemoveObserverCallCount, PreviousQueryCallCount + 1);
							});
						});
					});

#if WITH_EDITOR
					Describe("", [this]()
						{
							constexpr int32 NumberOfTestRows = 4;

							BeforeEach([this]()
								{
									CreateTestRows(NumberOfTestRows);
								});

							It("Processor Deferred Commands", [this]()
								{
									constexpr bool bIsTestingBatchModifications = false;
									TestDeferredModifications(bIsTestingBatchModifications, NumberOfTestRows);
								});

							It("Processor Deferred Commands with Batch Modifications", [this]()
								{
									constexpr bool bIsTestingBatchModifications = true;
									TestDeferredModifications(bIsTestingBatchModifications, NumberOfTestRows);
								});
						});
#endif // WITH_EDITOR

				});
			
			});
		});
	}
} // namespace UE::Editor::DataStorage::Tests

#endif