// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include <chrono>
#include <condition_variable>
#include <thread>
#include "TypedElementDatabaseScratchBuffer.h"
#include "Tests/TestHarnessAdapter.h"

// Tries to shutdown all threads within the maximum wait time. If it takes longer than the maximum wait time the test will fail.
template<int ThreadCount, class Rep, class Period>
static void WaitForFinalization(std::condition_variable& WaitVariable, std::mutex& WaitMutex, 
	std::atomic<bool>& bRunFlag, std::atomic<uint32>& CompletedThreadCount, const std::chrono::duration<Rep, Period>& MaxWaitTime)
{
	std::unique_lock<std::mutex> WaitLock(WaitMutex); // Wait to be re-enabled again.
	bRunFlag = false;
	std::chrono::system_clock::time_point StartTime = std::chrono::system_clock::now();
	while (CompletedThreadCount != ThreadCount 
		&& std::chrono::duration_cast<std::chrono::milliseconds>(std::chrono::system_clock::now() - StartTime) < MaxWaitTime)
	{
		WaitVariable.wait_for(WaitLock, MaxWaitTime);
	}
	REQUIRE(CompletedThreadCount == ThreadCount);
}

TEST_CASE_NAMED(Editor_DataStorage_ScratchBuffer_MultiThreaded_Tests, "Editor::DataStorage::Scratch Buffer - MT (FScratchBuffer)", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	SECTION("Stress test without recycling.")
	{
		using namespace std::chrono_literals;
		constexpr uint32 ThreadCount = 8;

		FScratchBuffer Buffer;

		std::atomic<bool> bKeepRunning = true;
		std::atomic<uint32> FailedAllocations = 0;
		std::atomic<uint32> CompletedThreadCount = 0;
		std::condition_variable CompletionSync;
		std::mutex CompletionMutex;
		TArray<std::thread> Threads;
		Threads.Reserve(ThreadCount);
		for (uint32 ThreadCounter = 0; ThreadCounter < ThreadCount; ++ThreadCounter)
		{
			Threads.Emplace([&Buffer, &bKeepRunning, &FailedAllocations, &CompletedThreadCount, &CompletionSync]()
				{
					while (bKeepRunning)
					{
						void* Data = Buffer.AllocateUninitialized(128, 4);
						if (Data == nullptr)
						{
							FailedAllocations++;
						}
					}
					if (++CompletedThreadCount == ThreadCount)
					{
						CompletionSync.notify_all();
					}
				});
		}

		std::this_thread::sleep_for(2000ms);
		WaitForFinalization<ThreadCount>(CompletionSync, CompletionMutex, bKeepRunning, CompletedThreadCount, 500ms);
		
		for (std::thread& Thread : Threads)
		{
			Thread.join();
		}

		CHECK(FailedAllocations == 0);
	}

	SECTION("Stress test with recycling.")
	{
		using namespace std::chrono_literals;

		constexpr uint32 ThreadCount = 8;

		FScratchBuffer Buffer;

		std::atomic<bool> bKeepRunning = true;
		std::atomic<uint32> FailedAllocations = 0;

		std::atomic<uint32> CompletedThreadCount = 0;
		std::condition_variable CompletionSync;
		std::mutex CompletionMutex;

		TArray<std::thread> Threads;
		Threads.Reserve(ThreadCount);
		for (uint32 ThreadCounter = 0; ThreadCounter < ThreadCount; ++ThreadCounter)
		{
			Threads.Emplace([
				ThreadCounter, &Buffer, &bKeepRunning, &FailedAllocations, &CompletedThreadCount, &CompletionSync]()
				{
					static constexpr uint32 MemoryAllocationSizes[] = { 128, 32, 14332, 741, 8871, 48, 27335 };
					static constexpr uint32 MemoryAllocationSizeCount = sizeof(MemoryAllocationSizes) / sizeof(MemoryAllocationSizes[0]);
					
					uint32 MemoryAllocationSizeIndex = ThreadCounter;
					while (bKeepRunning)
					{
						// Not using % as there's currently a false positive in MSVC that marks it as a warning (C6385).
						++MemoryAllocationSizeIndex;
						if (MemoryAllocationSizeIndex >= MemoryAllocationSizeCount)
						{
							MemoryAllocationSizeIndex = 0;
						}

						void* Data = Buffer.AllocateUninitialized(static_cast<size_t>(MemoryAllocationSizes[MemoryAllocationSizeIndex]), 4);
						if (Data == nullptr)
						{
							FailedAllocations++;
						}
						// Sleep a little while as to not to end up with an excessive amount of memory allocations.
						std::this_thread::sleep_for(1ms);
					}
					if (++CompletedThreadCount == ThreadCount)
					{
						CompletionSync.notify_all();
					}
				});
		}

		// While the threads are continuously allocating memory, use the main thread to periodically recycle blocks.
		for (uint32 IterationCounter = 0; IterationCounter < 60; ++IterationCounter)
		{
			Buffer.BatchDelete();
			std::this_thread::sleep_for(33ms);
		}
		WaitForFinalization<ThreadCount>(CompletionSync, CompletionMutex, bKeepRunning, CompletedThreadCount, 500ms);

		for (std::thread& Thread : Threads)
		{
			Thread.join();
		}

		CHECK(FailedAllocations == 0);
	}
}

TEST_CASE_NAMED(Editor_DataStorage_ScratchBuffer_Tests, "Editor::DataStorage::Scratch Buffer (FScratchBuffer)", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	SECTION("Create and destroy buffer")
	{
		FScratchBuffer Buffer;
	}

	SECTION("Allocate small block")
	{
		FScratchBuffer Buffer;
		void* Data = Buffer.AllocateUninitialized(4, 4);
		CHECK(Data != nullptr);
	}

	SECTION("Allocate over-sized block")
	{
		FScratchBuffer Buffer;
		void* Data = Buffer.AllocateUninitialized(Buffer.MaxAllocationSize() * 4, 4);
		CHECK(Data != nullptr);
	}

	SECTION("Alignment respected")
	{
		FScratchBuffer Buffer;
		void* Data1 = Buffer.AllocateUninitialized(1, 1);
		void* Data2 = Buffer.AllocateUninitialized(4, 4);

		REQUIRE(Data1 != nullptr);
		REQUIRE(Data2 != nullptr);
		// Check alignment is respected.
		CHECK((reinterpret_cast<uintptr_t>(Data2) & 3) == 0);
	}

	SECTION("Multiple blocks used.")
	{
		FScratchBuffer Buffer;

		int32 IncrementCount = (Buffer.MaxAllocationSize() * 4 /* Fill 4 blocks*/) / 64 /* With 64 byte allocations */;
		for (int32 Counter = 0; Counter < IncrementCount; ++Counter)
		{
			void* Data = Buffer.AllocateUninitialized(64, 4);
			REQUIRE(Data != nullptr);
		}
	}

	SECTION("Recycle full blocks.")
	{
		FScratchBuffer Buffer;

		for (int32 Iterations = 0; Iterations < 16; ++Iterations)
		{
			int32 IncrementCount = Buffer.MaxAllocationSize() / 64;
			for (int32 Counter = 0; Counter <= IncrementCount; ++Counter) // Increment one more than needed to fill the block and make sure it's recycled.
			{
				void* Data = Buffer.AllocateUninitialized(64, 4);
				REQUIRE(Data != nullptr);
			}
			Buffer.BatchDelete();
		}
	}

	SECTION("Order of destruction.")
	{
		int Counter = 5;
		
		{
			FScratchBuffer Buffer;
			struct TestObject
			{
				int Id;
				int* Counter;
				TestObject(int Id, int* Counter) : Id(Id), Counter(Counter) {}
				~TestObject()
				{
					REQUIRE(Id == *Counter);
					(*Counter)--;
				}
			};

			for (int Index = 0; Index <= Counter; ++Index)
			{
				Buffer.Emplace<TestObject>(Index, &Counter);
			}

			// The objects are expected to be deleted in the reverse order they are declared.
		}
		REQUIRE(Counter == -1);
	}
}
#endif // WITH_TESTS
