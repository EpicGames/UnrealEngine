// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/WidgetReferenceColumnUpdateProcessor.h"

#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WidgetReferenceColumnUpdateProcessor)

void UWidgetReferenceColumnUpdateFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterDeleteRowOnWidgetDeleteQuery(DataStorage);
	RegisterDeleteColumnOnWidgetDeleteQuery(DataStorage);
}

void UWidgetReferenceColumnUpdateFactory::RegisterDeleteRowOnWidgetDeleteQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
    	Select(
    		TEXT("Delete row with deleted widget"),
    		FPhaseAmble(FPhaseAmble::ELocation::Preamble, EQueryTickPhase::FrameEnd),
    		[](IQueryContext& Context, RowHandle Row, const FTypedElementSlateWidgetReferenceColumn& WidgetReference)
    		{
    			if (!WidgetReference.TedsWidget.IsValid())
    			{
    				Context.RemoveRow(Row);
    			}
    		})
    	.Where()
    		.All<FTypedElementSlateWidgetReferenceDeletesRowTag>()
    	.Compile());
}

void UWidgetReferenceColumnUpdateFactory::RegisterDeleteColumnOnWidgetDeleteQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Delete widget columns for deleted widget"),
			FPhaseAmble(FPhaseAmble::ELocation::Preamble, EQueryTickPhase::FrameEnd),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementSlateWidgetReferenceColumn& WidgetReference)
			{
				if (!WidgetReference.TedsWidget.IsValid())
				{
					Context.RemoveColumns<FTypedElementSlateWidgetReferenceColumn>(Row);
				}
			})
		.Where()
			.None<FTypedElementSlateWidgetReferenceDeletesRowTag>()
		.Compile());
}
