// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "WidgetReferenceColumnUpdateProcessor.generated.h"

/**
 * Queries that check whether or not a widget still exists. If it has been deleted
 * then it will remove the column from the Data Storage or deletes the entire row if
 * the FTypedElementSlateWidgetReferenceDeletesRowTag was found.
 */
UCLASS()
class UWidgetReferenceColumnUpdateFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UWidgetReferenceColumnUpdateFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterDeleteRowOnWidgetDeleteQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	void RegisterDeleteColumnOnWidgetDeleteQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
