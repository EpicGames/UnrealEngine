// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsUIModule.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "TedsUIModule"

namespace UE::Editor::DataStorage::Private
{
	FAutoConsoleCommandWithOutputDevice PrintWidgetPurposesConsoleCommand(
	TEXT("TEDS.UI.PrintWidgetPurposes"),
	TEXT("Prints a list of all the known widget purposes."),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Output)
		{
			if (IUiProvider* UiStorage = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName))
			{
				Output.Log(TEXT("The Typed Elements Data Storage has recorded the following widget purposes:"));
				UiStorage->ListWidgetPurposes(
					[&Output](FName Purpose, IUiProvider::EPurposeType, const FText& Description)
					{
						Output.Logf(TEXT("    %s - %s"), *Purpose.ToString(), *Description.ToString());
					});
				Output.Log(TEXT("End of Typed Elements Data Storage widget purpose list."));
			}
		}));

	static bool bUseNewTEDSUIWidgets = false;
	FAutoConsoleVariableRef UseNewWidgetsCvar(
		TEXT("TEDS.UI.UseNewWidgets"),
		bUseNewTEDSUIWidgets,
		TEXT("If true, TEDS UI will use new attribute binding driven widgets (needs to be set at startup)")
	);
} // namespace UE::Editor::DataStorage::Private

void FTedsUIModule::StartupModule()
{
}

void FTedsUIModule::ShutdownModule()
{
}

void FTedsUIModule::AddReferencedObjects(FReferenceCollector& Collector)
{
}

FString FTedsUIModule::GetReferencerName() const
{
	return TEXT("TEDS: Editor Data Storage UI Module");
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FTedsUIModule, TedsUI)