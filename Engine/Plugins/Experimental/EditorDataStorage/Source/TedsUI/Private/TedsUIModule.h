// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "UObject/GCObject.h"

class FReferenceCollector;

class FTedsUIModule : public IModuleInterface, public FGCObject
{
public:
	~FTedsUIModule() override = default;

	//
	// IModuleInterface
	//

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	
	//
	// FGCObject
	//
	
	void AddReferencedObjects(FReferenceCollector& Collector) override;
	FString GetReferencerName() const override;
};
