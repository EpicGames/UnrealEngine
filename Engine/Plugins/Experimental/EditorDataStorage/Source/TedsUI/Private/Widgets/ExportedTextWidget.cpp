// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ExportedTextWidget.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ExportedTextWidget)

#define LOCTEXT_NAMESPACE "TedsUI_ExportedTextWidget"

//
// UExportedTextWidgetFactory
//

void UExportedTextWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetFactory(DataStorageUi.FindPurpose(DataStorageUi.GetDefaultWidgetPurposeID()), FExportedTextWidgetConstructor::StaticStruct());
}

//
// FExportedTextWidgetConstructor
//

FExportedTextWidgetConstructor::FExportedTextWidgetConstructor()
	: Super(FExportedTextWidgetConstructor::StaticStruct())
{
}

TConstArrayView<const UScriptStruct*> FExportedTextWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;

	static TTypedElementColumnTypeList<
    		FTypedElementRowReferenceColumn,
    		FTypedElementScriptStructTypeInfoColumn,
    		FExportedTextWidgetTag> Columns;
    	return Columns;
}

const UE::Editor::DataStorage::Queries::FConditions* FExportedTextWidgetConstructor::GetQueryConditions(const UE::Editor::DataStorage::ICoreProvider* Storage) const
{
	// For the exported text widget, the query condition we are matched against is the column we are exporting text for
	if (MatchedColumn.IsCompiled() && !MatchedColumn.IsEmpty())
	{
		return &MatchedColumn;
	}

	return nullptr;
}

FText FExportedTextWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	if (FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = DataStorage->GetColumn<FTypedElementScriptStructTypeInfoColumn>(Row))
	{
		return DescribeColumnType(TypeInfoColumn->TypeInfo.Get());
	}
	// The default behavior is to display the name of the column this widget matched against
	// NOTE: This currently only works if CreateWidget has been called at least once because we rely on the TypeInfoColumn on the widget row
	// to know which column we matched against
	else if (MatchedColumn.IsCompiled() && !MatchedColumn.IsEmpty())
	{
		return DescribeColumnType(MatchedColumn.GetColumns()[0].Get());
	}

	return FText::GetEmpty();
}

TSharedPtr<SWidget> FExportedTextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	FAttributeBinder Binder(TargetRow, DataStorage);
	
	if (FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = DataStorage->GetColumn<FTypedElementScriptStructTypeInfoColumn>(WidgetRow))
	{
		TWeakObjectPtr<const UScriptStruct> ColumnTypeInfo = TypeInfoColumn->TypeInfo;

		if(ColumnTypeInfo.IsValid())
		{
			// NOTE: We are currently assuming that an instance of FExportedTextWidgetConstructor will only be used to show the same type info for all rows
			// matched with it. This isn't ideal but it's better than nothing since we need some sort of matched conditions for column based virtualization to work.
			// TEDS UI TODO: We should work around it by refactoring this into an STedsWidget in the future so it can store the column conditions per instance
			MatchedColumn = FConditions(TColumn(ColumnTypeInfo)).Compile(FEditorStorageQueryConditionCompileContext(DataStorage));
	
			return SNew(STextBlock)
					.Text(Binder.BindColumnData(ColumnTypeInfo, [](const TWeakObjectPtr<const UScriptStruct>& InTypeInfo, const void* InData)
					{
						if (InTypeInfo.IsValid())
						{
							if (InData)
							{
								FString Label;
								InTypeInfo->ExportText(Label, InData, InData, nullptr, PPF_None, nullptr);
								return FText::FromString(Label);
							}
							else
							{
								return FText::Format(LOCTEXT("ColumnNotFoundText", "Column {0} not found on row"), InTypeInfo->GetDisplayNameText());
							}
						}
						return LOCTEXT("MissingTypeInfoText", "Missing type info for column");
					}));
		}
	}
	
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
