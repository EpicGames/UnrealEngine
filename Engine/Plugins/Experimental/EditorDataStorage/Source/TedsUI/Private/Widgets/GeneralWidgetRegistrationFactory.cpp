// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/GeneralWidgetRegistrationFactory.h"

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeneralWidgetRegistrationFactory)

#define LOCTEXT_NAMESPACE "TypedElementsUI_GeneralRegistration"

const FName UGeneralWidgetRegistrationFactory::LargeCellPurpose(TEXT("General.Cell.Large"));
const FName UGeneralWidgetRegistrationFactory::HeaderPurpose(TEXT("General.Header"));

void UGeneralWidgetRegistrationFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
		DataStorageUi.GetDefaultWidgetPurposeID(),
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "Default",
			UE::Editor::DataStorage::IUiProvider::EPurposeType::Generic,
			LOCTEXT("GeneralCellDefaultPurpose", "The default widget to use in cells if no other specialization is provide.")));
	
	DataStorageUi.RegisterWidgetPurpose(
		DataStorageUi.GetGeneralWidgetPurposeID(),
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("GeneralCellPurpose", "General purpose widgets that can be used as cells for specific columns or column combinations.")));

	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "Large",
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("GeneralLargeCellPurpose", "General purpose widgets that are specifically designed to be embedded in a space larger than a single cell."),
			DataStorageUi.GetGeneralWidgetPurposeID()));
	
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Header", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("GeneralHeaderPurpose", "General purpose widget that can be used as a header.")));
}

#undef LOCTEXT_NAMESPACE
