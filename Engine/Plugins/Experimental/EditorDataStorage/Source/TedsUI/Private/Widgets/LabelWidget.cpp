// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/LabelWidget.h"

#include "ActorEditorUtils.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementValueCacheColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LabelWidget)

#define LOCTEXT_NAMESPACE "TedsUI_LabelWidget"

//
// ULabelWidgetFactory
//

void ULabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), 
		TColumn<FTypedElementLabelColumn>() || (TColumn<FTypedElementLabelColumn>() && TColumn<FTypedElementLabelHashColumn>()));

	DataStorageUi.RegisterWidgetFactory<FLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID()), 
		TColumn<FTypedElementLabelColumn>() || (TColumn<FTypedElementLabelColumn>() && TColumn<FTypedElementLabelHashColumn>()));
}

void ULabelWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("GeneralRowLabelPurpose", "Specific purpose to request a widget to display a user facing display name for a row."),
			DataStorageUi.GetGeneralWidgetPurposeID()));
}


//
// FLabelWidgetConstructor
//

FLabelWidgetConstructor::FLabelWidgetConstructor()
	: Super(StaticStruct())
{
}

TConstArrayView<const UScriptStruct*> FLabelWidgetConstructor::GetAdditionalColumnsList() const
{
	static const UE::Editor::DataStorage::TTypedElementColumnTypeList<
		FTypedElementRowReferenceColumn,
		FTypedElementU64IntValueCacheColumn,
		FExternalWidgetSelectionColumn> Columns;
	return Columns;
}

TSharedPtr<SWidget> FLabelWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, 
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(STextBlock)
		.Text(Binder.BindText(&FTypedElementLabelColumn::Label))
		.ToolTipText(Binder.BindText(&FTypedElementLabelColumn::Label));
}

#undef LOCTEXT_NAMESPACE
