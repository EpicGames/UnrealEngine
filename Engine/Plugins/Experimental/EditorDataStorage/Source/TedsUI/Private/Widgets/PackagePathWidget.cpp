// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PackagePathWidget.h"

#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Text/STextBlock.h"

//
// UPackagePathWidgetFactory
//

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackagePathWidget)

void UPackagePathWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FPackagePathWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementPackagePathColumn>());
	DataStorageUi.RegisterWidgetFactory<FLoadedPackagePathWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementPackageLoadedPathColumn>());
}



//
// FPackagePathWidgetConstructor
//

FPackagePathWidgetConstructor::FPackagePathWidgetConstructor()
	: Super(FPackagePathWidgetConstructor::StaticStruct())
{
}

FPackagePathWidgetConstructor::FPackagePathWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(FPackagePathWidgetConstructor::StaticStruct())
{
}

TSharedPtr<SWidget> FPackagePathWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(STextBlock)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		.Justification(ETextJustify::Right);
}

bool FPackagePathWidgetConstructor::FinalizeWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row,
	const TSharedPtr<SWidget>& Widget)
{
	UE::Editor::DataStorage::RowHandle TargetRow = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row)->Row;
	if (const FTypedElementPackagePathColumn* Path = DataStorage->GetColumn<FTypedElementPackagePathColumn>(TargetRow))
	{
		STextBlock* TextWidget = static_cast<STextBlock*>(Widget.Get());
		FText Text = FText::FromString(Path->Path);
		TextWidget->SetToolTipText(Text);
		TextWidget->SetText(MoveTemp(Text));
		return true;
	}
	else
	{
		return false;
	}
}



//
// FLoadedPackagePathWidgetConstructor
//

FLoadedPackagePathWidgetConstructor::FLoadedPackagePathWidgetConstructor()
	: Super(FLoadedPackagePathWidgetConstructor::StaticStruct())
{
}

bool FLoadedPackagePathWidgetConstructor::FinalizeWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row,
	const TSharedPtr<SWidget>& Widget)
{
	UE::Editor::DataStorage::RowHandle TargetRow = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row)->Row;
	if (const FTypedElementPackageLoadedPathColumn* Path = DataStorage->GetColumn<FTypedElementPackageLoadedPathColumn>(TargetRow))
	{
		STextBlock* TextWidget = static_cast<STextBlock*>(Widget.Get());
		FText Text = FText::FromString(Path->LoadedPath.GetLocalFullPath());
		TextWidget->SetToolTipText(Text);
		TextWidget->SetText(MoveTemp(Text));
		return true;
	}
	else
	{
		return false;
	}
}
