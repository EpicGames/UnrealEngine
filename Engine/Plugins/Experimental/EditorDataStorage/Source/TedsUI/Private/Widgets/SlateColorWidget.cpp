// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SlateColorWidget.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Colors/SColorPicker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateColorWidget)

namespace UE::SlateColorWidget::Private
{
	FReply SummonColorPicker(const FGeometry& Geometry, const FPointerEvent& PointerEvent,
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle TargetRow)
	{
		if (PointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
		{
			return FReply::Unhandled();
		}
	
		FColorPickerArgs PickerArgs;
		PickerArgs.bUseAlpha = true;
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([TargetRow, DataStorage](FLinearColor Color)
		{
			DataStorage->GetColumn<FSlateColorColumn>(TargetRow)->Color = Color;
			DataStorage->AddColumn(TargetRow, FTypedElementSyncBackToWorldTag::StaticStruct());
		});

		FSlateColor ColumnColor = DataStorage->GetColumn<FSlateColorColumn>(TargetRow)->Color;
		// If the color isn't set to a specified value, we just show it as white by default in the picker
		FLinearColor InitColor = ColumnColor.IsColorSpecified() ? ColumnColor.GetSpecifiedColor() : FLinearColor::White;
	
		PickerArgs.InitialColor = InitColor;
		PickerArgs.bOnlyRefreshOnOk = true;
		OpenColorPicker(PickerArgs);
		return FReply::Handled();
	}
}

void USlateColorWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FSlateColorWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FSlateColorColumn>());
}

FSlateColorWidgetConstructor::FSlateColorWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}



TSharedPtr<SWidget> FSlateColorWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	FPointerEventHandler OnMouseButtonDown;
	
	UE::Editor::DataStorage::FMetaDataEntryView MetaDataEntryView = Arguments.FindForColumn<FSlateColorColumn>(UE::Editor::DataStorage::IsEditableName);

	if (MetaDataEntryView.IsSet())
	{
		const bool* IsEditable = MetaDataEntryView.TryGetExact<bool>();
		
		// If the column is editable, summon a color picker on click
		if(IsEditable && *IsEditable)
		{
			OnMouseButtonDown = FPointerEventHandler::CreateStatic(&UE::SlateColorWidget::Private::SummonColorPicker, DataStorage, TargetRow);
		}
	}
	
	return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
					.Color(Binder.BindData(&FSlateColorColumn::Color, [](const FSlateColor& Color)
					{
						// If the color isn't set to a specified value, we just show it as white by default in the picker
						return Color.IsColorSpecified() ? Color.GetSpecifiedColor() : FLinearColor::White;
					}))
					.Size(FVector2D(64, 16))
					.OnMouseButtonDown(OnMouseButtonDown)
			];
}
