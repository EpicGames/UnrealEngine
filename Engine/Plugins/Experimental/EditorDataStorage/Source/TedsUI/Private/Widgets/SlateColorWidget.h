// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "SlateColorWidget.generated.h"

UCLASS()
class USlateColorWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~USlateColorWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to show and edit the color column in TEDS
USTRUCT()
struct FSlateColorWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FSlateColorWidgetConstructor();
	~FSlateColorWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};