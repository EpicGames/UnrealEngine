// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UrlWidget.h"

#include "Elements/Columns/TypedElementWebColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UrlWidget)

void UUrlWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FUrlWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FUrlColumn>());
}

FUrlWidgetConstructor::FUrlWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}



TSharedPtr<SWidget> FUrlWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);
	
	return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
					.Text(Binder.BindText(&UE::Editor::DataStorage::FUrlColumn::UrlString))
					.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
					.OnNavigate_Lambda([DataStorage, TargetRow] ()
					{
						if (UE::Editor::DataStorage::FUrlColumn* UrlColumn = DataStorage->GetColumn<UE::Editor::DataStorage::FUrlColumn>(TargetRow))
						{
							FPlatformProcess::LaunchURL(*UrlColumn->UrlString, nullptr, nullptr);
						}
					})
			];
}
