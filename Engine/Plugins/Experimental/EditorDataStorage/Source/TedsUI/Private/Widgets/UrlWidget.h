// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "UrlWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;
} // namespace UE::Editor::DataStorage

UCLASS()
class UUrlWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UUrlWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to display a URL in TEDS
USTRUCT()
struct FUrlWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FUrlWidgetConstructor();
	~FUrlWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};