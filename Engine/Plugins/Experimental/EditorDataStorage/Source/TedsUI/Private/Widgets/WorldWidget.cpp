// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/WorldWidget.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Engine/World.h"
#include "Internationalization/Text.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldWidget)

#define LOCTEXT_NAMESPACE "UWorldWidgetFactory"

void UWorldWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	                                                     UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetFactory<FWorldWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementWorldColumn>());
}

FWorldWidgetConstructor::FWorldWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FWorldWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);
	
	return SNew(STextBlock)
			.Text(Binder.BindData(&FTypedElementWorldColumn::World, [](const TWeakObjectPtr<UWorld>& WorldPtr)
				{
					if (UWorld* World = WorldPtr.Get())
					{
						FString WorldTypeString = LexToString(World->WorldType);
						
						return FText::Format(LOCTEXT("WorldName", "{0} ({1})"), FText::FromName(World->GetFName()),
							FText::FromString(WorldTypeString));
					}
					return FText::GetEmpty();
				}));
}

#undef LOCTEXT_NAMESPACE // "UWorldWidgetFactory"
