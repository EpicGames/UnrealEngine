// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "WorldWidget.generated.h"

class UScriptStruct;

UCLASS()
class UWorldWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UWorldWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to display the name and type of a UWorld
USTRUCT()
struct FWorldWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FWorldWidgetConstructor();
	~FWorldWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};