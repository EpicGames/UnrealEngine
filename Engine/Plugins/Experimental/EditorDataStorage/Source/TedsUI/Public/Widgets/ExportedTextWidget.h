// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ExportedTextWidget.generated.h"

#define UE_API TEDSUI_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

class UScriptStruct;

UCLASS(MinimalAPI)
class UExportedTextWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UExportedTextWidgetFactory() override = default;

	UE_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FExportedTextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FExportedTextWidgetConstructor();
	virtual ~FExportedTextWidgetConstructor() override = default;

	// To avoid name hiding since we are only overriding one of the GetQueryConditions overload
	using FTypedElementWidgetConstructor::GetQueryConditions;
	
	UE_API virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;
	UE_API virtual const UE::Editor::DataStorage::Queries::FConditions* GetQueryConditions(const UE::Editor::DataStorage::ICoreProvider* Storage) const override;

	UE_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const override;
	
	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
	// The column this exported text widget is operating on
	UE::Editor::DataStorage::Queries::FConditions MatchedColumn;

};

USTRUCT(meta = (DisplayName = "Exported text widget"))
struct FExportedTextWidgetTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

#undef UE_API
