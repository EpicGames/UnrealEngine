// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "GeneralWidgetRegistrationFactory.generated.h"

#define UE_API TEDSUI_API

UCLASS(MinimalAPI)
class UGeneralWidgetRegistrationFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	static UE_API const FName LargeCellPurpose;
	static UE_API const FName HeaderPurpose;

	~UGeneralWidgetRegistrationFactory() override = default;

	UE_API void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

#undef UE_API
