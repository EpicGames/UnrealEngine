// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "LabelWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

class UScriptStruct;

UCLASS()
class ULabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~ULabelWidgetFactory() override = default;

	TEDSUI_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	TEDSUI_API virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSUI_API FLabelWidgetConstructor();
	~FLabelWidgetConstructor() override = default;

	TEDSUI_API virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;

	TEDSUI_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};