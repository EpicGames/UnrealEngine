// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "PackagePathWidget.generated.h"

#define UE_API TEDSUI_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

class SWidget;
class UScriptStruct;

UCLASS(MinimalAPI)
class UPackagePathWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UPackagePathWidgetFactory() override = default;

	UE_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FPackagePathWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FPackagePathWidgetConstructor();
	~FPackagePathWidgetConstructor() override = default;

protected:
	UE_API explicit FPackagePathWidgetConstructor(const UScriptStruct* InTypeInfo);

	UE_API TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	UE_API bool FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget) override;
};

USTRUCT()
struct FLoadedPackagePathWidgetConstructor : public FPackagePathWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FLoadedPackagePathWidgetConstructor();
	~FLoadedPackagePathWidgetConstructor() override = default;

protected:
	UE_API bool FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget) override;
};

#undef UE_API
