// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsUI : ModuleRules
{
	public TedsUI(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"TypedElementFramework",
					"TedsCore"
				});
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ToolWidgets",
					"AppFramework",
					"InputCore"
				});

			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}
	}
}
